[CoreRedirects]
; Traits rename
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendByBoolDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendByBoolTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendInertializerDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendInertializerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendSmootherDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendSmootherTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendSmootherPerBoneDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendProfilePerChildProviderTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendTwoWayDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendTwoWayTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextReferencePoseDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextReferencePoseTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSequencePlayerDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextSequencePlayerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSubGraphHostDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextSubGraphHostTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSynchronizeUsingGroupsDecoratorSharedData", NewName="/Script/UAFAnimGraph.AnimNextSynchronizeUsingGroupsTraitSharedData")

; Plugin refactor
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimGraphNode_AnimNextGraph", NewName="/Script/UAFAnimGraphUncookedOnly.AnimGraphNode_AnimNextGraph")

+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendByBoolTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendByBoolTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendInertializerTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendInertializerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendSmootherTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendSmootherTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendSmootherPerBoneTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendProfilePerChildProviderTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextBlendTwoWayTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendTwoWayTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextReferencePoseTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextReferencePoseTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSequencePlayerTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextSequencePlayerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSubGraphHostTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextSubGraphHostTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSynchronizeUsingGroupsTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextSynchronizeUsingGroupsTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNode_AnimNextGraph", NewName="/Script/UAFAnimGraph.AnimNode_AnimNextGraph")

; AnimNextGraph -> AnimNextModule
+PropertyRedirects=(OldName="/Script/AnimNextGraph.AnimNode_AnimNextGraph.AnimNextGraph", NewName="Module")

; Rename K2 Node
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.K2Node_PlayAnim", NewName="/Script/UAFAnimGraphUncookedOnly.K2Node_AnimNextPlayAnim")

; Rename PlayAnim parameter
+PropertyRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.CreateProxyObjectForPlayAnim.AnimationObject", NewName="AnimSequence")
+PropertyRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.K2Node_PlayAnim.AnimationObject", NewName="AnimSequence")

; PlayAnim -> Injection
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextPlayAnimBlendMode", NewName="/Script/UAFAnimGraph.EAnimNextInjectionBlendMode")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextPlayAnimStatus", NewName="/Script/UAFAnimGraph.EAnimNextInjectionStatus")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextPlayAnimBlendSettings", NewName="/Script/UAFAnimGraph.AnimNextInjectionBlendSettings")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextPlayAnimRequestArgs", NewName="/Script/UAFAnimGraph.AnimNextInjectionRequestArgs")
+PropertyRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInjectionRequestArgs.SlotName", NewName="Name")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.K2Node_AnimNextPlayAsset", NewName="/Script/UAFAnimGraphUncookedOnly.K2Node_AnimNextInjection")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_AnimNextRunAnimationGraph", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextRunAnimationGraph_v1")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphInstancePtr", NewName="/Script/UAFAnimGraph.AnimNextAnimGraph")

; AnimNextAnimGraph plugin -> UAFAnimGraph plugin
+PackageRedirects=(OldName="/Script/AnimNextAnimGraph", NewName="/Script/UAFAnimGraph")
+PackageRedirects=(OldName="/Script/AnimNextAnimGraphEditor", NewName="/Script/UAFAnimGraphEditor")
+PackageRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly", NewName="/Script/UAFAnimGraphUncookedOnly")
+PackageRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite", NewName="/Script/UAFAnimGraphTestSuite")

+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNode_AnimNextGraph", NewName="/Script/UAFAnimGraph.AnimNode_AnimNextGraph")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphContextData", NewName="/Script/UAFAnimGraph.AnimNextGraphContextData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphLatentPropertiesContextData", NewName="/Script/UAFAnimGraph.AnimNextGraphLatentPropertiesContextData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextModuleAnimGraphComponent", NewName="/Script/UAFAnimGraph.AnimNextModuleAnimGraphComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigDecorator_AnimNextCppDecorator", NewName="/Script/UAFAnimGraph.RigDecorator_AnimNextCppDecorator")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_AnimNextGraphRoot", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextGraphRoot")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextRunGraphWorkData_v1", NewName="/Script/UAFAnimGraph.AnimNextRunGraphWorkData_v1")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_AnimNextRunAnimationGraph_v1", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextRunAnimationGraph_v1")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextRunGraphWorkData_v2", NewName="/Script/UAFAnimGraph.AnimNextRunGraphWorkData_v2")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_AnimNextShimRoot", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextShimRoot")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_AnimNextTraitStack", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextTraitStack")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendStackCoreTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendStackCoreTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendStackTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendStackTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendStackRequesterTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendStackRequesterTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextCallFunctionSharedData", NewName="/Script/UAFAnimGraph.AnimNextCallFunctionSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.GCGraphInstanceComponent", NewName="/Script/UAFAnimGraph.GCGraphInstanceComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_AnimNextSetNotifyContext", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextSetNotifyContext")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_GetPostProcessAnimation", NewName="/Script/UAFAnimGraph.RigUnit_GetPostProcessAnimation")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.RigUnit_HasValidEvaluationModifier", NewName="/Script/UAFAnimGraph.RigUnit_HasValidEvaluationModifier")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.SyncGroupGraphInstanceComponent", NewName="/Script/UAFAnimGraph.SyncGroupGraphInstanceComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNativeDataInterface_AnimSequencePlayer", NewName="/Script/UAFAnimGraph.AnimNextNativeDataInterface_AnimSequencePlayer")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNativeDataInterface_BlendSpacePlayer", NewName="/Script/UAFAnimGraph.AnimNextNativeDataInterface_BlendSpacePlayer")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNativeDataInterface_SynchronizeUsingGroups", NewName="/Script/UAFAnimGraph.AnimNextNativeDataInterface_SynchronizeUsingGroups")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextModuleInjectionComponent", NewName="/Script/UAFAnimGraph.AnimNextModuleInjectionComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.GraphInstanceInjectionComponent", NewName="/Script/UAFAnimGraph.GraphInstanceInjectionComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInjectionSiteTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextInjectionSiteTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextPlayAnimSlotTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextPlayAnimSlotTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextApplyAdditiveTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextApplyAdditiveTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.BoolAlphaInputTraitArgs", NewName="/Script/UAFAnimGraph.BoolAlphaInputTraitArgs")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.CurveAlphaInputTraitArgs", NewName="/Script/UAFAnimGraph.CurveAlphaInputTraitArgs")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.FloatAlphaInputTraitArgs", NewName="/Script/UAFAnimGraph.FloatAlphaInputTraitArgs")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.SimpleFloatAlphaInputTraitArgs", NewName="/Script/UAFAnimGraph.SimpleFloatAlphaInputTraitArgs")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendByBoolTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendByBoolTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendInertializerCoreTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendInertializerCoreTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendInertializerTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendInertializerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendLayerTraitSharedData", NewName="/Script/UAFAnimGraph.UAFBlendLayerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendSmootherPerBoneCoreTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendSmootherPerBoneCoreTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendProfilePerChildProviderTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendProfilePerChildProviderTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendProfileProviderTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendProfileProviderTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendTwoWayTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendTwoWayTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInertializationDeadBlendTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextInertializationDeadBlendTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInputPoseTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextInputPoseTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.ModifyCurveParameters", NewName="/Script/UAFAnimGraph.ModifyCurveParameters")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextModifyCurveTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextModifyCurveTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.ModifyCurveTask", NewName="/Script/UAFAnimGraph.ModifyCurveTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNotifyDispatcherComponent", NewName="/Script/UAFAnimGraph.AnimNextNotifyDispatcherComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNotifyFilterTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextNotifyFilterTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextPassthroughSharedData", NewName="/Script/UAFAnimGraph.AnimNextPassthroughSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextReferencePoseTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextReferencePoseTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSubGraphHostTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextSubGraphHostTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextEvaluationTask", NewName="/Script/UAFAnimGraph.AnimNextEvaluationTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.SerializableEvaluationProgram", NewName="/Script/UAFAnimGraph.SerializableEvaluationProgram")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextApplyAdditiveKeyframeTask", NewName="/Script/UAFAnimGraph.AnimNextApplyAdditiveKeyframeTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendTwoKeyframesTask", NewName="/Script/UAFAnimGraph.AnimNextBlendTwoKeyframesTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendOverwriteKeyframeWithScaleTask", NewName="/Script/UAFAnimGraph.AnimNextBlendOverwriteKeyframeWithScaleTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendAddKeyframeWithScaleTask", NewName="/Script/UAFAnimGraph.AnimNextBlendAddKeyframeWithScaleTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendOverwriteKeyframePerBoneWithScaleTask", NewName="/Script/UAFAnimGraph.AnimNextBlendOverwriteKeyframePerBoneWithScaleTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendAddKeyframePerBoneWithScaleTask", NewName="/Script/UAFAnimGraph.AnimNextBlendAddKeyframePerBoneWithScaleTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendKeyframePerBoneWithScaleTask", NewName="/Script/UAFAnimGraph.AnimNextBlendKeyframePerBoneWithScaleTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextConvertRotationsToLocalSpaceTask", NewName="/Script/UAFAnimGraph.AnimNextConvertRotationsToLocalSpaceTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextConvertRotationsToMeshSpaceTask", NewName="/Script/UAFAnimGraph.AnimNextConvertRotationsToMeshSpaceTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextDeadBlendingTransitionTask", NewName="/Script/UAFAnimGraph.AnimNextDeadBlendingTransitionTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextDeadBlendingApplyTask", NewName="/Script/UAFAnimGraph.AnimNextDeadBlendingApplyTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextExecuteProgramTask", NewName="/Script/UAFAnimGraph.AnimNextExecuteProgramTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNormalizeKeyframeRotationsTask", NewName="/Script/UAFAnimGraph.AnimNextNormalizeKeyframeRotationsTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextAnimSequenceKeyframeTask", NewName="/Script/UAFAnimGraph.AnimNextAnimSequenceKeyframeTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextPushPoseTask", NewName="/Script/UAFAnimGraph.AnimNextPushPoseTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextPushReferenceKeyframeTask", NewName="/Script/UAFAnimGraph.AnimNextPushReferenceKeyframeTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSwapTransformsTask", NewName="/Script/UAFAnimGraph.AnimNextSwapTransformsTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextStoreKeyframeTransformsTask", NewName="/Script/UAFAnimGraph.AnimNextStoreKeyframeTransformsTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextDuplicateTopKeyframeTask", NewName="/Script/UAFAnimGraph.AnimNextDuplicateTopKeyframeTask")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextAnimGraphBuilder", NewName="/Script/UAFAnimGraph.AnimNextAnimGraphBuilder")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextFactoryParams", NewName="/Script/UAFAnimGraph.AnimNextFactoryParams")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSimpleAnimGraphBuilderTraitStackDesc", NewName="/Script/UAFAnimGraph.AnimNextSimpleAnimGraphBuilderTraitStackDesc")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSimpleAnimGraphBuilder", NewName="/Script/UAFAnimGraph.AnimNextSimpleAnimGraphBuilder")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphInjectionData", NewName="/Script/UAFAnimGraph.AnimNextGraphInjectionData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextAnimGraph", NewName="/Script/UAFAnimGraph.AnimNextAnimGraph")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphEntryPoint", NewName="/Script/UAFAnimGraph.AnimNextGraphEntryPoint")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphEvaluatorExecuteArgument", NewName="/Script/UAFAnimGraph.AnimNextGraphEvaluatorExecuteArgument")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphEvaluatorExecuteDefinition", NewName="/Script/UAFAnimGraph.AnimNextGraphEvaluatorExecuteDefinition")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextGraphInstance", NewName="/Script/UAFAnimGraph.AnimNextGraphInstance")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.UAFGraphInstanceComponent", NewName="/Script/UAFAnimGraph.UAFGraphInstanceComponent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInjectionBlendSettings", NewName="/Script/UAFAnimGraph.AnimNextInjectionBlendSettings")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInjectionRequestArgs", NewName="/Script/UAFAnimGraph.AnimNextInjectionRequestArgs")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInjectionSite", NewName="/Script/UAFAnimGraph.AnimNextInjectionSite")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextEntryPointHandle", NewName="/Script/UAFAnimGraph.AnimNextEntryPointHandle")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextTraitHandle", NewName="/Script/UAFAnimGraph.AnimNextTraitHandle")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AlphaInputTraitArgs", NewName="/Script/UAFAnimGraph.AlphaInputTraitArgs")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendSmootherCoreTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendSmootherCoreTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendSmootherTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendSmootherTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextBlendSpacePlayerTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextBlendSpacePlayerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextInertializationRequestEvent", NewName="/Script/UAFAnimGraph.AnimNextInertializationRequestEvent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextNotifyDispatcherTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextNotifyDispatcherTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.PassthroughBlendTraitSharedData", NewName="/Script/UAFAnimGraph.PassthroughBlendTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSequencePlayerTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextSequencePlayerTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSynchronizeUsingGroupsTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextSynchronizeUsingGroupsTraitSharedData")

+ClassRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextAnimGraphSettings", NewName="/Script/UAFAnimGraph.AnimNextAnimGraphSettings")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraph.PlayAnimCallbackProxy", NewName="/Script/UAFAnimGraph.PlayAnimCallbackProxy")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextSkeletalMeshComponentLibrary", NewName="/Script/UAFAnimGraph.AnimNextSkeletalMeshComponentLibrary")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraph.AnimNextAnimationGraph", NewName="/Script/UAFAnimGraph.UAFAnimGraph")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraph.PostProcessAnimationUserAssetData", NewName="/Script/UAFAnimGraph.PostProcessAnimationUserAssetData")

+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EUninjectionResult", NewName="/Script/UAFAnimGraph.EUninjectionResult")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextCallFunctionCallSite", NewName="/Script/UAFAnimGraph.EAnimNextCallFunctionCallSite")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNext_ModifyCurveApplyMode", NewName="/Script/UAFAnimGraph.EAnimNext_ModifyCurveApplyMode")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextReferencePoseType", NewName="/Script/UAFAnimGraph.EAnimNextReferencePoseType")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextInjectionBlendMode", NewName="/Script/UAFAnimGraph.EAnimNextInjectionBlendMode")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextInjectionType", NewName="/Script/UAFAnimGraph.EAnimNextInjectionType")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextInjectionLifetimeType", NewName="/Script/UAFAnimGraph.EAnimNextInjectionLifetimeType")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimNextInjectionStatus", NewName="/Script/UAFAnimGraph.EAnimNextInjectionStatus")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimGroupSynchronizationRole", NewName="/Script/UAFAnimGraph.EAnimGroupSynchronizationRole")
+EnumRedirects=(OldName="/Script/AnimNextAnimGraph.EAnimGroupSynchronizationMode", NewName="/Script/UAFAnimGraph.EAnimGroupSynchronizationMode")

+StructRedirects=(OldName="/Script/AnimNextAnimGraphEditor.AnimNextSyncMarkerTraceInfo", NewName="/Script/UAFAnimGraphEditor.AnimNextSyncMarkerTraceInfo")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphEditor.AnimNextSequenceTraceInfo", NewName="/Script/UAFAnimGraphEditor.AnimNextSequenceTraceInfo")

+ClassRedirects=(OldName="/Script/AnimNextAnimGraphEditor.AnimNextAnimationGraphFactory", NewName="/Script/UAFAnimGraphEditor.UAFAnimGraphFactory")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphEditor.AssetDefinition_AnimNextAnimationGraphEntry", NewName="/Script/UAFAnimGraphEditor.AssetDefinition_AnimNextAnimationGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphEditor.AssetDefinition_AnimNextAnimationGraph", NewName="/Script/UAFAnimGraphEditor.AssetDefinition_AnimNextAnimationGraph")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphEditor.EvaluationProgramDetailsObject", NewName="/Script/UAFAnimGraphEditor.EvaluationProgramDetailsObject")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphEditor.SequenceInfoDetailsObject", NewName="/Script/UAFAnimGraphEditor.SequenceInfoDetailsObject")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphEditor.SequenceInfoDetailsObject", NewName="/Script/UAFAnimGraphEditor.SequenceInfoDetailsObject")

+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TestTraitSharedData", NewName="/Script/UAFAnimGraphTestSuite.TestTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TestDerivedVector", NewName="/Script/UAFAnimGraphTestSuite.TestDerivedVector")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitWithOneChildSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitWithOneChildSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitWithChildrenSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitWithChildrenSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitA_BaseSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitA_BaseSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitAB_AddSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitAB_AddSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitAC_AddSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitAC_AddSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitSerialization_BaseSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitSerialization_BaseSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitSerialization_AddSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitSerialization_AddSharedData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphTestSuite.TraitNativeSerialization_AddSharedData", NewName="/Script/UAFAnimGraphTestSuite.TraitNativeSerialization_AddSharedData")

+StructRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextAnimationGraphOutlinerData", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextSchemaAction_AddManifestNode", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextSchemaAction_AddManifestNode")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextSchemaAction_NotifyEvent", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextSchemaAction_NotifyEvent")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextBaseAction", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextBaseAction")
+StructRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextManifestAction", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextManifestAction")

+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextAnimGraphWorkspaceAssetUserData", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimGraphWorkspaceAssetUserData")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextController", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextController")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextTraitStackUnitNode", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextTraitStackUnitNode")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextAnimationGraphEntry", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimGraphNode_AnimNextGraph", NewName="/Script/UAFAnimGraphUncookedOnly.AnimGraphNode_AnimNextGraph")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextAnimationGraphSchema", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextAnimationGraphLibrary", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphLibrary")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.AnimNextAnimationGraph_EditorData", NewName="/Script/UAFAnimGraphUncookedOnly.UAFAnimGraph_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.K2Node_AnimNextInjection", NewName="/Script/UAFAnimGraphUncookedOnly.K2Node_AnimNextInjection")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.K2Node_AnimNextPlayAnim", NewName="/Script/UAFAnimGraphUncookedOnly.K2Node_AnimNextPlayAnim")
+ClassRedirects=(OldName="/Script/AnimNextAnimGraphUncookedOnly.K2Node_InjectionCallbackProxySetVariable", NewName="/Script/UAFAnimGraphUncookedOnly.K2Node_InjectionCallbackProxySetVariable")

; AnimNext -> UAF
+ClassRedirects=(OldName="/Script/UAFAnimGraph.AnimNextAnimationGraph",NewName="/Script/UAFAnimGraph.UAFAnimGraph")
+ClassRedirects=(OldName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraph_EditorData",NewName="/Script/UAFAnimGraphUncookedOnly.UAFAnimGraph_EditorData")
+ClassRedirects=(OldName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphFactory",NewName="/Script/UAFAnimGraphUncookedOnly.UAFAnimGraphFactory")
+StructRedirects=(OldName="/Script/UAFAnimGraph.AnimNextBlendLayerTraitSharedData",NewName="/Script/UAFAnimGraph.UAFBlendLayerTraitSharedData")
