// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/AnimNextAnimationGraph.h"

#include "AnimNextAnimGraphSettings.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FInstancedPropertyBag;
class UUAFAnimGraph;
class UAnimNextAnimGraphSettings;
struct FAnimNextFactoryParams;

UCLASS(MinimalAPI, Config=AnimNextAnimGraph, defaultconfig)
class UAnimNextAnimGraphSettings : public UObject
{
	GENERATED_BODY()

public:
	// Gets all allowed asset classes that users can reference that map via GetGraphFromObject
	UFUNCTION()
	static UE_API TArray<UClass*> GetAllowedAssetClasses();
};

#undef UE_API
