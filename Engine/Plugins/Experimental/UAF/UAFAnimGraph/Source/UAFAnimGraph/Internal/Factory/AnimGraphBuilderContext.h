// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "TraitCore/EntryPointHandle.h"
#include "TraitCore/TraitWriter.h"

class IRigVMRuntimeAssetInterface;
class UUAFRigVMAsset;
class UUAFAnimGraph;
class FAnimationAnimNextRuntimeTest_SimpleGraphBuilder;

namespace UE::UAF
{
	struct FAnimGraphFactory;
}

namespace UE::UAF
{

struct FAnimGraphBuilderContext
{
private:
	friend FAnimGraphFactory;
	friend FAnimationAnimNextRuntimeTest_SimpleGraphBuilder;

	// Builds a graph using the populated TraitWriter and VariableStructs
	UAFANIMGRAPH_API const UUAFAnimGraph* Build();

public:
	// Root trait handle
	FAnimNextEntryPointHandle RootTraitHandle;

	// Trait writer used to build the graph
	FTraitWriter TraitWriter;

	// Variable structs used to communicate with the graph
	TArray<const UScriptStruct*> VariableStructs;

	// Component structs used to communicate with the graph instance
	TArray<const UScriptStruct*> ComponentStructs;

	// All the IRigVMRuntimeAssetInterface whose variables we will reference/instance at runtime
	TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedVariableRigVMAssets;

	// All the assets whose variables we will reference/instance at runtime
	TSet<TObjectPtr<const UUAFRigVMAsset>> ReferencedVariableAssets;
};

}