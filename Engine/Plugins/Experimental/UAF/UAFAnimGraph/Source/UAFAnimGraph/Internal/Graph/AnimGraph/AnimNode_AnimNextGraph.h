// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNode_CustomProperty.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "AnimNode_AnimNextGraph.generated.h"

#define UE_API UAFANIMGRAPH_API

/**
 * Animation node that allows a AnimNextGraph output to be used in an animation graph
 */
USTRUCT()
struct FAnimNode_AnimNextGraph : public FAnimNode_CustomProperty
{
	GENERATED_BODY()

	UE_API FAnimNode_AnimNextGraph();

	// FAnimNode_Base interface
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext & Output) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }

	UE_API virtual void InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass) override;
	UE_API virtual void PropagateInputProperties(const UObject* InSourceInstance) override;

private:
#if WITH_EDITOR
	UE_API virtual void HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
#endif
private:

	/** The input pose we will pass to the graph */
	UPROPERTY(EditAnywhere, Category = Links, meta = (DisplayName = "Source"))
	FPoseLink SourceLink;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UUAFAnimGraph> AnimationGraph;

	// Our graph instance, we own it
	TSharedPtr<FAnimNextGraphInstance> GraphInstance;

	/*
	 * Max LOD that this node is allowed to run
	 * For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	 * when the component LOD becomes 3, it will stop update/evaluate
	 * currently transition would be issue and that has to be re-visited
	 */
	UPROPERTY(EditAnywhere, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold;

protected:
	virtual UClass* GetTargetClass() const override { return AnimationGraph ? AnimationGraph->StaticClass() : nullptr; }
	
public:

	UE_API void PostSerialize(const FArchive& Ar);
	UE_API void AddStructReferencedObjects(class FReferenceCollector& Collector);

	friend class UAnimGraphNode_AnimNextGraph;
};

template<>
struct TStructOpsTypeTraits<FAnimNode_AnimNextGraph> : public TStructOpsTypeTraitsBase2<FAnimNode_AnimNextGraph>
{
	enum
	{
		WithCopy = false,
		WithPostSerialize = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
