// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Module/AnimNextModuleContextData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "AnimNextGraphContextData.generated.h"

namespace UE::UAF
{
	struct FLatentPropertyHandle;
}

USTRUCT()
struct FAnimNextGraphContextData : public FUAFAssetContextData
{
	GENERATED_BODY()

	FAnimNextGraphContextData() = default;

	FAnimNextGraphContextData(FAnimNextGraphInstance& InInstance, FName InEventName, float InDeltaTime)
		: FUAFAssetContextData(InInstance, InEventName, InDeltaTime)
	{
	}

	const FAnimNextGraphInstance& GetGraphInstance() const { return static_cast<const FAnimNextGraphInstance&>(GetInstance()); }

private:
	friend struct FAnimNextGraphInstance;
	friend struct FAnimNextExecuteContext;
};
