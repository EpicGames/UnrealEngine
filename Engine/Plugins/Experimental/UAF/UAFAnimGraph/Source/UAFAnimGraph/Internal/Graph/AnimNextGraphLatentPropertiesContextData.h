// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextGraphContextData.h"
#include "RigUnit_AnimNextShimRoot.h"
#include "Module/AnimNextModuleContextData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "AnimNextGraphLatentPropertiesContextData.generated.h"

namespace UE::UAF
{
	struct FLatentPropertyHandle;
}

USTRUCT()
struct FAnimNextGraphLatentPropertiesContextData : public FAnimNextGraphContextData
{
	GENERATED_BODY()

	FAnimNextGraphLatentPropertiesContextData() = default;

	FAnimNextGraphLatentPropertiesContextData(FAnimNextGraphInstance& InInstance, const TConstArrayView<UE::UAF::FLatentPropertyHandle>& InLatentHandles, void* InDestinationBasePtr, float InDeltaTime, bool bInIsFrozen, bool bInJustBecameRelevant)
		: FAnimNextGraphContextData(InInstance, FRigUnit_AnimNextShimRoot::EventName, InDeltaTime)
		, LatentHandles(InLatentHandles)
		, DestinationBasePtr(InDestinationBasePtr)
		, bIsFrozen(bInIsFrozen)
		, bJustBecameRelevant(bInJustBecameRelevant)
	{
	}

	const TConstArrayView<UE::UAF::FLatentPropertyHandle>& GetLatentHandles() const { return LatentHandles; }
	void* GetDestinationBasePtr() const { return DestinationBasePtr; }
	bool IsFrozen() const { return bIsFrozen; }
	bool JustBecameRelevant() const { return bJustBecameRelevant; }

private:
	TConstArrayView<UE::UAF::FLatentPropertyHandle> LatentHandles;
	void* DestinationBasePtr = nullptr;
	bool bIsFrozen = false;
	bool bJustBecameRelevant = false;

	friend struct FAnimNextGraphInstance;
	friend struct FAnimNextExecuteContext;
};
