// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFAssetInstance.h"
#include "Injection/InjectionInfo.h"
#include "Module/UAFModuleInstanceComponent.h"
#include "AnimNextModuleAnimGraphComponent.generated.h"

struct FAnimNextGraphInstance;
class UUAFAnimGraph;

namespace UE::UAF
{
	struct FVariableOverrides;
	struct FGraphAllocationParams;
}

// Module component that owns/allocates/release all animation graph instances on a module
USTRUCT()
struct FAnimNextModuleAnimGraphComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	TWeakPtr<FAnimNextGraphInstance> AllocateInstance(const UUAFAnimGraph* InAnimationGraph, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides);

	void ReleaseInstance(TWeakPtr<FAnimNextGraphInstance> InInstance);

	void AddStructReferencedObjects(class FReferenceCollector& Collector);

private:
	// All the owned graph instances for the module
	TArray<TSharedPtr<FAnimNextGraphInstance>> GraphInstances;
};

template<>
struct TStructOpsTypeTraits<FAnimNextModuleAnimGraphComponent> : public TStructOpsTypeTraitsBase2<FAnimNextModuleAnimGraphComponent>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};
