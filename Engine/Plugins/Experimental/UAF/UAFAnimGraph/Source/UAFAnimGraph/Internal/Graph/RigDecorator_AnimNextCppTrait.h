// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMTrait.h"

#include "RigDecorator_AnimNextCppTrait.generated.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF { struct FTrait; }

/**
 * AnimNext RigDecorator for all C++ traits.
 * The trait shared data UScriptStruct determines which properties are exposed.
 */
USTRUCT(BlueprintType)
struct FRigDecorator_AnimNextCppDecorator : public FRigVMTrait
{
	GENERATED_BODY()

	// The struct the trait exposes with its shared data. Each one of its properties will be added as a pin.
	UPROPERTY(meta = (Hidden))
	TObjectPtr<UScriptStruct> DecoratorSharedDataStruct = nullptr;

#if WITH_EDITOR
	UE_API virtual void GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, struct FRigVMPinInfoArray& OutPinArray) const override;

	virtual UScriptStruct* GetTraitSharedDataStruct() const override
	{
		return DecoratorSharedDataStruct;
	}

	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	UE_API const UE::UAF::FTrait* GetTrait() const;
#endif
};

#undef UE_API
