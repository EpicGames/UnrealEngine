// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "AnimNextExecuteContext.h"
#include "RigUnit_AnimNextGraphEvaluator.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FAnimNextGraphEvaluatorExecuteDefinition;

/**
 * Animation graph evaluator
 * This node is only used at runtime.
 * It performs the animation graph update and evaluation through the data provided in the execution context.
 * It also holds all latent/lazy pins that the graph references in the editor.
 */
USTRUCT(meta=(Hidden, DisplayName="Animation Runtime Output", Category="Events", NodeColor="1, 0, 0"))
struct FRigUnit_AnimNextGraphEvaluator : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	// The execute context used for the execution flow
	UPROPERTY(meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;

	static UE_API void StaticExecute(FRigVMExtendedExecuteContext& RigVMExecuteContext, FRigVMMemoryHandleArray RigVMMemoryHandles, FRigVMPredicateBranchArray RigVMBranches);

	// The graph evaluator entry point is dynamically registered because it can have any number of latent pins with their own name/type
	// RigVM requires a mapping between argument list and method name and so we register things manually
	static UE_API void RegisterExecuteMethod(const FAnimNextGraphEvaluatorExecuteDefinition& ExecuteDefinition);

	// Returns an existing execute method, if found. Otherwise, nullptr is returned.
	static UE_API const FAnimNextGraphEvaluatorExecuteDefinition* FindExecuteMethod(uint32 ExecuteMethodHash);
};

#undef UE_API
