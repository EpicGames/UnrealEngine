// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "TraitCore/TraitHandle.h"
#include "RigUnit_AnimNextGraphRoot.generated.h"

#define UE_API UAFANIMGRAPH_API

/**
 * Animation graph output
 * This is a synthetic node that represents the entry point for an animation graph for RigVM.
 * The graph editor will see this as the graph output in which to hook up the first animation node
 * to evaluate.
 * This node isn't used at runtime.
 */
USTRUCT(meta=(Hidden, DisplayName="Animation Output", Category="Events", NodeColor="1, 0, 0", Keywords="Root,Output"))
struct FRigUnit_AnimNextGraphRoot : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void DummyExecute();

	virtual FName GetEventName() const override { return EventName; }
	virtual FString GetUnitSubTitle() const override { return EntryPoint.ToString(); };
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Category = Result, meta = (Input, HideSubPins))
	FAnimNextTraitHandle Result;

	// In order for this node to be considered an executable RigUnit, it needs a pin to derive from FRigVMExecuteContext
	// We keep it hidden it since we don't need it
	UPROPERTY()
	FAnimNextExecuteContext ExecuteContext;

	// The name of the entry point
	UPROPERTY(VisibleAnywhere, Category = Result, meta = (Hidden))
	FName EntryPoint = DefaultEntryPoint;

	// This unit is our graph entry point, it needs an event so we can call it
	static UE_API FName EventName;

	// Default entry point name
	static UE_API FName DefaultEntryPoint;
};

#undef UE_API
