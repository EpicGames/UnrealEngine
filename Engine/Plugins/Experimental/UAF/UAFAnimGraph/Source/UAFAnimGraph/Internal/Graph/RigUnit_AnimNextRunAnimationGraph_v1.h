// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Variables/AnimNextVariableOverridesCollection.h"

#include "RigUnit_AnimNextRunAnimationGraph_v1.generated.h"

#define UE_API UAFANIMGRAPH_API

USTRUCT()
struct FAnimNextRunGraphWorkData_v1
{
	GENERATED_BODY()

	// Weak ptr to the instance we run (ownership is with the module component)
	TWeakPtr<FAnimNextGraphInstance> WeakInstance;
};

/** Runs an animation graph */
USTRUCT(meta=(Hidden, DisplayName="Run Graph (Deprecated)", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Trait,Stack"))
struct FRigUnit_AnimNextRunAnimationGraph_v1 : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// Graph to run
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input, ExportAsReference="true"))
	TObjectPtr<UUAFAnimGraph> Graph;

	// Instance used to hold graph state
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input, Output))
	FAnimNextAnimGraph Instance;

	// LOD to run the graph at. If this is -1 then the reference pose's source LOD will be used
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	int32 LOD = -1;

	// Reference pose for the graph
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FAnimNextGraphReferencePose ReferencePose;

	// Variable overrides to be applied to subgraphs
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FAnimNextVariableOverridesCollection Overrides;

	// Pose result
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	FAnimNextGraphLODPose Result;

	UPROPERTY(Transient)
	FAnimNextRunGraphWorkData_v1 WorkData;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API
