// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Variables/AnimNextVariableOverridesCollection.h"

#include "RigUnit_AnimNextRunAnimationGraph_v2.generated.h"

#define UE_API UAFANIMGRAPH_API

USTRUCT()
struct FAnimNextRunGraphWorkData_v2
{
	GENERATED_BODY()

	// Weak ptr to the instance we run (ownership is with the module component)
	TWeakPtr<FAnimNextGraphInstance> WeakHost;

	// Graph we are injecting into the host
	UPROPERTY(transient)
	FAnimNextVariableReference InjectedGraphReference;
};

/** Runs an animation graph */
USTRUCT(meta=(DisplayName="Run Graph", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Trait,Stack", RequiredComponents="AnimNextModuleInjectionComponent"))
struct FRigUnit_AnimNextRunAnimationGraph_v2 : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	static UE_API const UUAFAnimGraph* GetHostGraphToRun(FAnimNextExecuteContext& InExecuteContext);

	// The graph to run
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input, ShowOnlyInnerProperties, ExportAsReference="true"))
	FAnimNextAnimGraph Graph;

	// LOD to run the graph at. If this is -1 then the reference pose's source LOD will be used
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	int32 LOD = -1;

	// Reference pose for the graph
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FAnimNextGraphReferencePose ReferencePose;

	// Variable overrides to be applied to subgraphs
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FAnimNextVariableOverridesCollection Overrides;

	// The name of the attribute set to execute with, if 'None' will include everything
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FName AttributeSet;

	// Pose result
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Output))
	FAnimNextGraphLODPose Result;

	// Internal work data
	UPROPERTY(transient)
	FAnimNextRunGraphWorkData_v2 WorkData;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
