// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "AnimNextExecuteContext.h"
#include "RigUnit_AnimNextShimRoot.generated.h"

#define UE_API UAFANIMGRAPH_API

/**
 * Animation graph output
 * This is a synthetic node that represents the entry point for an animation graph for RigVM.
 * The graph editor will not see this node. It is added during compilation as a shim to start the evaluation process.
 * This node is only used at runtime.
 */
USTRUCT(meta=(Hidden, DisplayName="Animation Output Shim", Category="Events", NodeColor="1, 0, 0", Keywords="Root,Output"))
struct FRigUnit_AnimNextShimRoot : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// In order for this node to be considered an executable RigUnit, it needs a pin to derive from FRigVMExecuteContext
	// Although the anim graph flows from left to right with the graph output being on the right hand side with a single
	// input as the final output, RigVM execution flows from outputs to inputs and as such an entry point has a single output.
	UPROPERTY(meta = (Output))
	FAnimNextExecuteContext ExecuteContext;

	// This unit is our graph entry point, it needs an event so we can call it
	static UE_API FName EventName;
};

#undef UE_API
