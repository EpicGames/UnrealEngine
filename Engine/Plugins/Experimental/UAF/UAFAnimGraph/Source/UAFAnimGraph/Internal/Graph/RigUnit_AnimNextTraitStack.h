// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "TraitCore/TraitHandle.h"

#include "RigUnit_AnimNextTraitStack.generated.h"

#define UE_API UAFANIMGRAPH_API

/**
 * Animation graph trait container
 * It contains a stack of traits which are used during compilation to output a cooked graph
 * This node is synthetic and only present in the editor. During compilation, it is disconnected from
 * the graph and replaced by the cooked trait metadata.
 */
USTRUCT(meta=(DisplayName="Trait Stack", Category="Animation", NodeColor="0, 1, 1", Keywords="Trait,Stack"))
struct FRigUnit_AnimNextTraitStack : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// The execution result
	UPROPERTY(EditAnywhere, Category = Result, meta = (Output))
	FAnimNextTraitHandle Result;
};

#undef UE_API
