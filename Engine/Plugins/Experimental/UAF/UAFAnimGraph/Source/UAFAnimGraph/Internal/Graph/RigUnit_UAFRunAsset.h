// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "Variables/AnimNextVariableOverridesCollection.h"

#include "RigUnit_UAFRunAsset.generated.h"

#define UE_API UAFANIMGRAPH_API

USTRUCT()
struct FUAFRunAssetWorkData
{
	GENERATED_BODY()

	// Weak ptr to the instance we run (ownership is with the module component)
	TWeakPtr<FAnimNextGraphInstance> WeakHost;

	// Graph we are injecting into the host
	UPROPERTY(transient)
	FAnimNextVariableReference InjectedGraphReference;
};

/** Runs a UAF asset to generate an output value */
USTRUCT(meta=(DisplayName="Run Asset", Category="Anim", NodeColor="0, 1, 1", Keywords="Trait,Stack", RequiredComponents="AnimNextModuleInjectionComponent"))
struct FRigUnit_UAFRunAsset : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	static UE_API const UUAFAnimGraph* GetHostGraphToRun(FAnimNextExecuteContext& InExecuteContext);

	// The graph to run
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input, ShowOnlyInnerProperties, ExportAsReference="true"))
	FAnimNextAnimGraph Graph;

	// Variable overrides to be applied to subgraphs
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FAnimNextVariableOverridesCollection Overrides;

	// The name of the attribute set to execute with, if 'None' will include everything
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FName AttributeSet;

	// Result value
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Output))
	FUAFValueBundle Result;

	// Internal work data
	UPROPERTY(transient)
	FUAFRunAssetWorkData WorkData;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API
