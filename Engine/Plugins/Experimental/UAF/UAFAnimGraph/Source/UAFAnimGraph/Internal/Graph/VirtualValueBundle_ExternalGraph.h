// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/WeakAnimGraphReference.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{

struct FVirtualValueBundle_ExternalGraph : public IVirtualValueBundle
{
	explicit FVirtualValueBundle_ExternalGraph(const FWeakAnimGraphReference& InGraphReference)
		: WeakGraphReference(InGraphReference)
	{}

	explicit FVirtualValueBundle_ExternalGraph(const FAnimGraphReference& InGraphReference)
		: WeakGraphReference(InGraphReference)
	{}

private:
	// IVirtualValueBundle interface
	UE_API virtual const FWeakAnimGraphReference* GetAnimGraphReference() const override;

	FWeakAnimGraphReference WeakGraphReference;
};

}

#undef UE_API