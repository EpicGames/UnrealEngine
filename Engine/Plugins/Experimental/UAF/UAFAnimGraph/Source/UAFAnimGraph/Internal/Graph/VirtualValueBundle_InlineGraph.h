// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/WeakAnimGraphReference.h"
#include "TraitCore/TraitPtr.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{

struct FVirtualValueBundle_InlineGraph : public IVirtualValueBundle
{
	explicit FVirtualValueBundle_InlineGraph(const FTraitPtr& InTraitPtr)
		: TraitPtr(InTraitPtr)
	{}

private:
	// IVirtualValueBundle interface
	UE_API virtual const FTraitPtr* GetInlineGraph() const override;

	FTraitPtr TraitPtr;
};

}

#undef UE_API