// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/AnimNext_LODPose.h"
#include "Graph/WeakAnimGraphReference.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{

struct FVirtualValueBundle_ValueBundle : public IVirtualValueBundle
{
	explicit FVirtualValueBundle_ValueBundle(const FAnimNextGraphLODPose& InPose)
		: Pose(InPose)
	{}

	explicit FVirtualValueBundle_ValueBundle(FAnimNextGraphLODPose&& InPose)
		: Pose(MoveTemp(InPose))
	{}

	explicit FVirtualValueBundle_ValueBundle(const FValueBundleHeap& InValueBundle)
		: ValueBundle(InValueBundle)
	{}
	
		explicit FVirtualValueBundle_ValueBundle(FValueBundleHeap&& InValueBundle)
		: ValueBundle(MoveTemp(InValueBundle))
	{}
	
	// IVirtualValueBundle interface
	UE_API virtual const FAnimNextGraphLODPose* GetLODPose() const override;
	UE_API virtual const FValueBundleHeap* GetValueBundle() const override;

private:
	FAnimNextGraphLODPose Pose;
	FValueBundleHeap ValueBundle;
};

}

#undef UE_API
