// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/CancellableAsyncAction.h"
#include "Injection/InjectionRequest.h"

#include "InjectionCallbackProxy.generated.h"

class UUAFComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnInjectionDelegate);

UENUM()
enum class EUninjectionResult : uint8
{
	Succeeded,
	Failed
};

UCLASS(MinimalAPI, meta=(ExposedAsyncProxy = "AsyncTask", HasDedicatedAsyncNode))
class UInjectionCallbackProxy : public UCancellableAsyncAction
{
	GENERATED_BODY()

public:
	// Called when the provided animation object finished playing and hasn't been interrupted
	UPROPERTY(BlueprintAssignable)
	FOnInjectionDelegate OnCompleted;

	// Called when the provided animation object starts blending out and hasn't been interrupted
	UPROPERTY(BlueprintAssignable)
	FOnInjectionDelegate OnBlendOut;

	// Called when the provided animation object has been interrupted (or failed to play)
	UPROPERTY(BlueprintAssignable)
	FOnInjectionDelegate OnInterrupted;

	// Called to perform the query internally
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UAFANIMGRAPH_API UInjectionCallbackProxy* CreateProxyObjectForInjection(
		UUAFComponent* AnimNextComponent,
		UPARAM(meta=(AllowedType=FAnimNextAnimGraph)) FAnimNextVariableReference Site,
		UObject* Object,
		FAnimNextFactoryParams FactoryParams = FAnimNextFactoryParams(),
		FAnimNextInjectionBlendSettings BlendInSettings = FAnimNextInjectionBlendSettings(),
		FAnimNextInjectionBlendSettings BlendOutSettings = FAnimNextInjectionBlendSettings());

	// Un-inject a previously injected object. Cancelling this async tasks will also un-inject.
	UFUNCTION(BlueprintCallable, Category="Animation|UAF", meta=(ExpandEnumAsExecs = "ReturnValue"))
	UAFANIMGRAPH_API EUninjectionResult Uninject();

	// Sets a variable's value.
	// @param    Name     The variable to set
	// @param    Value    The value to set the variable to
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "UAF", DisplayName = "Set Variable", CustomThunk, meta = (CustomStructureParam = Value, AutoCreateRefTerm = "Variable, Value", UnsafeDuringActorConstruction))
	UAFANIMGRAPH_API void SetVariable(const FAnimNextVariableReference& Variable, const int32& Value);

	// UObject Interface
	UAFANIMGRAPH_API virtual void BeginDestroy() override;

	// UCancellableAsyncAction interface
	UAFANIMGRAPH_API virtual void Cancel() override;

protected:
	UAFANIMGRAPH_API void OnInjectionCompleted(const UE::UAF::FInjectionRequest& Request);
	UAFANIMGRAPH_API void OnInjectionInterrupted(const UE::UAF::FInjectionRequest& Request);
	UAFANIMGRAPH_API void OnInjectionBlendingOut(const UE::UAF::FInjectionRequest& Request);

	// Attempts to play an object with the specified payload. Returns whether it started or not.
	UAFANIMGRAPH_API bool Inject(
		UUAFComponent* AnimNextComponent,
		FAnimNextVariableReference Site,
		UObject* Object,
		FAnimNextFactoryParams&& FactoryParams,
		const UE::UAF::FInjectionBlendSettings& BlendInSettings,
		const UE::UAF::FInjectionBlendSettings& BlendOutSettings);

private:
	void Reset();

	DECLARE_FUNCTION(execSetVariable);

	UE::UAF::FInjectionRequestPtr PlayingRequest;
	bool bWasInterrupted = false;
};
