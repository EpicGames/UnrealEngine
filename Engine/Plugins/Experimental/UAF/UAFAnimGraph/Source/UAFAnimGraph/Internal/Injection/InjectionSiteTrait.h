// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Injection/InjectionRequest.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IEvaluate.h"

#include "InjectionSiteTrait.generated.h"

namespace UE::UAF
{
	struct FInjection_InjectEvent;
	struct FInjection_UninjectEvent;
}

USTRUCT(meta = (DisplayName = "Injection Site"))
struct FAnimNextInjectionSiteTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** Default input when no animation request has been made on this slot. */
	UPROPERTY()
	FAnimNextTraitHandle Source;

	/** The graph to inject */
	UPROPERTY(EditAnywhere, Category = "Graph")
	FAnimNextAnimGraph Graph;

	/** The default blend settings to use when blending in */ 
	UPROPERTY(EditAnywhere, Category = "Blending")
	FAnimNextInjectionBlendSettings DefaultBlendInSettings;

	/** The default blend settings to use when blending out */ 
	UPROPERTY(EditAnywhere, Category = "Blending")
	FAnimNextInjectionBlendSettings DefaultBlendOutSettings;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Graph) \
		GeneratorMacro(DefaultBlendInSettings) \
		GeneratorMacro(DefaultBlendOutSettings) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextInjectionSiteTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	// Namespaced alias
	using FInjectionSiteTraitData = FAnimNextInjectionSiteTraitSharedData; 
}

namespace UE::UAF
{
	/**
	 * FInjectionSiteTrait
	 * 
	 * A trait that provides a site into which animation graph logic can be 'injected'.
	 * It allows for this trait to act as a pass-through when not actively used and
	 * when an injection request is made to start playing a child instance, we blend to it.
	 */
	struct FInjectionSiteTrait : FAdditiveTrait, IUpdate, IDiscreteBlend, IGarbageCollection, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FInjectionSiteTrait, FAdditiveTrait)

		// Injection tracker state
		enum class EInjectionTrackerState : uint8
		{
			// Injection request is inactive
			Inactive,

			// Injection request is active and using a sub-graph that came from an event
			ActiveEvent,

			// Injection request is active and using a sub-graph that came from a change to our pinned input
			ActivePin,

			// Injection request is active and using the source input
			ActiveSource,
		};

		struct FPendingInjectionRequest
		{
			// The injection request
			FInjectionRequestPtr Request;

			// Whether or not a Stop request was issued
			bool bStop = false;

			// Returns whether or not we have a pending request
			bool IsValid() const { return !!Request || bStop; }

			// Resets the pending request
			void Reset()
			{
				Request = nullptr;
				bStop = false;
			}
		};

		struct FInjectionTracker
		{
			// The Injection request
			FInjectionRequestPtr Request;

			// The current tracker state
			EInjectionTrackerState State = EInjectionTrackerState::Inactive;

			// Initializes a tracker to the specified state
			void Initialize(FInjectionRequestPtr InRequest, EInjectionTrackerState InState);

			// Resets the tracker for re-use
			void Reset();

			// Check whether this pending request is valid for tracking
			bool IsRequestValid() const
			{
				return Request.IsValid();
			}
		};

		using FSharedData = FAnimNextInjectionSiteTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			// Pending tracker
			FInjectionTracker PendingTracker;

			// List of Injection slot requests
			TArray<FInjectionTracker> InjectionTrackers;

			// Pending injection request
			FPendingInjectionRequest PendingRequest;

			// Cached graph to use to compare pin equality
			FAnimNextAnimGraph CachedGraph;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// Internal impl - ensure that we have a pending request that we have dispatched to the
		// blend stack, reserves space for tracking, then commits the pending tracking request
		static void TrackPendingInjectionRequest(int32 InChildIndex, FInstanceData& InstanceData);

		// Event handling impl
		ETraitStackPropagation OnInjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_InjectEvent& Event) const;
		ETraitStackPropagation OnUninjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_UninjectEvent& Event) const;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IDiscreteBlend impl
		virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
	};
}
