// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "Animation/AnimSequence.h"
#include "TraitInterfaces/ITimeline.h"
#include "TraitInterfaces/IGroupSynchronization.h"

#include "MontageTrait.generated.h"

struct FUAFMontageComponent;

/** A trait that can play a montage based on the provided slot name */
USTRUCT(meta = (DisplayName = "Montage Trait", ShowTooltip=true))
struct FUAFMontageTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimNextTraitHandle Source;

	/** The name of this slot */
	UPROPERTY(EditAnywhere, Category = "Montage")
	FName SlotName;

	/** If true it will always update the source input, even when montages are fully blended in */
	UPROPERTY(EditAnywhere, Category = "Montage")
	bool bAlwaysUpdateSource = false;

	/** Disables synchronization for this slot even if the playing montage has a sync group setup */
	UPROPERTY(EditAnywhere, Category = "Montage")
	bool bDisableSynchronization = false;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(SlotName) \
		GeneratorMacro(bAlwaysUpdateSource) \
		GeneratorMacro(bDisableSynchronization)

GENERATE_TRAIT_LATENT_PROPERTIES(FUAFMontageTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR

};

namespace UE::UAF
{
	struct FMontageTrait : FBaseTrait, IUpdate, IEvaluate, IHierarchy, IUpdateTraversal, ITimeline, IGroupSynchronization, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FMontageTrait, FBaseTrait)

		struct FMontageEvalTaskData
		{
			FMontageEvalTaskData(float InAnimPosition, UAnimSequence* InAnimSequence)
				: AnimPosition(InAnimPosition)
				, WeakAnimationSequence(InAnimSequence)
			{
			}

			float AnimPosition = 0.0f;
			TWeakObjectPtr<UAnimSequence> WeakAnimationSequence = nullptr;
		};

		struct FInstanceData : FTrait::FInstanceData
		{
			/** If the pre-update or post-update state should be returned on GetState() */
			bool bReturnPreUpdateState = true;

			/**
			 * If any of the active montages are using blend profiles,
			 * this will affect what blend is performed and what blend data is created
			 */
			bool bUsesBlendProfiles = false;

			/** Cached Ptr to the montage component */
			FUAFMontageComponent* MontageComponent = nullptr;

			/** The eval data index into the montage component for the active synchronization candidate */
			int32 SyncCandidateIndex = INDEX_NONE;

			/** Delta in seconds between updates, populated during PreUpdate */
			float DeltaTime = 0.f;

			/** The current blend weight of the source input */
			float SourceWeight = 1.0f;

			/** The source weight of the previous update, used to calculate if the source is blending out */
			float LastUpdateSourceWeight = 0.0f;

			/** Input node from which we receive the source pose to blend montages against during blend windows */
			FTraitPtr Source;

			/** The state representing the timeline before the montage update */
			FTimelineState PreMontageUpdateState = FTimelineState();

			/** The state representing the timeline after the montage update */
			FTimelineState PostMontageUpdateState = FTimelineState();

			/** The sync group params used to participate in synchronization */
			FSyncGroupParameters SyncGroupParams;

			/** The individual weights for each montage animation, plus source if relevant */
			TArray<float> PoseWeights;

			/** The individual weights for each additive montage animation */
			TArray<float> AdditivePoseWeights;

			/** The anim data needed for the evaluation task for this slot */
			TArray<FMontageEvalTaskData> EvalTaskDataForSlot;

			/** The additive anim data needed for the evaluation task for this slot */
			TArray<FMontageEvalTaskData> AdditiveEvalTaskDataForSlot;

			/** Blend data for non-additive poses if blend profiles are used */
			TArray<FBlendSampleData> BlendDataForBlendProfiles;

			/** Blend data for additive poses if blend profiles are used */
			TArray<FBlendSampleData> AdditiveBlendDataForBlendProfiles;

			/** The skeleton that owns each blend profile used, or nullptr if montage doesn't use a blend profile */
			TArray<TObjectPtr<const USkeleton>> SkeletonDataForBlendProfiles;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
			void ResetData();
		};

		using FSharedData = FUAFMontageTraitSharedData;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// ITimeline impl
		virtual void GetSyncMarkers(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineSyncMarkerArray& OutSyncMarkers) const override;
		virtual bool GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const override;

		// IGroupSynchronization impl
		virtual FSyncGroupParameters GetGroupParameters(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding) const override;
		virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding, float DeltaTime, bool bDispatchEvents) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};
}
