// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "MontageTrait.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimNotifyQueue.h"
#include "Engine/EngineBaseTypes.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Module/UAFModuleInstanceComponent.h"
#include "UAFMontageComponent.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FUAFUpdateMontageTickFunction : public FTickFunction
{
public:
	// Begin FTickFunction overrides
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
	// End FTickFunction overrides

private:
	TWeakObjectPtr<class UUAFComponent> Target = nullptr;
	TWeakObjectPtr<class UAnimInstance> WeakAnimInstance = nullptr;
	FUAFMontageComponent* MontageComponent = nullptr;

	friend struct FUAFMontageComponent;
};

USTRUCT()
struct FUAFMontageEvalData
{
	GENERATED_BODY()

public:
	FUAFMontageEvalData() = default;
	FUAFMontageEvalData(UAnimMontage* InMontage, float InPosition, float InPlayRate, float InBlendStartAlpha, const FAlphaBlend& InBlendInfo, const UBlendProfile* InBlendProfile)
		: Montage(InMontage)
		, BlendInfo(InBlendInfo)
		, ActiveBlendProfile(InBlendProfile)
		, MontagePosition(InPosition)
		, MontagePlayRate(InPlayRate)
		, BlendStartAlpha(InBlendStartAlpha)
	{
	}
	
	// Returns true if the montage in this eval data is active (valid and not blending out)
	bool IsActive() const { return (Montage != nullptr && BlendInfo.GetDesiredValue() != 0.0f); }
	
	// Returns the montage for this eval data
	const UAnimMontage* GetMontage() const {return Montage;}
	
	// Returns the alpha blend info for this eval data
	const FAlphaBlend& GetBlendInfo() const {return BlendInfo;}
	
private:
	UPROPERTY(Transient)
	TObjectPtr<UAnimMontage> Montage = nullptr;

	UPROPERTY(Transient)
	FAlphaBlend BlendInfo = FAlphaBlend();

	UPROPERTY(Transient)
	TObjectPtr<const UBlendProfile> ActiveBlendProfile = nullptr;

	float MontagePosition = 0.0f;
	float MontagePlayRate = 1.0f;
	float BlendStartAlpha = 0.0f;

	friend struct UE::UAF::FMontageTrait;
};

// This module instance component is responsible for supporting montages within UAF. 
// It handles required setup, update logic and provides relevant information to montage traits.
USTRUCT()
struct FUAFMontageComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

public:
	FUAFMontageComponent() = default;
	virtual ~FUAFMontageComponent() override;

	UE_API bool IsSlotActive(FName SlotNodeName) const;
	UE_API float GetSlotLocalWeight(FName SlotNodeName) const;
	UE_API TObjectPtr<const UAnimMontage> GetActiveMontageForSlot(FName SlotNodeName) const;
	
private:
	virtual void OnBindToInstance() override;

	// Retrieves all active montage animation notifies for a specific slot name
	// returns true if any notifies for the given slot name exist
	bool GetActiveNotifiesForSlot(FName SlotName, TArray<FAnimNotifyEventReference>& OutNotifies) const;
	
	bool HasActiveInertializationRequests() const;
	const FInertializationRequest* GetInertializationRequestForGroupName(FName SlotGroupName) const;

private:
	// All active and weighted montages, copied from the Anim Instance
	UPROPERTY(Transient)
	TArray<FUAFMontageEvalData> MontageEvalData;

	// Active animation notifies per slot, used to have them easily accessible in montage traits 
	UPROPERTY(Transient)
	TMap<FName, FAnimNotifyArray> ActiveSlotNotifies;
	
	UPROPERTY(Transient)
	TMap<FName, FInertializationRequest> ActiveInertializationRequests;

	// The tick function that is used to update montages on the game thread
	FUAFUpdateMontageTickFunction MontageTickFunction;

	// The event name for the module event tick function that the montage tick function will be enqueued as prerequisite for
	// If None is set, it will fall back to DefaultTickDependencyEventName
	UPROPERTY(EditAnywhere, Category = Montages, meta = (CustomWidget = AnimNextModuleEvent))
	FName TickMontagesBeforeEventName = NAME_None;

	// The Instance Class that Montages will run on
	UPROPERTY(EditAnywhere, Category= Montages, DisplayName="Legacy Montage AnimInstance")
	TSubclassOf<UAnimInstance> LegacyMontageAnimInstanceClass = nullptr;

	// The event name of the module tick function used at runtime to enqueue the montage tick function as a prerequisite to. 
	// If the exposed TickMontagesBeforeEventName does not provide an override, it will fall back to DefaultTickDependencyEventName
	UPROPERTY(Transient)
	FName TickDependencyEventName = NAME_None;
	
	bool bHasTickDependencyRegistered = false;
	
	// The event name to fall back to for the module event tick function that the montage tick function will be enqueued as prerequisite for
	static inline const FLazyName DefaultTickDependencyEventName = FLazyName("PrePhysics");

	friend struct UE::UAF::FMontageTrait;
	friend struct FUAFUpdateMontageTickFunction;
};

template<>
struct TStructOpsTypeTraits<FUAFMontageComponent> : public TStructOpsTypeTraitsBase2<FUAFMontageComponent>
{
	enum
	{
		WithCopy = false,
	};
};

// Returns true if the running UAF System has an active montage in the given slot. A UAnimMontage that is playing in the slot and blending out is not determined to be "active". 
USTRUCT(meta=(DisplayName="Is Slot Active", Category="Injection", NodeColor="0, 1, 1", Keywords="Slot, Active, Montage"))
struct FRigUnit_IsSlotActive : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();
	
	// The name of the slot
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FName SlotNodeName = NAME_None;
	
	// True if the running UAF System has an active montage in the given slot, and it's not blending out
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	bool Result = false;
};

// Gets the local weight of any montages this slot node is playing. If this slot is not currently playing a montage, it will return 0. 
USTRUCT(meta=(DisplayName="Get Slot Local Weight", Category="Injection", NodeColor="0, 1, 1", Keywords="Slot, Weight, Montage"))
struct FRigUnit_GetSlotLocalWeight : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();
	
	// The name of the slot
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FName SlotNodeName = NAME_None;
	
	// The local weight of any montages this slot is playing
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	float LocalWeight = 0.0f;
};

#undef UE_API
