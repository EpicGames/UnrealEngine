// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Injection/InjectionRequest.h"

#include "PlayAnimCallbackProxy.generated.h"

class UAnimSequence;
class UUAFComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPlayAnimPlayDelegate);

UCLASS(MinimalAPI)
class UPlayAnimCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when the provided animation object finished playing and hasn't been interrupted
	UPROPERTY(BlueprintAssignable)
	FOnPlayAnimPlayDelegate OnCompleted;

	// Called when the provided animation object starts blending out and hasn't been interrupted
	UPROPERTY(BlueprintAssignable)
	FOnPlayAnimPlayDelegate OnBlendOut;

	// Called when the provided animation object has been interrupted (or failed to play)
	UPROPERTY(BlueprintAssignable)
	FOnPlayAnimPlayDelegate OnInterrupted;

	// Called to perform the query internally
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UAFANIMGRAPH_API UPlayAnimCallbackProxy* CreateProxyObjectForPlayAnim(
		UUAFComponent* AnimNextComponent,
		FName SiteName,
		UAnimSequence* AnimSequence,
		float PlayRate = 1.0f,
		float StartPosition = 0.0f,
		FAnimNextInjectionBlendSettings BlendInSettings = FAnimNextInjectionBlendSettings(),
		FAnimNextInjectionBlendSettings BlendOutSettings = FAnimNextInjectionBlendSettings());

	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", DeprecatedFunction, DeprecatedMessage="Please use Inject instead."))
	static UPlayAnimCallbackProxy* CreateProxyObjectForPlayAsset(
		UUAFComponent* AnimNextComponent,
		FName SiteName,
		UObject* Object,
		const FInstancedStruct& Payload,
		FAnimNextInjectionBlendSettings BlendInSettings = FAnimNextInjectionBlendSettings(),
		FAnimNextInjectionBlendSettings BlendOutSettings = FAnimNextInjectionBlendSettings())
	{
		return nullptr;
	}

public:
	//~ Begin UObject Interface
	UAFANIMGRAPH_API virtual void BeginDestroy() override;
	//~ End UObject Interface

protected:
	UAFANIMGRAPH_API void OnPlayAnimCompleted(const UE::UAF::FInjectionRequest& Request);
	UAFANIMGRAPH_API void OnPlayAnimInterrupted(const UE::UAF::FInjectionRequest& Request);
	UAFANIMGRAPH_API void OnPlayAnimBlendingOut(const UE::UAF::FInjectionRequest& Request);

	// Attempts to play an animation with the specified settings. Returns whether it started or not.
	UAFANIMGRAPH_API bool Play(
		UUAFComponent* AnimNextComponent,
		FName SiteName,
		UAnimSequence* AnimSequence,
		float PlayRate,
		float StartPosition,
		const UE::UAF::FInjectionBlendSettings& BlendInSettings,
		const UE::UAF::FInjectionBlendSettings& BlendOutSettings);

private:
	void Reset();

	UE::UAF::FInjectionRequestPtr PlayingRequest;
	bool bWasInterrupted = false;
};
