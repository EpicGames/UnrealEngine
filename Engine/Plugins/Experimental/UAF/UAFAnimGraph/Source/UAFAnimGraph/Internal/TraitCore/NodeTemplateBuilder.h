// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitUID.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FNodeTemplate;

	/**
	  * FNodeTemplateBuilder
	  * 
	  * Utility to help construct node templates.
	  */
	struct FNodeTemplateBuilder
	{
		FNodeTemplateBuilder() = default;

		// Adds the specified trait type to the node template
		UE_API void AddTrait(FTraitUID TraitUID);

		// Returns a node template for the provided list of traits
		// The node template will be built into the provided buffer and a pointer to it is returned
		UE_API FNodeTemplate* BuildNodeTemplate(TArray<uint8>& NodeTemplateBuffer) const;

		// Returns a node template for the provided list of traits
		// The node template will be built into the provided buffer and a pointer to it is returned
		static UE_API FNodeTemplate* BuildNodeTemplate(TConstArrayView<FTraitUID> InTraitUIDs, TArray<uint8>& NodeTemplateBuffer);

		// Resets the node template builder
		UE_API void Reset();

	private:
		static UE_API uint32 GetNodeTemplateUID(TConstArrayView<FTraitUID> InTraitUIDs);
		static UE_API void AppendTemplateTrait(
			TConstArrayView<FTraitUID>, int32 TraitIndex,
			TArray<uint8>& NodeTemplateBuffer);

		TArray<FTraitUID> TraitUIDs;	// The list of traits to use when building the node template
	};
}

#undef UE_API
