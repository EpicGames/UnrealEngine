// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/NodeTemplateRegistryHandle.h"
#include "Containers/PagedArray.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF::AnimGraph
{
	class FAnimNextAnimGraphModule;
}

namespace UE::UAF
{
	struct FNodeTemplate;

	/**
	 * FNodeTemplateRegistry
	 * 
	 * A global registry of all existing node templates that can be shared between animation graph instances.
	 * 
	 * @see FNodeTemplate
	 */
	struct FNodeTemplateRegistry final
	{
		UE_NONCOPYABLE(FNodeTemplateRegistry);	// We disallow copy/move semantics

		// Access the global registry
		static UE_API FNodeTemplateRegistry& Get();

		// Finds the specified node template from its UID and returns its offset
		UE_API FNodeTemplateRegistryHandle Find(uint32 NodeTemplateUID) const;

		// Finds or adds the specified node template and returns its offset
		UE_API FNodeTemplateRegistryHandle FindOrAdd(const FNodeTemplate* NodeTemplate);

		// Finds and returns a node template based on its handle or nullptr if the handle is invalid
		UE_API const FNodeTemplate* Find(FNodeTemplateRegistryHandle TemplateHandle) const;

		// Returns the number of registered node templates
		UE_API uint32 GetNum() const;

		// Removes the specified node template from the registry
		UE_API void Unregister(const FNodeTemplate* NodeTemplate);

	private:
		FNodeTemplateRegistry() = default;

		// Finds and returns a node template based on its handle or nullptr if the handle is invalid
		UE_API FNodeTemplate* FindMutable(FNodeTemplateRegistryHandle TemplateHandle);

		// Module lifetime functions
		static UE_API void Init();
		static UE_API void Destroy();

#if WITH_DEV_AUTOMATION_TESTS
		// Swaps the global node template registry with the specified instance
		// Swap it back for the original once tests are done
		static UE_API FNodeTemplateRegistry* Swap(FNodeTemplateRegistry* Other);
#endif

		// We only ever append node templates to this contiguous buffers
		// This is an optimization, we share these and by being contiguous
		// we improve cache locality and cache line density
		// Because node templates are trivially copyable, we could remove
		// from this buffer when there are holes and coalesce everything.
		// To do so, we would have to fix-up any outstanding handles within
		// the shared data of loaded anim graphs.
		TPagedArray<uint8>							TemplateBuffer;
		TMap<uint32, FNodeTemplateRegistryHandle>	TemplateUIDToHandleMap;
		mutable FRWLock								TemplateLock;

		friend class UE::UAF::AnimGraph::FAnimNextAnimGraphModule;
		friend class FTraitWriter;

#if WITH_DEV_AUTOMATION_TESTS
		friend struct FScopedClearNodeTemplateRegistry;
		friend struct FScopedNewNodeTemplateRegistry;
#endif
	};
}

#undef UE_API
