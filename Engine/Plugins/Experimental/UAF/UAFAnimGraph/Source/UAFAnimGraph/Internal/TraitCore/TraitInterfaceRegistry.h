// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitInterfaceUID.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF::AnimGraph
{
	class FAnimNextAnimGraphModule;
}

namespace UE::UAF
{
	/**
	 * FTraitInterfaceRegistry
	 * 
	 * A global registry of all existing trait interfaces that can be used in animation graph traits.
	 * 
	 * @see FTraitInterface
	 */
	struct FTraitInterfaceRegistry final
	{
		// Access the global registry
		static UE_API FTraitInterfaceRegistry& Get();

		// Finds and returns the trait interface associated with the provided trait interface UID.
		// If the trait interface is not registered, nullptr is returned.
		UE_API const ITraitInterface* Find(FTraitInterfaceUID InterfaceUID) const;

		// Registers a trait interface dynamically
		UE_API void Register(const TSharedPtr<ITraitInterface>& TraitInterface);

		// Unregisters a trait intrerface dynamically
		UE_API void Unregister(const TSharedPtr<ITraitInterface>& TraitInterface);

		// Returns a list of all registered trait interfaces
		UE_API TArray<const ITraitInterface*> GetTraitInterfaces() const;

		// Returns the number of registered trait interfaces
		UE_API uint32 GetNum() const;

	private:
		FTraitInterfaceRegistry() = default;
		FTraitInterfaceRegistry(const FTraitInterfaceRegistry&) = delete;
		FTraitInterfaceRegistry(FTraitInterfaceRegistry&&) = default;
		FTraitInterfaceRegistry& operator=(const FTraitInterfaceRegistry&) = delete;
		FTraitInterfaceRegistry& operator=(FTraitInterfaceRegistry&&) = default;

		// Static init lifetime functions
		static UE_API void StaticRegister(const TSharedPtr<ITraitInterface>& TraitInterface);
		static UE_API void StaticUnregister(const TSharedPtr<ITraitInterface>& TraitInterface);

		// Module lifetime functions
		static UE_API void Init();
		static UE_API void Destroy();

		// Holds information for each registered trait
		struct FRegistryEntry
		{
			// A pointer to the trait
			TSharedPtr<ITraitInterface> TraitInterface = nullptr;
		};

		TMap<FTraitInterfaceUIDRaw, FRegistryEntry>	TraitInterfaceUIDToEntryMap;

		friend class UE::UAF::AnimGraph::FAnimNextAnimGraphModule;
		friend struct FTraitInterfaceStaticInitHook;
	};
}

#undef UE_API
