// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/EntryPointHandle.h"
#include "TraitCore/NodeHandle.h"
#include "Serialization/ArchiveProxy.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	/**
	  * FTraitReader
	  * 
	  * The trait reader is used to read from a serialized binary blob that contains
	  * the anim graph data. An anim graph contains the following:
	  *     - A list of FNodeTemplates that the nodes use
	  *     - The graph shared data (FNodeDescription for every node)
	  */
	class FTraitReader final : public FArchiveProxy
	{
	public:
		// The largest size allowed for the shared data of a single graph
		// Node/trait handles use an unsigned 24 bits value to represent graph offsets
		// which limits us to 16 MB
		// We could extend this range by leveraging the fact that nodes have a minimum required alignment
		static constexpr uint32 MAXIMUM_GRAPH_SHARED_DATA_SIZE = (1 << 24) - 1;

		enum class EErrorState
		{
			None,						// All good, no error
			GraphTooLarge,				// Exceeded the maximum graph size, @see FTraitReader::MAXIMUM_GRAPH_SHARED_DATA_SIZE
			NodeSharedDataTooLarge,		// Exceeded the maximum node shared data size, @see FNodeDescription::MAXIMUM_NODE_SHARED_DATA_SIZE
			NodeInstanceDataTooLarge,	// Exceeded the maximum node instance data size, @see FNodeInstance::MAXIMUM_NODE_INSTANCE_DATA_SIZE
			InvalidData,				// Unexpected or corrupted data
		};

		UE_API explicit FTraitReader(const TArray<TObjectPtr<UObject>>& InGraphReferencedObjects, const TArray<FSoftObjectPath>& InGraphReferencedSoftObjects, FArchive& Ar);

		[[nodiscard]] UE_API EErrorState ReadGraph(TArray<uint8>& GraphSharedData);

		// Takes a node handle representing a node index and resolves it into a node handle representing a shared data offset
		// Must be called after ReadGraphSharedData as it populates the necessary data
		[[nodiscard]] UE_API FNodeHandle ResolveNodeHandle(FNodeHandle NodeHandle) const;

		// Takes a trait handle representing a node index and resolves it into a node handle representing a shared data offset
		// Must be called after ReadGraphSharedData as it populates the necessary data
		[[nodiscard]] UE_API FAnimNextTraitHandle ResolveTraitHandle(FAnimNextTraitHandle TraitHandle) const;

		// Takes an entry point handle representing a node index and resolves it into a node handle representing a shared data offset
		// Must be called after ReadGraphSharedData as it populates the necessary data
		[[nodiscard]] UE_API FAnimNextTraitHandle ResolveEntryPointHandle(FAnimNextEntryPointHandle EntryPointHandle) const;

		// FArchive implementation
		UE_API virtual FArchive& operator<<(UObject*& Obj) override;
		UE_API virtual FArchive& operator<<(FObjectPtr& Obj) override;
		UE_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
		UE_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
		UE_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;

	private:
		// Call first to read the graph shared data
		[[nodiscard]] UE_API EErrorState ReadGraphSharedData(TArray<uint8>& GraphSharedData);

		// A list of UObject references within the graph
		const TArray<TObjectPtr<UObject>>& GraphReferencedObjects;

		// A list of soft object references within the graph
		const TArray<FSoftObjectPath>& GraphReferencedSoftObjects;

		// A list of node handles for each node within the archive
		TArray<FNodeHandle> NodeHandles;

		friend FAnimNextTraitHandle;
	};
}

#undef UE_API
