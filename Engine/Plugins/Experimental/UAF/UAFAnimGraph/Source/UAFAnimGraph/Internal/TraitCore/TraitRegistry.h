// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitRegistryHandle.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF::AnimGraph
{
	class FAnimNextAnimGraphModule;
}

namespace UE::UAF
{
	/**
	 * FTraitRegistry
	 * 
	 * A global registry of all existing traits that can be used in animation graphs.
	 * 
	 * @see FTrait
	 */
	struct FTraitRegistry final
	{
		// Access the global registry
		static UE_API FTraitRegistry& Get();

		// Finds and returns the trait handle for the provided trait UID or an invalid
		// handle if that trait hasn't been registered yet.
		UE_API FTraitRegistryHandle FindHandle(FTraitUID TraitUID) const;

		// Finds and returns the trait associated with the provided handle.
		// If the handle is not valid, nullptr is returned.
		UE_API const FTrait* Find(FTraitRegistryHandle TraitHandle) const;

		// Finds and returns the trait associated with the provided trait UID.
		// If the trait is not registered, nullptr is returned.
		UE_API const FTrait* Find(FTraitUID TraitUID) const;

		// Finds and returns the trait associated with the provided trait shared data UScriptStruct.
		// If the matching trait is not registered, nullptr is returned.
		UE_API const FTrait* Find(const UScriptStruct* TraitSharedDataStruct) const;

		// Finds and returns the trait associated with the provided trait name.
		// If the matching trait is not registered, nullptr is returned.
		UE_API const FTrait* Find(FName TraitTypeName) const;

		// Registers a trait dynamically
		UE_API void Register(FTrait* Trait);

		// Unregisters a trait dynamically
		UE_API void Unregister(FTrait* Trait);

		// Returns a list of all registered traits
		UE_API TArray<const FTrait*> GetTraits() const;

		// Returns the number of registered traits
		UE_API uint32 GetNum() const;

	private:
		FTraitRegistry() = default;
		FTraitRegistry(const FTraitRegistry&) = delete;
		FTraitRegistry(FTraitRegistry&&) = default;
		FTraitRegistry& operator=(const FTraitRegistry&) = delete;
		FTraitRegistry& operator=(FTraitRegistry&&) = default;

		// Static init lifetime functions
		static UE_API void StaticRegister(TraitConstructorFunc TraitConstructor);
		static UE_API void StaticUnregister(TraitConstructorFunc TraitConstructor);
		UE_API void AutoRegisterImpl(TraitConstructorFunc TraitConstructor);
		UE_API void AutoUnregisterImpl(TraitConstructorFunc TraitConstructor);

		// Module lifetime functions
		static UE_API void Init();
		static UE_API void Destroy();

		// Holds information for each registered trait
		struct FRegistryEntry
		{
			// A pointer to the trait
			// TODO: Do we want a shared ptr here? we don't always own it
			FTrait*					Trait = nullptr;

			// A pointer to the constructor function
			// Only valid when the trait has been auto-registered
			TraitConstructorFunc	TraitConstructor = nullptr;

			// The trait handle
			FTraitRegistryHandle	TraitHandle;
		};

		// For performance reasons, we store static traits that never unload into
		// a single contiguous memory buffer. However, traits cannot be guaranteed
		// to be trivially copyable because they contain virtual functions. As such,
		// we cannot resize the buffer once they have been allocated. We reserve a fixed
		// amount of space that should easily cover our needs. Static traits are
		// generally stateless and only contain a few v-tables. Their size is usually
		// less than 32 bytes. Additionally, we will likely only ever load a few hundred
		// traits. If we exceed the size of the buffer, additional traits will
		// be treated as dynamic. Dynamic traits are instead allocated on the heap.
		static constexpr uint32			STATIC_TRAIT_BUFFER_SIZE = 8 * 1024;

		uint8							StaticTraitBuffer[STATIC_TRAIT_BUFFER_SIZE] = { 0 };
		int32							StaticTraitBufferOffset = 0;

		TArray<intptr_t>				DynamicTraits;
		int32							DynamicTraitFreeIndexHead = INDEX_NONE;

		TMap<FTraitUIDRaw, FRegistryEntry>	TraitUIDToEntryMap;
		TMap<FName, FTraitUIDRaw>			TraitNameToUIDMap;

		friend class UE::UAF::AnimGraph::FAnimNextAnimGraphModule;
		friend struct FTraitStaticInitHook;
	};
}

#undef UE_API
