// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Serialization/MemoryWriter.h"
#include "StructUtils/StructView.h"

#include "TraitCore/LatentPropertyHandle.h"
#include "TraitCore/NodeHandle.h"
#include "TraitCore/NodeTemplateRegistryHandle.h"
#include "UObject/SoftObjectPath.h"

struct FAnimNextTraitSharedData;

namespace UE::UAF
{
	struct FNodeTemplate;

	/**
	  * FTraitWriter
	  *
	  * The trait writer is used to write a serialized binary blob that contains
	  * the anim graph data. An anim graph contains the following:
	  *     - A list of FNodeTemplates that the nodes use
	  *     - The graph shared data (FNodeDescription for every node)
	  */
	class FTraitWriter final : public FMemoryWriter
	{
	public:
		enum class EErrorState
		{
			None,					// All good, no error
			TooManyNodes,			// Exceeded the maximum number of nodes in a graph, @see FNodeDescription::MAXIMUM_COUNT
			NodeTemplateNotFound,	// Failed to find a necessary node template
			NodeTemplateTooLarge,	// Exceeded the maximum node template size, @see FNodeTemplate::MAXIMUM_SIZE
			NodeHandleNotFound,		// Failed to find the mapping for a node handle, it was likely not registered
		};

		UAFANIMGRAPH_API FTraitWriter();

		// Registers an instance of the provided node template and assigns a node handle and node UID to it
		[[nodiscard]] UAFANIMGRAPH_API FNodeHandle RegisterNode(const FNodeTemplate& NodeTemplate);

		// Called before node writing can begin
		UAFANIMGRAPH_API void BeginNodeWriting();

		// Called once node writing has terminated
		UAFANIMGRAPH_API void EndNodeWriting();

#if WITH_EDITOR
		// Writes out the provided node using the trait properties
		// Nodes must be written in the same order they were registered in
		UAFANIMGRAPH_API void WriteNode(
			const FNodeHandle NodeHandle,
			const TFunction<FString(uint32 TraitIndex, FName PropertyName)>& GetTraitProperty,
			const TFunction<uint16(uint32 TraitIndex, FName PropertyName)>& GetTraitLatentPropertyIndex
			);
#endif

		// Writes out the provided node using the trait properties
		// Nodes must be written in the same order they were registered in
		void WriteNode(
			const FNodeHandle NodeHandle,
			TFunctionRef<uint16(uint32 TraitIndex, FName PropertyName)> GetTraitLatentPropertyIndex,
			TFunctionRef<TConstStructView<FAnimNextTraitSharedData>(uint32 TraitIndex)> GetTraitData);

		// Returns the error state
		[[nodiscard]] UAFANIMGRAPH_API EErrorState GetErrorState() const;

		// Returns the populated raw graph shared data buffer
		[[nodiscard]] UAFANIMGRAPH_API const TArray<uint8>& GetGraphSharedData() const;

		// Returns the list of referenced UObjects in this graph
		[[nodiscard]] UAFANIMGRAPH_API const TArray<UObject*>& GetGraphReferencedObjects() const;

		// Returns the list of referenced soft objects in this graph
		[[nodiscard]] UAFANIMGRAPH_API const TArray<FSoftObjectPath>& GetGraphReferencedSoftObjects() const;

		// FArchive implementation
		UAFANIMGRAPH_API virtual FArchive& operator<<(UObject*& Obj) override;
		UAFANIMGRAPH_API virtual FArchive& operator<<(FObjectPtr& Obj) override;
		UAFANIMGRAPH_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
		UAFANIMGRAPH_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
		UAFANIMGRAPH_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;

	private:
		struct FNodeMapping
		{
			// The node handle for this entry (encoded as a node ID)
			FNodeHandle NodeHandle;

			// The node template handle the node uses
			FNodeTemplateRegistryHandle NodeTemplateHandle;

			// The unique node template index that we'll serialize
			uint32 NodeTemplateIndex;
		};

		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<FNodeMapping> NodeMappings;

		// To track the node registration process
		FNodeID NextNodeID;

		// To track node writing
		TArray<UObject*> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		uint32 NumNodesWritten;
		bool bIsNodeWriting;

		EErrorState ErrorState;
	};
}

