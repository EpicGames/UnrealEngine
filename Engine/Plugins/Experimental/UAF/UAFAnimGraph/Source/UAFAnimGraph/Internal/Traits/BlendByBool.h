// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"

#include "BlendByBool.generated.h"

/** A trait that can blend two discrete inputs through a boolean. */
USTRUCT(meta = (DisplayName = "Blend By Bool", ShowTooltip=true))
struct FAnimNextBlendByBoolTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** First output to be blended. */
	UPROPERTY()
	FAnimNextTraitHandle TrueChild;

	/** Second output to be blended. */
	UPROPERTY()
	FAnimNextTraitHandle FalseChild;

	/** The boolean condition that decides which child is active. */
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bCondition = false;

	/** Always update TrueChild, regardless of whether or not that child has weight. */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	bool bAlwaysUpdateTrueChild = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(bCondition) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextBlendByBoolTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	/**
	 * FBlendByBoolTrait
	 * 
	 * A trait that can blend two discrete inputs through a boolean.
	 */
	struct FBlendByBoolTrait : FBaseTrait, IUpdate, IUpdateTraversal, IHierarchy, IDiscreteBlend
	{
		DECLARE_ANIM_TRAIT(FBlendByBoolTrait, FBaseTrait)

		using FSharedData = FAnimNextBlendByBoolTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr TrueChild;
			FTraitPtr FalseChild;

			int32 PreviousChildIndex = INDEX_NONE;
			bool bWasTrueChildRelevant = false;
			bool bWasFalseChildRelevant = false;
		};

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IDiscreteBlend impl
		virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual int32 GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const override;
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;
		virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
	};
}
