// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/AnimNextAnimationGraph.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/IInertializerBlend.h"
#include "TraitInterfaces/ITimeline.h"
#include "TraitInterfaces/ISmoothBlend.h"
#include "TraitInterfaces/ISmoothBlendPerBone.h"

#include "BlendStackTrait.generated.h"

/** Used as base trait in state machine-like setups where state machine is the additive trait that pushes graphs and blend settings to the blend stack. */
USTRUCT(meta = (DisplayName = "Blend Stack Core", ShowTooltip=true))
struct FAnimNextBlendStackCoreTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// This struct is empty but required so that we can give a nice display name to the trait
};

namespace UE::UAF
{
	// Namespaced alias
	using FBlendStackCoreTraitData = FAnimNextBlendStackCoreTraitSharedData; 
}

/**
 * Standalone blend stack that exposes desired object and blend time as pins.
 * A new graph will be pushed every time the newest object does not match the pin (or bForceBlend is true).
 */	
USTRUCT(meta = (DisplayName = "Blend Stack", ShowTooltip=true))
struct FAnimNextBlendStackTraitSharedData : public FAnimNextBlendStackCoreTraitSharedData
{
	GENERATED_BODY()

	/** Asset to use as a sub-graph */
	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<const UObject> Object;

	/** Blend time to use when sub-graph pin input changes */
	UPROPERTY(EditAnywhere, Category = "Default")
	float BlendTime = 0.2f;

	/** Force a blend to the current graph, even if there's been no change in selection. */
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bForceBlend = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Object) \
		GeneratorMacro(BlendTime) \
		GeneratorMacro(bForceBlend) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextBlendStackTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

/**
 * Example additive trait used to push graphs to a Blend Stack base trait.
 * Same behavior as FBlendStackTrait, but as additive (i.e. FBlendStackCoreTrait base + FBlendStackRequesterTrait as additive).
 */	
USTRUCT(meta = (DisplayName = "Blend Stack Requester", ShowTooltip=true))
struct FAnimNextBlendStackRequesterTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** Asset to use as a sub-graph */
	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<UObject> Object;

	/** Blend time to use when sub-graph pin input changes */
	UPROPERTY(EditAnywhere, Category = "Default")
	float BlendTime = 0.2f;

	/** Force a blend to the current graph, even if there's been no change in selection. */
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bForceBlend = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Object) \
		GeneratorMacro(BlendTime) \
		GeneratorMacro(bForceBlend) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextBlendStackRequesterTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	/**
	 * FBlendStackCoreTrait
	 * 
	 * Used as base trait in state machine-like setups where state machine is the additive trait that pushes graphs and blend settings to the blend stack.
	 */
	struct FBlendStackCoreTrait : FBaseTrait, IUpdate, IUpdateTraversal, IHierarchy, IDiscreteBlend, ISmoothBlend, IInertializerBlend, ITimeline, IGarbageCollection, IBlendStack, ISmoothBlendPerBone
	{
		DECLARE_ANIM_TRAIT(FBlendStackCoreTrait, FBaseTrait)

		enum class EGraphState : uint8
		{
			Inactive,
			Active,
		};

		struct FGraphState
		{
			// The blend settings to use
			IBlendStack::FGraphRequest Request;

			// The graph instance
			TSharedPtr<FAnimNextGraphInstance> Instance;

			// The length of time this subgraph has been alive
			float Lifetime;

			FBlendStackCoreTrait::EGraphState State;

			// Whether or not this graph was previously relevant
			bool bNewlyCreated = false;

			void Initialize(IBlendStack::FGraphRequest&& GraphRequest);
			void Terminate();

			FWeakTraitPtr GetChildPtr() const
			{
				if (Request.Type == EGraphRequestType::Child)
				{
					return ChildPtr;
				}
				else if (Instance)
				{
					return Instance->GraphInstancePtr;
				}
				return FWeakTraitPtr();
			}

			void SetChildPtr(FWeakTraitPtr InChildPtr)
			{
				ChildPtr = InChildPtr;
			}

		private:
			// Our child handle
			// Weak handle to the graph instance's root
			FTraitPtr ChildPtr;
		};

		struct FInstanceData : FTrait::FInstanceData
		{
			// List of current graphs in the blend stack
			TArray<FGraphState> ChildGraphs;

			// The index of the currently active graph
			// All other graphs are blending out
			int32 CurrentlyActiveGraphIndex = INDEX_NONE;
		};

		using FSharedData = FAnimNextBlendStackCoreTraitSharedData;

		// Internal impl
		static int32 FindFreeGraphIndexOrAdd(FInstanceData& InstanceData);

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
		virtual FWeakTraitPtr GetStackForwardingChild(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;

		// IDiscreteBlend impl
		virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual int32 GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const override;
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;
		virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;

		// IInertializerBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const override;

		// ISmoothBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;
		virtual EAlphaBlendOption GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;
		virtual UCurveFloat* GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;

		// ITimeline impl
		virtual bool GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

		// IBlendStack impl
		virtual int32 PushGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequest&& GraphRequest) const override;
		virtual int32 GetActiveGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequestPtr& OutGraphRequest) const override;
		virtual int32 GetActiveGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, FAnimNextGraphInstance*& OutGraphInstance) const override;
		virtual IBlendStack::FGraphRequestPtr GetGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const override;
		virtual FAnimNextGraphInstance* GetGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const override;

		// ISmoothBlendPerBone impl
		virtual const UUAFBlendProfile* GetBlendProfile(FExecutionContext& Context, const TTraitBinding<ISmoothBlendPerBone>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return true; }
#endif
	};

	/**
	 * FBlendStackTrait
	 * 
	 * Standalone blend stack that exposes desired object and blend time as pins.
	 * A new graph will be pushed every time the newest object does not match the pin (or bForceBlend is true).
	 */	
	struct FBlendStackTrait : FBlendStackCoreTrait
	{
		DECLARE_ANIM_TRAIT(FBlendStackTrait, FBlendStackCoreTrait)

		using FSharedData = FAnimNextBlendStackTraitSharedData;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return false; }
#endif
	};

	/**
	 * FBlendStackRequesterTrait
	 * 
	 * Example additive trait used to push graphs to a Blend Stack base trait.
	 * Same behavior as FBlendStackTrait, but as additive (i.e. FBlendStackCoreTrait base + FBlendStackRequesterTrait as additive).
	 */	
	struct FBlendStackRequesterTrait : FAdditiveTrait, IUpdate, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FBlendStackRequesterTrait, FAdditiveTrait)

		using FSharedData = FAnimNextBlendStackRequesterTraitSharedData;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};
}
