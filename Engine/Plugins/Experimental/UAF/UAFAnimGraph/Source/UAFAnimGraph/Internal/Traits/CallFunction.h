// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IUpdate.h"
#include "CallFunction.generated.h"

#define UE_API UAFANIMGRAPH_API

UENUM()
enum class EAnimNextCallFunctionCallSite : uint8
{
	// Called each time the trait becomes relevant
	BecomeRelevant,

	// Called on update before the trait and any stack super-traits update
	PreUpdate,

	// Called on update after the trait and any stack super-traits update
	PostUpdate,
};


/** Struct used to help call / pick functions across UAF */
USTRUCT(meta = (DisplayName = "UAF Function"))
struct FUAFCallFunctionInfo 
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = "Call Function", meta = (PinHiddenByDefault, Hidden))
	FRigVMGraphFunctionHeader FunctionHeader;
#endif

	/** The function to call */
	UPROPERTY(EditAnywhere, Category = "Call Function", meta = (PinHiddenByDefault, Hidden))
	FName Function;

	/** Internal event name derived from Function above */
	UPROPERTY(VisibleAnywhere, Category = "Call Function", meta = (PinHiddenByDefault, Hidden))
	FName FunctionEvent;
};


/** A trait that calls a RigVM function at the specified update point */
USTRUCT(meta = (DisplayName = "Call RigVM Function", AllowDuplicates = "true"))
struct FAnimNextCallFunctionSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** @TODO: Replace these properties with FUAFCallFunctionInfo once we determine UAF deprecation path */

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = "Call Function", meta=(Hidden))
	FRigVMGraphFunctionHeader FunctionHeader;
#endif

	/** The function to call */
	UPROPERTY(EditAnywhere, Category = "Call Function", meta=(Hidden))
	FName Function;

	/** Internal event name derived from Function above */
	UPROPERTY(VisibleAnywhere, Category = "Call Function", meta=(Hidden))
	FName FunctionEvent;

	/** The call site to use */
	UPROPERTY(EditAnywhere, Category = "Call Function", meta=(PinHiddenByDefault))
	EAnimNextCallFunctionCallSite CallSite = EAnimNextCallFunctionCallSite::BecomeRelevant;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(CallSite) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextCallFunctionSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
/**
 * FCallFunctionTrait
 *
 * A trait that calls a RigVM function at the specified update point
 */
struct FCallFunctionTrait : FAdditiveTrait, IUpdate
{
	DECLARE_ANIM_TRAIT(FCallFunctionTrait, FAdditiveTrait)

	using FSharedData = FAnimNextCallFunctionSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
	};

	// IUpdate impl
	virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// Calls the RigVM function that is assigned to us
	void CallFunctionForMatchingSite(const TTraitBinding<IUpdate>& InBinding, const FTraitUpdateState& InTraitState, EAnimNextCallFunctionCallSite InCallSite) const;
};
}

#undef UE_API