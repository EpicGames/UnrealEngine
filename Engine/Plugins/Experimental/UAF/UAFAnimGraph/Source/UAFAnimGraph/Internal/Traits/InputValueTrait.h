// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"

#include "InputValueTrait.generated.h"

USTRUCT(meta = (DisplayName = "Input Value", ShowTooltip=true))
struct FUAFInputValueTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// The variable to use as an input value
	UPROPERTY(EditAnywhere, Category="Input Value", meta=(Input, HideSubPins, AllowedType = FUAFValueBundle))
	FAnimNextVariableReference Input;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(Input)

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFInputValueTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	struct FInputValueTrait : FBaseTrait, IUpdate, IEvaluate, IUpdateTraversal, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FInputValueTrait, FBaseTrait)

		using FSharedData = FUAFInputValueTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FAnimNextVariableReference VariableReference;
			FAnimNextGraphInstance* GraphInstance = nullptr;
		};

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		const FInstanceData* UpdateInstanceData(const FTraitBinding& InBinding) const;
	};
}