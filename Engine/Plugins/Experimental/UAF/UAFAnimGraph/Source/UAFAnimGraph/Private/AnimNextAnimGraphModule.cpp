// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphModule.h"
#include "AnimNextAnimGraphSettings.h"
#include "Animation/BlendProfile.h"
#include "Module/AnimNextModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Asset/UAFAnimGraphAssetData.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Factory/AnimGraphFactory.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "Injection/InjectionRequest.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "TraitCore/TraitInterfaceRegistry.h"
#include "TraitCore/TraitRegistry.h"
#include "Traits/ModifyCurveTrait.h"
#include "Traits/BlendSpacePlayerTraitData.h"
#include "Traits/NotifyDispatcherTraitData.h"
#include "Traits/SequencePlayer.h"
#include "Traits/BoneMapping.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Factory/SystemFactory.h"
#include "Factory/UAFSystemFactoryParams.h"
#include "Injection/InjectionSiteTrait.h"
#include "Native/UAFSimplePrePhysicsGraphComponent.h"
#include "Traits/BlendSmoother.h"
#include "Traits/BlendStackTrait.h"
#include "Traits/CallFunction.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Traits/InlineSubGraphTraitData.h"
#include "UAF/UAFAssetFactory.h"


#if WITH_ANIMNEXT_CONSOLE_COMMANDS
#include "HAL/IConsoleManager.h"
#include "UObject/UObjectIterator.h"

#include "TraitCore/NodeDescription.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/TraitTemplate.h"
#endif

namespace UE::UAF::AnimGraph
{

void FAnimNextAnimGraphModule::StartupModule()
{
	// Ensure that AnimNext modules are loaded so we can correctly load plugin content
	FModuleManager::LoadModuleChecked<IModuleInterface>("UAF");
#if WITH_EDITORONLY_DATA
	FModuleManager::LoadModuleChecked<IModuleInterface>("UAFUncookedOnly");
#endif

	// Setup default settings/factories
	{
		UAnimNextAnimGraphSettings* Settings = GetMutableDefault<UAnimNextAnimGraphSettings>();
		Settings->LoadConfig();
	}

	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UUAFAnimGraph::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		{ USkeletalMeshComponent::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::ClassAndParents },
		{ UInstancedSkinnedMeshComponent::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::ClassAndParents },
	};

	RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);
	
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FAnimNextAnimGraph::StaticStruct(),
		FRigVMTrait_AnimNextPublicVariables::StaticStruct(),
		FAnimNextInjectionBlendSettings::StaticStruct(),
		FModifyCurveParameters::StaticStruct(),
		FAnimNextBoneMapping::StaticStruct(),
		FUAFCallFunctionInfo::StaticStruct(),
		FUAFInlineSubGraphInputBinding::StaticStruct()
	};

	RigVMRegistry.RegisterStructTypes(AllowedStructTypes);

	FTraitRegistry::Init();
	FTraitInterfaceRegistry::Init();
	FNodeTemplateRegistry::Init();
	FAnimGraphFactory::Init();

	RegisterGraphFactories();
	RegisterSystemFactories();
	RegisterAssetData();

#if WITH_ANIMNEXT_CONSOLE_COMMANDS
	if (!IsRunningCommandlet())
	{
		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("UAF.ListNodeTemplates"),
			TEXT("Dumps statistics about node templates to the log."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAnimNextAnimGraphModule::ListNodeTemplates),
			ECVF_Default
		));
		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("UAF.Systems"),
			TEXT("Dumps statistics about systems to the log."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAnimNextAnimGraphModule::ListAnimationGraphs),
			ECVF_Default
		));
	}
#endif
}

void FAnimNextAnimGraphModule::ShutdownModule()
{
	UnregisterAssetData();
	
#if WITH_ANIMNEXT_CONSOLE_COMMANDS
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}
	ConsoleCommands.Empty();
#endif

	FAnimGraphFactory::Destroy();
	FNodeTemplateRegistry::Destroy();
	FTraitInterfaceRegistry::Destroy();
	FTraitRegistry::Destroy();

	LoadedGraphs.Reset();
}

void FAnimNextAnimGraphModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(LoadedGraphs);
}

FString FAnimNextAnimGraphModule::GetReferencerName() const
{
	return TEXT("AnimNextAnimGraphModule");
}

void FAnimNextAnimGraphModule::RegisterGraphFactories()
{
	{
		// Register default graph host
		FAnimNextFactoryParams Params;
		Params.AddTraitStruct<FBlendStackCoreTraitData>(ETraitVariableMapping::None, 0);
		Params.AddTraitStruct<FBlendSmootherCoreData>(ETraitVariableMapping::None, 0);
		Params.AddTraitStruct<FInjectionSiteTraitData>(ETraitVariableMapping::All, 0);
#if UE_TRACE_ENABLED
		Params.AddInitializeTask([](const UE::UAF::FInstanceTaskContext& InContext)
			{
				InContext.GetAssetInstance().SetDebugName(TEXT("Default Graph Host"));
			});
#endif
		FAnimGraphFactory::RegisterAsset<FUAFGraphFactoryAsset_Graph>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_Graph& AssetData, FAnimNextFactoryParams& InOutParams)
				{
					ensure(InOutParams.AccessTraitStruct<FInjectionSiteTraitData>(0, [&AssetData, &InOutParams](FInjectionSiteTraitData& InStruct)
						{
						InStruct.Graph.Asset = AssetData.AnimationGraph;
						}));

					if (AssetData.AnimationGraph != nullptr)
					{
						InOutParams.AddPublicVariablesAsset(AssetData.AnimationGraph); // Add the graph's own variables

						// Copy all references from the graph to our host so users can set variable on graphs
						for (const TObjectPtr<const UUAFRigVMAsset>& VariableAsset : AssetData.AnimationGraph->ReferencedVariableAssets)
						{
							InOutParams.AddPublicVariablesAsset(VariableAsset);
						}

						for (const TScriptInterface<const IRigVMRuntimeAssetInterface>& VariableRigVMAsset : AssetData.AnimationGraph->
						     ReferencedVariableRigVMAssets)
						{
							InOutParams.AddPublicVariablesRigVMAsset(VariableRigVMAsset);
						}

						for (const TObjectPtr<const UScriptStruct>& VariableStruct : AssetData.AnimationGraph->ReferencedVariableStructs)
						{
							InOutParams.AddPublicVariablesStructByType(VariableStruct);
						}
					}
				});
	}

	{
		// Register UAnimSequence 'player'
		FAnimNextFactoryParams Params;
		Params.AddTraitStruct<FSequencePlayerData>(ETraitVariableMapping::All, 0);
		Params.AddTraitStruct<FNotifyDispatcherData>(ETraitVariableMapping::None, 0);

		FAnimGraphFactory::RegisterAsset<FUAFGraphFactoryAsset_Animation>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_Animation& AssetData, FAnimNextFactoryParams& InOutParams)
			{
				ensure(InOutParams.AccessTraitStruct<FSequencePlayerData>(0, [&AssetData, &InOutParams](FSequencePlayerData& InStruct)
				{
					InStruct.AnimSequence = AssetData.AnimationSequence;
					InStruct.PlayRate = AssetData.PlayRate;
					InStruct.LoopMode = AssetData.LoopMode;
					InStruct.StartPosition = AssetData.CalculateStartTime();
				}));
			});
	}

	{
		// Register UBlendSpace 'player'
		FAnimNextFactoryParams Params;
		Params.AddTraitStruct<FBlendSpacePlayerData>(ETraitVariableMapping::All, 0);

		FAnimGraphFactory::RegisterAsset<FUAFGraphFactoryAsset_BlendSpace>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_BlendSpace& AssetData, FAnimNextFactoryParams& InOutParams)
			{
				ensure(InOutParams.AccessTraitStruct<FBlendSpacePlayerData>(0, [&AssetData, &InOutParams](FBlendSpacePlayerData& InStruct)
				{
					InStruct.BlendSpace = AssetData.BlendSpace;
					InStruct.XAxisSamplePoint = AssetData.XAxisSamplePoint;
					InStruct.YAxisSamplePoint = AssetData.YAxisSamplePoint;
					InStruct.PlayRate = AssetData.PlayRate;
				}));
			});
	}
}

void FAnimNextAnimGraphModule::RegisterSystemFactories()
{
	{
		// Register default graph host
		FUAFSystemFactoryParams Params;
		Params.AddComponent<FUAFSimplePrePhysicsGraphComponent>();
		Params.AddPublicVariablesStruct<FInjectionSiteTraitData>();

		FSystemFactory::RegisterAsset<FUAFGraphFactoryAsset_Graph>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_Graph& AssetData, FUAFSystemFactoryParams& Params)
			{
				ensure(Params.AccessVariablesStruct<FInjectionSiteTraitData>([&AssetData, &Params](FInjectionSiteTraitData& InStruct)
				{
					InStruct.Graph.Asset = AssetData.AnimationGraph;
				}));

					// Add the same variables as the graph has to the system, so public set variable API will function correctly
					if (AssetData.AnimationGraph != nullptr)
					{
						Params.AddPublicVariablesAsset(AssetData.AnimationGraph); // Add the graph's own variables
						for (const TObjectPtr<const UUAFRigVMAsset>& VariableAsset : AssetData.AnimationGraph->ReferencedVariableAssets)
						{
							Params.AddPublicVariablesAsset(VariableAsset);
						}

						for (const TScriptInterface<const IRigVMRuntimeAssetInterface>& VariableRigVMAsset : AssetData.AnimationGraph->ReferencedVariableRigVMAssets)
						{
							Params.AddPublicVariablesRigVMAsset(VariableRigVMAsset);
						}

						for (const TObjectPtr<const UScriptStruct>& VariableStruct : AssetData.AnimationGraph->ReferencedVariableStructs)
						{
							Params.AddPublicVariablesStructByType(VariableStruct);
						}
					}
			});
	}

	{
		// Register UAnimSequence 'player'
		FUAFSystemFactoryParams Params;
		Params.AddComponent<FUAFSimplePrePhysicsGraphComponent>();
		Params.AddPublicVariablesStruct<FInjectionSiteTraitData>();
		Params.AddPublicVariablesStruct<FSequencePlayerData>();

		FSystemFactory::RegisterAsset<FUAFGraphFactoryAsset_Animation>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_Animation& AssetData, FUAFSystemFactoryParams& InOutParams)
			{
				ensure(InOutParams.AccessVariablesStruct<FInjectionSiteTraitData>([&AssetData, &InOutParams](FInjectionSiteTraitData& InStruct)
				{
					InStruct.Graph.Asset = AssetData.AnimationSequence;
				}));

				ensure(InOutParams.AccessVariablesStruct<FSequencePlayerData>([&AssetData, &InOutParams](FSequencePlayerData& InStruct)
				{
					InStruct.AnimSequence = AssetData.AnimationSequence;
					InStruct.LoopMode = AssetData.LoopMode;
					InStruct.PlayRate = AssetData.PlayRate;
					InStruct.StartPosition = AssetData.CalculateStartTime();
				}));
			});
	}

	{
		// Register UBlendSpace 'player'
		FUAFSystemFactoryParams Params;
		Params.AddComponent<FUAFSimplePrePhysicsGraphComponent>();
		Params.AddPublicVariablesStruct<FInjectionSiteTraitData>();
		Params.AddPublicVariablesStruct<FBlendSpacePlayerData>();

		FSystemFactory::RegisterAsset<FUAFGraphFactoryAsset_BlendSpace>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_BlendSpace& AssetData, FUAFSystemFactoryParams& InOutParams)
			{
				ensure(InOutParams.AccessVariablesStruct<FInjectionSiteTraitData>([&AssetData, &InOutParams](FInjectionSiteTraitData& InStruct)
				{
					InStruct.Graph.Asset = AssetData.BlendSpace;
				}));

				ensure(InOutParams.AccessVariablesStruct<FBlendSpacePlayerData>([&AssetData, &InOutParams](FBlendSpacePlayerData& InStruct)
				{
					InStruct.BlendSpace = AssetData.BlendSpace;
				}));
			});
	}
}

void FAnimNextAnimGraphModule::RegisterAssetData()
{
	UAFAnimGraphClassPath = FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_Graph, UUAFAnimGraph>(
		[](const UUAFAnimGraph* AnimGraph)
			{
				return FUAFGraphFactoryAsset_Graph(AnimGraph);
			});

	AnimSequenceClassPath = FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_Animation, UAnimSequence>(
		[](const UAnimSequence* AnimSequence)
			{
				return FUAFGraphFactoryAsset_Animation(AnimSequence);
			});

	BlendSpaceClassPath = FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_BlendSpace, UBlendSpace>(
		[](const UBlendSpace* BlendSpace)
			{
				return FUAFGraphFactoryAsset_BlendSpace(BlendSpace);
			});
}

void FAnimNextAnimGraphModule::UnregisterAssetData()
{
	FAssetDataFactory::UnregisterAsset(UAFAnimGraphClassPath);
	FAssetDataFactory::UnregisterAsset(AnimSequenceClassPath);
	FAssetDataFactory::UnregisterAsset(BlendSpaceClassPath);
}

#if WITH_ANIMNEXT_CONSOLE_COMMANDS
void FAnimNextAnimGraphModule::ListNodeTemplates(const TArray<FString>& Args)
{
	// Turn off log times to make diff-ing easier
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	// Make sure to log everything
	const ELogVerbosity::Type OldVerbosity = LogAnimation.GetVerbosity();
	LogAnimation.SetVerbosity(ELogVerbosity::All);

	const FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();
	const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

	UE_LOGF(LogAnimation, Log, "===== UAF Node Templates =====");
	UE_LOGF(LogAnimation, Log, "Template Buffer Size: %zu bytes", NodeTemplateRegistry.TemplateBuffer.GetAllocatedSize());

	for (auto It = NodeTemplateRegistry.TemplateUIDToHandleMap.CreateConstIterator(); It; ++It)
	{
		const FNodeTemplateRegistryHandle Handle = It.Value();
		const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(Handle);

		const uint32 NumTraits = NodeTemplate->GetNumTraits();

		UE_LOGF(LogAnimation, Log, "[%x] has %u traits ...", NodeTemplate->GetUID(), NumTraits);
		UE_LOGF(LogAnimation, Log, "    Template Size: %u bytes", NodeTemplate->GetNodeTemplateSize());
		UE_LOGF(LogAnimation, Log, "    Shared Data Size: %u bytes", NodeTemplate->GetNodeSharedDataSize());
		UE_LOGF(LogAnimation, Log, "    Instance Data Size: %u bytes", NodeTemplate->GetNodeInstanceDataSize());
		UE_LOGF(LogAnimation, Log, "    Traits ...");

		const FTraitTemplate* TraitTemplates = NodeTemplate->GetTraits();
		for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FTraitTemplate* TraitTemplate = TraitTemplates + TraitIndex;
			const FTrait* Trait = TraitRegistry.Find(TraitTemplate->GetRegistryHandle());
			const FString TraitName = Trait != nullptr ? Trait->GetTraitName() : TEXT("<Unknown>");

			const uint32 NextTraitIndex = TraitIndex + 1;
			const uint32 EndOfNextTraitSharedData = NextTraitIndex < NumTraits ? TraitTemplates[NextTraitIndex].GetNodeSharedOffset() : NodeTemplate->GetNodeSharedDataSize();
			const uint32 TraitSharedDataSize = EndOfNextTraitSharedData - TraitTemplate->GetNodeSharedOffset();

			const uint32 EndOfNextTraitInstanceData = NextTraitIndex < NumTraits ? TraitTemplates[NextTraitIndex].GetNodeInstanceOffset() : NodeTemplate->GetNodeInstanceDataSize();
			const uint32 TraitInstanceDataSize = EndOfNextTraitInstanceData - TraitTemplate->GetNodeInstanceOffset();

			UE_LOGF(LogAnimation, Log, "            %u: [%x] %ls (%ls)", TraitIndex, TraitTemplate->GetUID().GetUID(), *TraitName, TraitTemplate->GetMode() == ETraitMode::Base ? TEXT("Base") : TEXT("Additive"));
			UE_LOGF(LogAnimation, Log, "                Shared Data: [Offset: %u bytes, Size: %u bytes]", TraitTemplate->GetNodeSharedOffset(), TraitSharedDataSize);
			if (TraitTemplate->HasLatentProperties() && Trait != nullptr)
			{
				UE_LOGF(LogAnimation, Log, "                Shared Data Latent Property Handles: [Offset: %u bytes, Count: %u]", TraitTemplate->GetNodeSharedLatentPropertyHandlesOffset(), Trait->GetNumLatentTraitProperties());
			}
			UE_LOGF(LogAnimation, Log, "                Instance Data: [Offset: %u bytes, Size: %u bytes]", TraitTemplate->GetNodeInstanceOffset(), TraitInstanceDataSize);
		}
	}

	LogAnimation.SetVerbosity(OldVerbosity);
}

void FAnimNextAnimGraphModule::ListAnimationGraphs(const TArray<FString>& Args)
{
	// Turn off log times to make diff-ing easier
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	// Make sure to log everything
	const ELogVerbosity::Type OldVerbosity = LogAnimation.GetVerbosity();
	LogAnimation.SetVerbosity(ELogVerbosity::All);

	TArray<const UUAFAnimGraph*> AnimationGraphs;

	for (TObjectIterator<UUAFAnimGraph> It; It; ++It)
	{
		AnimationGraphs.Add(*It);
	}

	struct FCompareObjectNames
	{
		FORCEINLINE bool operator()(const UUAFAnimGraph& Lhs, const UUAFAnimGraph& Rhs) const
		{
			return Lhs.GetPathName().Compare(Rhs.GetPathName()) < 0;
		}
	};
	AnimationGraphs.Sort(FCompareObjectNames());

	const FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();
	const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
	const bool bDetailedOutput = true;

	UE_LOGF(LogAnimation, Log, "===== UAF Modules =====");
	UE_LOGF(LogAnimation, Log, "Num Graphs: %u", AnimationGraphs.Num());

	for (const UUAFAnimGraph* AnimationGraph : AnimationGraphs)
	{
		uint32 TotalInstanceSize = 0;
		uint32 NumNodes = 0;
		{
			// We always have a node at offset 0
			int32 NodeOffset = 0;

			while (NodeOffset < AnimationGraph->SharedDataBuffer.Num())
			{
				const FNodeDescription* NodeDesc = reinterpret_cast<const FNodeDescription*>(&AnimationGraph->SharedDataBuffer[NodeOffset]);

				TotalInstanceSize += NodeDesc->GetNodeInstanceDataSize();
				NumNodes++;

				const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(NodeDesc->GetTemplateHandle());
				NodeOffset += NodeTemplate->GetNodeSharedDataSize();
			}
		}

		UE_LOGF(LogAnimation, Log, "    %ls ...", *AnimationGraph->GetPathName());
		UE_LOGF(LogAnimation, Log, "        Shared Data Size: %.2f KB", double(AnimationGraph->SharedDataBuffer.Num()) / 1024.0);
		UE_LOGF(LogAnimation, Log, "        Max Instance Data Size: %.2f KB", double(TotalInstanceSize) / 1024.0);
		UE_LOGF(LogAnimation, Log, "        Num Nodes: %u", NumNodes);

		if (bDetailedOutput)
		{
			// We always have a node at offset 0
			int32 NodeOffset = 0;

			while (NodeOffset < AnimationGraph->SharedDataBuffer.Num())
			{
				const FNodeDescription* NodeDesc = reinterpret_cast<const FNodeDescription*>(&AnimationGraph->SharedDataBuffer[NodeOffset]);
				const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(NodeDesc->GetTemplateHandle());

				const uint32 NumTraits = NodeTemplate->GetNumTraits();

				UE_LOGF(LogAnimation, Log, "        Node %u: [Template %x with %u traits]", NodeDesc->GetUID().GetNodeIndex(), NodeTemplate->GetUID(), NumTraits);
				UE_LOGF(LogAnimation, Log, "            Shared Data: [Offset: %u bytes, Size: %u bytes]", NodeOffset, NodeTemplate->GetNodeSharedDataSize());
				UE_LOGF(LogAnimation, Log, "            Instance Data Size: %u bytes", NodeDesc->GetNodeInstanceDataSize());
				UE_LOGF(LogAnimation, Log, "            Traits ...");

				const FTraitTemplate* TraitTemplates = NodeTemplate->GetTraits();
				for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
				{
					const FTraitTemplate* TraitTemplate = TraitTemplates + TraitIndex;
					const FTrait* Trait = TraitRegistry.Find(TraitTemplate->GetRegistryHandle());
					const FString TraitName = Trait != nullptr ? Trait->GetTraitName() : TEXT("<Unknown>");

					const uint32 NextTraitIndex = TraitIndex + 1;
					const uint32 EndOfNextTraitSharedData = NextTraitIndex < NumTraits ? TraitTemplates[NextTraitIndex].GetNodeSharedOffset() : NodeTemplate->GetNodeSharedDataSize();
					const uint32 TraitSharedDataSize = EndOfNextTraitSharedData - TraitTemplate->GetNodeSharedOffset();

					const uint32 EndOfNextTraitInstanceData = NextTraitIndex < NumTraits ? TraitTemplates[NextTraitIndex].GetNodeInstanceOffset() : NodeTemplate->GetNodeInstanceDataSize();
					const uint32 TraitInstanceDataSize = EndOfNextTraitInstanceData - TraitTemplate->GetNodeInstanceOffset();

					UE_LOGF(LogAnimation, Log, "                    %u: [%x] %ls (%ls)", TraitIndex, TraitTemplate->GetUID().GetUID(), *TraitName, TraitTemplate->GetMode() == ETraitMode::Base ? TEXT("Base") : TEXT("Additive"));
					UE_LOGF(LogAnimation, Log, "                        Shared Data: [Offset: %u bytes, Size: %u bytes]", TraitTemplate->GetNodeSharedOffset(), TraitSharedDataSize);
					if (TraitTemplate->HasLatentProperties() && Trait != nullptr)
					{
						UE_LOGF(LogAnimation, Log, "                        Shared Data Latent Property Handles: [Offset: %u bytes, Count: %u]", TraitTemplate->GetNodeSharedLatentPropertyHandlesOffset(), Trait->GetNumLatentTraitProperties());
					}
					UE_LOGF(LogAnimation, Log, "                        Instance Data: [Offset: %u bytes, Size: %u bytes]", TraitTemplate->GetNodeInstanceOffset(), TraitInstanceDataSize);
				}

				NodeOffset += NodeTemplate->GetNodeSharedDataSize();
			}
		}
	}

	LogAnimation.SetVerbosity(OldVerbosity);
}
#endif

}

IMPLEMENT_MODULE(UE::UAF::AnimGraph::FAnimNextAnimGraphModule, UAFAnimGraph)
