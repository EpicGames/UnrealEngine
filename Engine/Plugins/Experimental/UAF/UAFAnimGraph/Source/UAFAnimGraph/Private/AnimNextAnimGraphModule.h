// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "UObject/GCObject.h"

// Enable console commands only in development builds when logging is enabled
#define WITH_ANIMNEXT_CONSOLE_COMMANDS (!UE_BUILD_SHIPPING && !NO_LOGGING)

class UUAFAnimGraph;
class UAnimNextAnimGraphSettings;

namespace UE::UAF::AnimGraph
{

class FAnimNextAnimGraphModule : public IModuleInterface, public FGCObject
{
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// Register asset class -> graph factory mappings
	void RegisterGraphFactories();

	// Register asset class -> system factory mappings
	void RegisterSystemFactories();

	void RegisterAssetData();
	void UnregisterAssetData();

	friend UAnimNextAnimGraphSettings;
	
	// References to loaded default graphs to prevent GC.
	// They must exist here rather than on the UAnimNextAnimGraphSettings CDO to avoid breaking DisregardForGC assumptions
	TArray<TObjectPtr<const UUAFAnimGraph>> LoadedGraphs;

#if WITH_ANIMNEXT_CONSOLE_COMMANDS
	void ListNodeTemplates(const TArray<FString>& Args);
	void ListAnimationGraphs(const TArray<FString>& Args);

	TArray<IConsoleObject*> ConsoleCommands;
#endif

	FTopLevelAssetPath UAFAnimGraphClassPath;
	FTopLevelAssetPath AnimSequenceClassPath;
	FTopLevelAssetPath BlendSpaceClassPath;
};

}
