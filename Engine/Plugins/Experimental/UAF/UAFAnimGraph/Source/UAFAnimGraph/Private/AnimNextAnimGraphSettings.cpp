// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphSettings.h"
#include "Factory/AnimGraphFactory.h"
#include "UAF/UAFAssetFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraphSettings)

TArray<UClass*> UAnimNextAnimGraphSettings::GetAllowedAssetClasses()
{
	TArray<UClass*> Array;
	Array = UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset>();
	return Array;
}
