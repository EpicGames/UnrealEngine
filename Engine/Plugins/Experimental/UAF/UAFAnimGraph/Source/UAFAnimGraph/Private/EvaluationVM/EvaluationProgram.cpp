// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/EvaluationProgram.h"

#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/EvaluationVM.h"
#include "AnimNextAnimGraphStats.h"
#include "Misc/StringBuilder.h"

DEFINE_STAT(STAT_AnimNext_EvaluationProgram_Execute);

namespace UE::UAF
{
	bool FEvaluationProgram::IsEmpty() const
	{
		return Tasks.IsEmpty();
	}

	void FEvaluationProgram::Execute(FEvaluationVM& VM) const
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_EvaluationProgram_Execute);

		if (VM.IsValid())
		{
			for (const TSharedPtr<FAnimNextEvaluationTask>& Task : Tasks)
			{
				Task->Execute(VM);
			}
		}
	}

	FString FEvaluationProgram::ToString() const
	{
		TStringBuilder<0> Result;

		FString PropertyValue;
		for (const TSharedPtr<FAnimNextEvaluationTask>& Task : Tasks)
		{
			const UScriptStruct* TaskStruct = Task->GetStruct();

			Result.Appendf(TEXT("%s\n"), *TaskStruct->GetName());

			for (TFieldIterator<FProperty> It(TaskStruct); It; ++It)
			{
				if (const FProperty* Property = *It)
				{
					PropertyValue.Reset();

					const bool bIsDefaultValue = !Property->ExportText_InContainer(0, PropertyValue, Task.Get(), nullptr, nullptr, PPF_SerializedAsImportText);

					if (bIsDefaultValue)
					{
						if (Property->IsA<FBoolProperty>())
						{
							PropertyValue = TEXT("False");
						}
						else if (Property->IsA<FNumericProperty>())
						{
							PropertyValue = TEXT("0");
						}
					}

					Result.Appendf(TEXT("\t%s: %s\n"), *Property->GetName(), *PropertyValue);
				}
			}
		}

		return FString(Result);
	}
}
