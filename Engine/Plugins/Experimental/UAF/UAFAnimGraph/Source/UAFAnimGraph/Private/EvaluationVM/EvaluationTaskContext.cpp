// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/EvaluationTaskContext.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"

namespace UE::UAF
{
	FEvaluationTaskContext::FEvaluationTaskContext() = default;

	FEvaluationTaskContext::FEvaluationTaskContext(TNonNullPtr<const UAbstractSkeletonSetBinding> InSetBinding, const USkeletalMesh* InSkeletalMesh, FName InSetName)
	{
		if (const FAttributeBindingDataPtr& InBindingData = GAttributeBindingDataCache.GetOrAdd(InSetBinding, InSkeletalMesh))
		{
			if (FAttributeNamedSetPtr InNamedSet = InBindingData->FindNamedSet(InSetName))
			{
				SetBinding = InSetBinding;
				SkeletalMesh = InSkeletalMesh;
				Skeleton = InSetBinding->GetSkeleton();
				BindingData = InBindingData;
				NamedSet = MoveTemp(InNamedSet);
				DefaultAttributeValues = NamedSet->GetDefaultAttributeValues();
			}
		}
	}

	const UAbstractSkeletonSetBinding* FEvaluationTaskContext::GetSetBinding() const
	{
		return SetBinding;
	}

	const USkeleton* FEvaluationTaskContext::GetSkeleton() const
	{
		return Skeleton;
	}

	const USkeletalMesh* FEvaluationTaskContext::GetSkeletalMesh() const
	{
		return SkeletalMesh;
	}

	const FAttributeNamedSetPtr& FEvaluationTaskContext::GetNamedSet() const
	{
		return NamedSet;
	}

	bool FEvaluationTaskContext::SetNamedSet(FName InSetName)
	{
		if (BindingData)
		{
			if (FAttributeNamedSetPtr InNamedSet = BindingData->FindNamedSet(InSetName))
			{
				NamedSet = MoveTemp(InNamedSet);
				DefaultAttributeValues = NamedSet->GetDefaultAttributeValues();
				return true;
			}
		}

		// Named set not found
		return false;
	}

	const FValueBundlePtr& FEvaluationTaskContext::GetDefaultAttributeValues() const
	{
		return DefaultAttributeValues;
	}

	bool FEvaluationTaskContext::IsValid() const
	{
		return SetBinding != nullptr;
	}
}
