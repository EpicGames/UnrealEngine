// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/EvaluationVM.h"

#include "Animation/Skeleton.h"
#include "BoneContainer.h"
#include "EvaluationVM/KeyframeState.h"

namespace UE::UAF
{
	const FEvaluationVMStackName KEYFRAME_STACK_NAME = FName(TEXT("KeyframeStack"));
	const FEvaluationVMStackName ATTRIBUTE_STACK_NAME = FName(TEXT("AttributeStack"));

	FEvaluationVMStack::~FEvaluationVMStack()
	{
		FEvaluationVMStackEntry* EntryPtr = Top;
		while (EntryPtr != nullptr)
		{
			if (TypeDestructor)
			{
				TypeDestructor(EntryPtr->GetValuePtr());
			}

			FEvaluationVMStackEntry* PrevEntryPtr = EntryPtr->Prev;

			FMemory::Free(EntryPtr);

			EntryPtr = PrevEntryPtr;
		}
	}

	FEvaluationVM::FEvaluationVM(EEvaluationFlags InEvaluationFlags, const UE::UAF::FReferencePose& InReferencePose, int32 InCurrentLOD)
		: ReferencePose(&InReferencePose)
		, CurrentLOD(InCurrentLOD)
		, EvaluationFlags(InEvaluationFlags)
	{
		// Push an invalid context
		EvaluationContextStack.Push(FEvaluationTaskContext());
	}

	FEvaluationVM::FEvaluationVM(EEvaluationFlags InEvaluationFlags, TNonNullPtr<const UAbstractSkeletonSetBinding> InSetBinding, const USkeletalMesh* InSkeletalMesh, FName InSetName, int32 InCurrentLOD)
		: ValueTransformerMap(FValueRuntimeRegistry::Get().GetTransformerMap())
		, CurrentLOD(InCurrentLOD)
		, EvaluationFlags(InEvaluationFlags)
	{
		EvaluationContextStack.Push(FEvaluationTaskContext(InSetBinding, InSkeletalMesh, InSetName));
	}

	// REMOVE ME: Exclusively for testing that before/after yields the same results
	FEvaluationVM::FEvaluationVM(EEvaluationFlags InEvaluationFlags, const FReferencePose& InReferencePose, const UAbstractSkeletonSetBinding* InSetBinding, const USkeletalMesh* InSkeletalMesh, FName InSetName, int32 InCurrentLOD)
		: ReferencePose(&InReferencePose)
		, ValueTransformerMap(FValueRuntimeRegistry::Get().GetTransformerMap())
		, CurrentLOD(InCurrentLOD)
		, EvaluationFlags(InEvaluationFlags)
	{
		if (InSetBinding != nullptr)
		{
			EvaluationContextStack.Push(FEvaluationTaskContext(InSetBinding, InSkeletalMesh, InSetName));
		}
		else
		{
			// Push an invalid context
			EvaluationContextStack.Push(FEvaluationTaskContext());
		}
	}

	bool FEvaluationVM::IsValid() const
	{
		return ReferencePose != nullptr || EvaluationContextStack.Last().IsValid();
	}

	void FEvaluationVM::Shrink()
	{
		InternalStacks.Shrink();
		EvaluationContextStack.Shrink();
	}

	FKeyframeState FEvaluationVM::MakeReferenceKeyframe(bool bAdditiveKeyframe) const
	{
		// TODO: It would be great if we could support immutable poses that we can push/pop like normal mutable poses
		// This would allow us to cheaply push reference/identity poses
		// Tasks that consume poses can then choose to re-use a mutable pose (to avoid allocating a new one) and only
		// allocate one if all their inputs are immutable.
		// This would reduce the number of required intermediate poses.

		FKeyframeState Keyframe;

		if (EnumHasAnyFlags(EvaluationFlags, EEvaluationFlags::Bones) && ReferencePose != nullptr)
		{
			const bool bInitWithRefPose = true;

			Keyframe.Pose.PrepareForLOD(*ReferencePose, CurrentLOD, bInitWithRefPose, bAdditiveKeyframe);
		}

		if (EnumHasAnyFlags(EvaluationFlags, EEvaluationFlags::Curves))
		{
			Keyframe.Curves.SetFilter(&CurveFilter);
		}

		return Keyframe;
	}

	FKeyframeState FEvaluationVM::MakeUninitializedKeyframe(bool bAdditiveKeyframe) const
	{
		FKeyframeState Keyframe;

		if (EnumHasAnyFlags(EvaluationFlags, EEvaluationFlags::Bones) && ReferencePose != nullptr)
		{
			const bool bInitWithRefPose = false;

			Keyframe.Pose.PrepareForLOD(*ReferencePose, CurrentLOD, bInitWithRefPose, bAdditiveKeyframe);
		}

		if (EnumHasAnyFlags(EvaluationFlags, EEvaluationFlags::Curves))
		{
			Keyframe.Curves.SetFilter(&CurveFilter);
		}

		return Keyframe;
	}

	const FEvaluationTaskContext& FEvaluationVM::GetActiveEvaluationContext() const
	{
		return EvaluationContextStack.Last();
	}

	const FAttributeNamedSetPtr& FEvaluationVM::GetActiveNamedSet() const
	{
		return EvaluationContextStack.Last().GetNamedSet();
	}

	bool FEvaluationVM::PushEvaluationContext(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName)
	{
		FEvaluationTaskContext NewContext(SetBinding, SkeletalMesh, SetName);
		if (NewContext.IsValid())
		{
			EvaluationContextStack.Push(MoveTemp(NewContext));
			return true;
		}

		// New evaluation context isn't valid
		return false;
	}

	bool FEvaluationVM::PushEvaluationContext(FName SetName)
	{
		// Copy the currently active context
		FEvaluationTaskContext NewContext = EvaluationContextStack.Last();
		if (NewContext.SetNamedSet(SetName))
		{
			EvaluationContextStack.Push(MoveTemp(NewContext));
			return true;
		}

		// Failed to find the named set in our currently active binding
		return false;
	}

	bool FEvaluationVM::PopEvaluationContext()
	{
		if (EvaluationContextStack.Num() > 1)
		{
			EvaluationContextStack.Pop(EAllowShrinking::No);
			return true;
		}

		// Cannot pop an entry from an empty stack or the last entry
		return false;
	}

	const FValueTransformerMapPtr& FEvaluationVM::GetTransformerMap() const
	{
		return ValueTransformerMap;
	}

	FEvaluationVMStack& FEvaluationVM::GetOrCreateStack(const FEvaluationVMStackName& StackName, uint32 TypeID)
	{
		if (FEvaluationVMStack* Stack = InternalStacks.FindByHash(StackName.NameHash, StackName.Name))
		{
			return *Stack;
		}

		FEvaluationVMStack& Stack = InternalStacks.AddByHash(StackName.NameHash, StackName.Name, FEvaluationVMStack());
		Stack.Name = StackName.Name;
		Stack.TypeID = TypeID;

		return Stack;
	}

	FEvaluationVMStack* FEvaluationVM::FindStack(const FEvaluationVMStackName& StackName, uint32 TypeID)
	{
		FEvaluationVMStack* Stack = InternalStacks.FindByHash(StackName.NameHash, StackName.Name);
		if (Stack == nullptr)
		{
			return nullptr;
		}

		if (!ensureMsgf(Stack->TypeID == TypeID, TEXT("Type mismatch! This evaluation stack is being queried with a different type than it was created with")))
		{
			return nullptr;
		}

		return Stack;
	}

	const FEvaluationVMStack* FEvaluationVM::FindStack(const FEvaluationVMStackName& StackName, uint32 TypeID) const
	{
		const FEvaluationVMStack* Stack = InternalStacks.FindByHash(StackName.NameHash, StackName.Name);
		if (Stack == nullptr)
		{
			return nullptr;
		}

		if (!ensureMsgf(Stack->TypeID == TypeID, TEXT("Type mismatch! This evaluation stack is being queried with a different type than it was created with")))
		{
			return nullptr;
		}

		return Stack;
	}

	const void* FEvaluationVM::PeekValueImpl(const FEvaluationVMStackName& StackName, uint32 TypeID, uint32 Offset) const
	{
		const FEvaluationVMStack* Stack = FindStack(StackName, TypeID);
		if (Stack == nullptr)
		{
			// Stack not found
			return nullptr;
		}

		const FEvaluationVMStackEntry* EntryPtr = Stack->Top;
		if (EntryPtr == nullptr)
		{
			// Stack is empty
			return nullptr;
		}

		uint32 Count = 0;
		while (EntryPtr != nullptr && Count < Offset)
		{
			EntryPtr = EntryPtr->Prev;
			Count++;
		}

		if (EntryPtr == nullptr)
		{
			// Offset is too far, not enough entries
			return nullptr;
		}

		return EntryPtr->GetValuePtr();
	}
}
