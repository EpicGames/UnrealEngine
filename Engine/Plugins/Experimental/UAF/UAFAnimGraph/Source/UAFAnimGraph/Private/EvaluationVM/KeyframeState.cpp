// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/KeyframeState.h"
#include "Graph/AnimNext_LODPose.h"
#include "RemapPoseDataPool.h"
#include "AnimNextAnimGraphStats.h"

DEFINE_STAT(STAT_AnimNext_Keyframe_CopyFrom);

namespace UE::UAF
{
	void FKeyframeState::CopyFrom(const FAnimNextGraphLODPose& Source)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_Keyframe_CopyFrom);

		if (!Source.LODPose.IsValid())
		{
			return;
		}

		const FReferencePose* SourceRef = Source.LODPose.RefPose;
		const FReferencePose* TargetRef = Pose.RefPose;

		if (SourceRef != nullptr && TargetRef != nullptr && FRemapPoseDataPool::NeedsRemapping(*SourceRef, *TargetRef))
		{
			// Cross-skeleton: Fill ref pose transforms for unmapped bones, then remap mapped ones.
			Pose.SetRefPose(Source.LODPose.IsAdditive());
			Pose.Flags = Source.LODPose.Flags;

			const FRemapPoseData& RemapData = FRemapPoseDataPool::Get().GetRemapData(*SourceRef, *TargetRef);
			RemapData.RemapBones(Source.LODPose, Pose);
			RemapData.RemapAttributes(Source.LODPose, Source.Attributes, Pose, Attributes);
		}
		else
		{
			Pose.CopyFrom(Source.LODPose);
			Attributes.CopyFrom(Source.Attributes);
		}

		Curves.CopyFrom(Source.Curves);
	}
}
