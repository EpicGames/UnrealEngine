// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/SerializableEvaluationProgram.h"
#include "EvaluationVM/EvaluationProgram.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SerializableEvaluationProgram)

FSerializableEvaluationProgram::FSerializableEvaluationProgram(const UE::UAF::FEvaluationProgram& Other)
{
	for (const TSharedPtr<FAnimNextEvaluationTask>& Task : Other.Tasks)
	{
		Tasks.Add(FInstancedStruct(Task->GetStruct()));
		Task->GetStruct()->CopyScriptStruct(Tasks.Last().GetMutableMemory(), &*Task);
	}
}
