// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/ApplyAdditiveKeyframe.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ApplyAdditiveKeyframe)

FAnimNextApplyAdditiveKeyframeTask FAnimNextApplyAdditiveKeyframeTask::Make(float BlendWeight)
{
	FAnimNextApplyAdditiveKeyframeTask Task;
	Task.BlendWeight = BlendWeight;

	return Task;
}

FAnimNextApplyAdditiveKeyframeTask FAnimNextApplyAdditiveKeyframeTask::Make(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn)
{
	FAnimNextApplyAdditiveKeyframeTask Task;
	Task.AlphaSourceCurveName = AlphaSourceCurveName;
	Task.AlphaCurveInputIndex = AlphaCurveInputIndex;
	Task.InputScaleBiasClampFn = MoveTemp(InputScaleBiasClampFn);
	return Task;
}

void FAnimNextApplyAdditiveKeyframeTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		TUniquePtr<FValueBundle> AdditiveCollection;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, AdditiveCollection))
		{
			// We have no inputs, nothing to do
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "FAnimNextApplyAdditiveKeyframeTask::Execute: Could not apply additive keyframe - no inputs provided. Pushing a ref pose instead.");
			return;
		}

		TUniquePtr<FValueBundle> BaseCollection;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, BaseCollection))
		{
			// We have a single input, discard it since it must be the additive pose, either way something went wrong
			// Push the reference pose since we'll expect a non-additive pose
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "FAnimNextApplyAdditiveKeyframeTask::Execute: Could not apply additive keyframe - only a single input was provided. Pushing a ref pose instead.");
			return;
		}

		const FValueSpace AdditiveSpace = AdditiveCollection->GetValueSpace();
		if (!AdditiveSpace.IsAdditive())
		{
			// Additive input must be additive, push reference pose if not the case
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "FAnimNextApplyAdditiveKeyframeTask::Execute: Could not apply additive keyframe - The expected additive pose was not of an additive type. Pushing a ref pose instead.");
			return;
		}

		const float AdditiveWeight = GetInterpolationAlpha(*BaseCollection, *AdditiveCollection);

		bool bConvertBaseToMeshSpace = false;
		if (AdditiveSpace.GetMixedSpaceFlags() == EMixedSpaceFlags::MeshRotation)
		{
			// If our additive input is in mesh rotation space, we ensure our base is in that space as well
			const FValueSpace BaseSpace = BaseCollection->GetValueSpace();
			bConvertBaseToMeshSpace = BaseSpace.GetMixedSpaceFlags() != EMixedSpaceFlags::MeshRotation;
		}

		// Modify our base input in-place
		if (bConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::LocalToMeshRotation(FPoseValueBundle::From(*BaseCollection), FPoseValueBundle::From(*BaseCollection));
		}

		Transformers::FApplyAdditiveSpace::Apply(VM.GetTransformerMap(), *BaseCollection, *AdditiveCollection, AdditiveWeight, *BaseCollection);

		if (bConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::MeshRotationToLocal(FPoseValueBundle::From(*BaseCollection), FPoseValueBundle::From(*BaseCollection));
		}

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(BaseCollection));
	}
	else
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FKeyframeState> AdditiveKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, AdditiveKeyframe))
		{
			// We have no inputs, nothing to do
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "FAnimNextApplyAdditiveKeyframeTask::Execute: Could not apply additive keyframe - no inputs provided. Pushing a ref pose instead.");
			return;
		}

		TUniquePtr<FKeyframeState> BaseKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, BaseKeyframe))
		{
			// We have a single input, discard it since it must be the additive pose, either way something went wrong
			// Push the reference pose since we'll expect a non-additive pose
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "FAnimNextApplyAdditiveKeyframeTask::Execute: Could not apply additive keyframe - only a single input was provided. Pushing a ref pose instead.");
			return;
		}

		if (!AdditiveKeyframe->Pose.IsAdditive())
		{
			// Additive must be additive type, push reference pose if not the case
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "FAnimNextApplyAdditiveKeyframeTask::Execute: Could not apply additive keyframe - The expected additive pose was not of an additive type. Pushing a ref pose instead.");
			return;
		}

		const float AdditiveWeight = GetInterpolationAlpha(BaseKeyframe.Get(), AdditiveKeyframe.Get());

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			check(BaseKeyframe->Pose.GetNumBones() == AdditiveKeyframe->Pose.GetNumBones());

			const FTransformArrayView BaseTransformsView = BaseKeyframe->Pose.LocalTransforms.GetView();

			if (AdditiveKeyframe->Pose.IsMeshSpaceAdditive())
			{
				BlendWithIdentityAndAccumulateMesh(
					BaseTransformsView, AdditiveKeyframe->Pose.LocalTransforms.GetConstView(), 
					AdditiveKeyframe->Pose.GetLODBoneIndexToParentLODBoneIndexMap(), AdditiveWeight);
			}
			else
			{
				BlendWithIdentityAndAccumulate(BaseTransformsView, AdditiveKeyframe->Pose.LocalTransforms.GetConstView(), AdditiveWeight);
			}

			NormalizeRotations(BaseTransformsView);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			BaseKeyframe->Curves.Accumulate(AdditiveKeyframe->Curves, AdditiveWeight);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			UE::Anim::Attributes::AccumulateAttributes(AdditiveKeyframe->Attributes, BaseKeyframe->Attributes, AdditiveWeight, AAT_None);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(BaseKeyframe));
	}
}

float FAnimNextApplyAdditiveKeyframeTask::GetInterpolationAlpha(const UE::UAF::FKeyframeState* KeyframeA, const UE::UAF::FKeyframeState* KeyframeB) const
{
	float Alpha = BlendWeight;

	if (AlphaSourceCurveName != NAME_None && AlphaCurveInputIndex != INDEX_NONE)
	{
		if (ensure(KeyframeA != nullptr && KeyframeB != nullptr))
		{
			const FBlendedCurve& Curves = AlphaCurveInputIndex == 0 ? KeyframeA->Curves : KeyframeB->Curves;
			Alpha = Curves.Get(AlphaSourceCurveName); // if the curve does not exist, it returns 0.f

			if (InputScaleBiasClampFn.IsSet())
			{
				Alpha = InputScaleBiasClampFn(Alpha);
			}
		}
	}

	return FMath::Clamp(Alpha, 0.0f, 1.0f);
}

float FAnimNextApplyAdditiveKeyframeTask::GetInterpolationAlpha(const UE::UAF::FValueBundle& CollectionA, const UE::UAF::FValueBundle& CollectionB) const
{
	using namespace UE::UAF;

	float Alpha = BlendWeight;

	if (AlphaSourceCurveName != NAME_None && AlphaCurveInputIndex != INDEX_NONE)
	{
		const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>();
		const FValueBundle& AlphaCurveCollection = AlphaCurveInputIndex == 0 ? CollectionA : CollectionB;
		if (const TBoundValueMap<FFloatAnimationAttribute>* FloatMap = AlphaCurveCollection.GetBoundValueMaps().Find<FFloatAnimationAttribute>(MappingKey))
		{
			const FAttributeTypedSetPtr& TypedSet = FloatMap->GetTypedSet();
			if (FAttributeSetIndex CurveIndex = TypedSet->FindIndex(AlphaSourceCurveName))
			{
				Alpha = (*FloatMap)[CurveIndex].Value;
			}
			else
			{
				// Curve name wasn't found
				Alpha = 0.0f;
			}
		}
		else
		{
			// No float curves found
			Alpha = 0.0f;
		}

		if (InputScaleBiasClampFn.IsSet())
		{
			Alpha = InputScaleBiasClampFn(Alpha);
		}
	}

	return FMath::Clamp(Alpha, 0.0f, 1.0f);
}
