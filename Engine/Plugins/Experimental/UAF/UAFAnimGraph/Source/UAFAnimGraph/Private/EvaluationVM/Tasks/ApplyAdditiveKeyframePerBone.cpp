// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/ApplyAdditiveKeyframePerBone.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"
#include "Animation/AnimationAsset.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "Animation/AttributeTypes.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"
#include "UAF/BlendMask/UAFBlendMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ApplyAdditiveKeyframePerBone)

FUAFApplyAdditiveKeyframePerBoneTask FUAFApplyAdditiveKeyframePerBoneTask::Make(const float BlendWeight, const FBlendSampleData& BlendData, bool bPerBoneDataUsesSkeletonIndex)
{
	FUAFApplyAdditiveKeyframePerBoneTask Task;
	Task.BlendData = &BlendData;
	Task.BlendWeight = BlendWeight;
	Task.bPerBoneDataUsesSkeletonIndex = bPerBoneDataUsesSkeletonIndex;
	return Task;
}

FUAFApplyAdditiveKeyframePerBoneTask FUAFApplyAdditiveKeyframePerBoneTask::Make(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn, const FBlendSampleData& BlendData, bool bPerBoneDataUsesSkeletonIndex)
{
	FUAFApplyAdditiveKeyframePerBoneTask Task;
	Task.AlphaSourceCurveName = AlphaSourceCurveName;
	Task.AlphaCurveInputIndex = AlphaCurveInputIndex;
	Task.InputScaleBiasClampFn = MoveTemp(InputScaleBiasClampFn);
	Task.BlendData = &BlendData;
	Task.bPerBoneDataUsesSkeletonIndex = bPerBoneDataUsesSkeletonIndex;
	return Task;
}

void FUAFApplyAdditiveKeyframePerBoneTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (BlendData == nullptr || BlendData->PerBoneBlendData.Num() == 0)
	{
		// If we do not have any or enough blend data revert to a full pose additive application
		Super::Execute(VM);
		return;
	}

	if (VM.GetActiveNamedSet())
	{
		// Pop our top two poses, the top one being the additive pose and the second one the base
		TUniquePtr<FValueBundle> AdditiveInput;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, AdditiveInput))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FValueBundle> BaseInput;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, BaseInput))
		{
			// We have a single input, discard it since it must be the additive pose, either way something went wrong
			// Push the reference pose since we'll expect a non-additive pose
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "FUAFApplyAdditiveKeyframePerBoneTask::Execute: Could not apply additive keyframe - only a single input was provided. Pushing a ref pose instead.");
			return;
		}

		const FValueSpace AdditiveSpace = AdditiveInput->GetValueSpace();
		if (!AdditiveSpace.IsAdditive())
		{
			// Additive input must be additive, push reference pose if not the case
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "FUAFApplyAdditiveKeyframePerBoneTask::Execute: Could not apply additive keyframe - The expected additive pose was not of an additive type. Pushing a ref pose instead.");
			return;
		}

		const float AdditiveWeight = GetInterpolationAlpha(*BaseInput, *AdditiveInput);

		UScriptStruct* FloatAttributeType = FFloatAnimationAttribute::StaticStruct();

		// Build our per-value weights (only for bound values)
		FValueBundleStack PerValueWeights(VM.GetActiveNamedSet());
		FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();

		for (auto It = BaseInput->GetBoundValueMaps().CreateConstIterator(); It; ++It)
		{
			UScriptStruct* AttributeType = It.GetAttributeType();

			// Every attribute type maps to a float, our weight
			FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo(AttributeType, FloatAttributeType);

			TBoundValueMap<FFloatAnimationAttribute>* SetWeights = PerValueWeightMaps.Add<FFloatAnimationAttribute>(MappingKey);

			if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneMap = Cast<FBoneTransformAnimationAttribute>(It.GetMap()))
			{
				const FAttributeTypedSetPtr& BoneSet = BoneMap->GetTypedSet();
				const int32 NumBonesInSet = BoneMap->Num();

				check(BlendData->PerBoneBlendData.Num() >= NumBonesInSet);

				for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
				{
					if (bPerBoneDataUsesSkeletonIndex)
					{
						const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);

						const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;
						(*SetWeights)[SetIndex].Value = TargetSkeletonBoneIndex.IsValid() ? BlendData->PerBoneBlendData[TargetSkeletonBoneIndex.GetInt()] : AdditiveWeight;
					}
					else
					{
						(*SetWeights)[SetIndex].Value = BlendData->PerBoneBlendData[SetIndex.GetInt()];
					}
				}
			}
			else
			{
				// Everything else defaults to full weight
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 1.0f });
			}
		}

		bool bConvertBaseToMeshSpace = false;
		if (AdditiveSpace.GetMixedSpaceFlags() == EMixedSpaceFlags::MeshRotation)
		{
			// If our additive input is in mesh rotation space, we ensure our base is in that space as well
			const FValueSpace BaseSpace = BaseInput->GetValueSpace();
			bConvertBaseToMeshSpace = BaseSpace.GetMixedSpaceFlags() != EMixedSpaceFlags::MeshRotation;
		}

		if (bConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::LocalToMeshRotation(FPoseValueBundle::From(*BaseInput), FPoseValueBundle::From(*BaseInput));
		}

		Transformers::FApplyAdditiveSpace::Apply(VM.GetTransformerMap(), *BaseInput, *AdditiveInput, PerValueWeights, AdditiveWeight, *BaseInput);

		if (bConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::MeshRotationToLocal(FPoseValueBundle::From(*BaseInput), FPoseValueBundle::From(*BaseInput));
		}

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(BaseInput));
	}
	else
	{
		// Pop our top two poses, the top one being the additive pose and the second one the base
		TUniquePtr<FKeyframeState> AdditiveKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, AdditiveKeyframe))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> BaseKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, BaseKeyframe))
		{
			// We have a single input, discard it since it must be the additive pose, either way something went wrong
			// Push the reference pose since we'll expect a non-additive pose
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "FUAFApplyAdditiveKeyframePerBoneTask::Execute: Could not apply additive keyframe - only a single input was provided. Pushing a ref pose instead.");
			return;
		}

		if (!AdditiveKeyframe->Pose.IsAdditive())
		{
			// Additive must be additive type, push reference pose if not the case
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "FUAFApplyAdditiveKeyframePerBoneTask::Execute: Could not apply additive keyframe - The expected additive pose was not of an additive type. Pushing a ref pose instead.");
			return;
		}

		const float AdditiveWeight = GetInterpolationAlpha(BaseKeyframe.Get(), AdditiveKeyframe.Get());

		const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = BaseKeyframe->Pose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const int32 NumLODBones = LODBoneIndexToSkeletonBoneIndexMap.Num();

		TArray<int32> LODBoneIndexToWeightIndexMap;
		LODBoneIndexToWeightIndexMap.AddUninitialized(NumLODBones);

		for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
		{
			if (bPerBoneDataUsesSkeletonIndex)
			{
				const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = TargetSkeletonBoneIndex.GetInt();
			}
			else
			{
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = LODBoneIndex;
			}
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			check(BaseKeyframe->Pose.GetNumBones() == AdditiveKeyframe->Pose.GetNumBones());

			const FTransformArrayView BaseTransformsView = BaseKeyframe->Pose.LocalTransforms.GetView();

			if (AdditiveKeyframe->Pose.IsMeshSpaceAdditive())
			{
				BlendWithIdentityAndAccumulatePerBoneWeightMesh(
					BaseTransformsView, AdditiveKeyframe->Pose.LocalTransforms.GetConstView(),
					AdditiveKeyframe->Pose.GetLODBoneIndexToParentLODBoneIndexMap(), LODBoneIndexToWeightIndexMap, BlendData->PerBoneBlendData, AdditiveWeight);
			}
			else
			{
				BlendWithIdentityAndAccumulatePerBoneWeight(BaseTransformsView, AdditiveKeyframe->Pose.LocalTransforms.GetConstView(), LODBoneIndexToWeightIndexMap, BlendData->PerBoneBlendData, AdditiveWeight);
			}

			NormalizeRotations(BaseTransformsView);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			BaseKeyframe->Curves.Accumulate(AdditiveKeyframe->Curves, AdditiveWeight);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			UE::Anim::Attributes::AccumulateAttributes(AdditiveKeyframe->Attributes, BaseKeyframe->Attributes, AdditiveWeight, AAT_None);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(BaseKeyframe));
	}
}

FUAFApplyAdditiveKeyframeWithBlendMaskTask FUAFApplyAdditiveKeyframeWithBlendMaskTask::Make(const float InBlendWeight, const UUAFBlendMask* InBlendMask)
{
	FUAFApplyAdditiveKeyframeWithBlendMaskTask Task;
	Task.BlendWeight = InBlendWeight;
	Task.BlendMask = InBlendMask;
	return Task;
}

void FUAFApplyAdditiveKeyframeWithBlendMaskTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (!BlendMask)
	{
		// If we do not have a valid blend profile - fall back to a full body additive
		UE_LOGF(LogAnimation, Warning, "UAFApplyAdditiveKeyframeWithBlendMaskTask::Execute: No valid blend profile provided. Falling back to full body additive.");
		Super::Execute(VM);
		return;
	}
	
	const USkeleton* SourceSkeleton = BlendMask->GetSkeleton();
	const UUAFBlendMask::FSkeletonBoneWeightArray& BoneMaskWeights = BlendMask->GetSkeletonBoneWeights(); 
	const UUAFBlendMask::FSkeletonCurveWeightArray& CurveMaskWeights = BlendMask->GetSkeletonCurveWeights(); 
	const UUAFBlendMask::FSkeletonAttributeWeightArray& AttributeMaskWeights = BlendMask->GetSkeletonAttributeWeights();
	
	if (VM.GetActiveNamedSet())
	{
		// Pop our top two poses, the top one being the additive pose and the second one the base
		TUniquePtr<FValueBundle> AdditiveInput;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, AdditiveInput))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FValueBundle> BaseInput;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, BaseInput))
		{
			// We have a single input, discard it since it must be the additive pose, either way something went wrong
			// Push the reference pose since we'll expect a non-additive pose
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "UAFApplyAdditiveKeyframeWithBlendMaskTask::Execute: Could not apply additive keyframe - only a single input was provided. Pushing a ref pose instead.");
			return;
		}

		const FValueSpace AdditiveSpace = AdditiveInput->GetValueSpace();
		if (!AdditiveSpace.IsAdditive())
		{
			// Additive input must be additive, push reference pose if not the case
			FValueBundleStack Collection(VM.GetActiveNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
			UE_LOGF(LogAnimation, Warning, "UAFApplyAdditiveKeyframeWithBlendMaskTask::Execute: Could not apply additive keyframe - The expected additive pose was not of an additive type. Pushing a ref pose instead.");
			return;
		}
		
		const float AdditiveWeight = GetInterpolationAlpha(*BaseInput, *AdditiveInput);
	
		UScriptStruct* FloatAttributeType = FFloatAnimationAttribute::StaticStruct();

		// Build our per-value weights (only for bound values)
		FValueBundleStack PerValueWeights(VM.GetActiveNamedSet());
		FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();

		for (auto It = BaseInput->GetBoundValueMaps().CreateConstIterator(); It; ++It)
		{
			UScriptStruct* AttributeType = It.GetAttributeType();

			// Every attribute type maps to a float, our weight
			FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo(AttributeType, FloatAttributeType);

			TBoundValueMap<FFloatAnimationAttribute>* SetWeights = PerValueWeightMaps.Add<FFloatAnimationAttribute>(MappingKey);
			
			// Bone transforms
			if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneMap = Cast<FBoneTransformAnimationAttribute>(It.GetMap()))
			{
				const FAttributeTypedSetPtr& BoneSet = BoneMap->GetTypedSet();
				const int32 NumBonesInSet = BoneMap->Num();

				check(BlendMask->GetSkeletonBoneWeights().Num() >= NumBonesInSet);
				
				const USkeleton* TargetSkeleton = VM.GetActiveEvaluationContext().GetSkeleton();
				const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
				
				for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
				{
					const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);
					const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;
					int32 WeightIndex = TargetSkeletonBoneIndex.GetInt();
					
					if (SkeletonRemapping.IsValid())
					{
						const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));
						WeightIndex = SourceSkeletonBoneIndex.IsValid() ? SourceSkeletonBoneIndex.GetInt() : INDEX_NONE;
					}
					;
					
					(*SetWeights)[SetIndex].Value = WeightIndex != INDEX_NONE ? BoneMaskWeights[WeightIndex] : 0.0f;
					(*SetWeights)[SetIndex].Value *= AdditiveWeight;
				}
			}
			// Curves and attributes
			else if (const TBoundValueMap<FFloatAnimationAttribute>* FloatMap = Cast<FFloatAnimationAttribute>(It.GetMap()))
			{
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 0.0f });
				
				// Curves
				const FAttributeTypedSetPtr& FloatSet = FloatMap->GetTypedSet();
				CurveMaskWeights.ForEachElement([&FloatSet, SetWeights, AdditiveWeight](const UE::Anim::FCurveElement& Curve)
					{
						const FAttributeSetIndex SetIndex = FloatSet->FindIndex(Curve.Name);
						if (SetIndex.IsValid())
						{
							(*SetWeights)[SetIndex].Value = Curve.Value;
							(*SetWeights)[SetIndex].Value *= AdditiveWeight;
						}
					});
				
				// Attributes
				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : AttributeMaskWeights)
				{
					const FAttributeSetIndex SetIndex = FloatSet->FindIndex(AttributeWeight.Attribute.GetName());
					if (SetIndex.IsValid())
					{
						(*SetWeights)[SetIndex].Value = AttributeWeight.Weight;
						(*SetWeights)[SetIndex].Value *= AdditiveWeight;
					}
				}
			}
			else
			{
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 0.0f });

				const FAttributeTypedSetPtr& TypedSet = It.GetMap()->GetTypedSet();
				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : AttributeMaskWeights)
				{
					const FAttributeSetIndex SetIndex = TypedSet->FindIndex(AttributeWeight.Attribute.GetName());
					if (SetIndex.IsValid())
					{
						(*SetWeights)[SetIndex].Value = AttributeWeight.Weight;
					}
				}
			}
		}

		bool bConvertBaseToMeshSpace = false;
		if (AdditiveSpace.GetMixedSpaceFlags() == EMixedSpaceFlags::MeshRotation)
		{
			// If our additive input is in mesh rotation space, we ensure our base is in that space as well
			const FValueSpace BaseSpace = BaseInput->GetValueSpace();
			bConvertBaseToMeshSpace = BaseSpace.GetMixedSpaceFlags() != EMixedSpaceFlags::MeshRotation;
		}

		if (bConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::LocalToMeshRotation(FPoseValueBundle::From(*BaseInput), FPoseValueBundle::From(*BaseInput));
		}

		Transformers::FApplyAdditiveSpace::Apply(VM.GetTransformerMap(), *BaseInput, *AdditiveInput, PerValueWeights, AdditiveWeight, *BaseInput);

		if (bConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::MeshRotationToLocal(FPoseValueBundle::From(*BaseInput), FPoseValueBundle::From(*BaseInput));
		}

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(BaseInput));
	}
	else
	{
		// Pop our top two poses, the top one being the additive pose and the second one the base
		TUniquePtr<FKeyframeState> AdditiveKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, AdditiveKeyframe))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> BaseKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, BaseKeyframe))
		{
			// We have a single input, discard it since it must be the additive pose, either way something went wrong
			// Push the reference pose since we'll expect a non-additive pose
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "UAFApplyAdditiveKeyframeWithBlendMaskTask::Execute: Could not apply additive keyframe - only a single input was provided. Pushing a ref pose instead.");
			return;
		}

		if (!AdditiveKeyframe->Pose.IsAdditive())
		{
			// Additive must be additive type, push reference pose if not the case
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false)));
			UE_LOGF(LogAnimation, Warning, "UAFApplyAdditiveKeyframeWithBlendMaskTask::Execute: Could not apply additive keyframe - The expected additive pose was not of an additive type. Pushing a ref pose instead.");
			return;
		}

		const float AdditiveWeight = GetInterpolationAlpha(BaseKeyframe.Get(), AdditiveKeyframe.Get());
		
		const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = BaseKeyframe->Pose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const int32 NumLODBones = LODBoneIndexToSkeletonBoneIndexMap.Num();
		
		TArray<int32> LODBoneIndexToWeightIndexMap;
		LODBoneIndexToWeightIndexMap.AddUninitialized(NumLODBones);
		
		const USkeleton* TargetSkeleton = BaseKeyframe->Pose.GetSkeletonAsset();
		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
		if (SkeletonRemapping.IsValid())
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));
				
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SourceSkeletonBoneIndex.IsValid() ? SourceSkeletonBoneIndex.GetInt() : INDEX_NONE;
			}
		}
		else
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SkeletonBoneIndex.IsValid() ? SkeletonBoneIndex.GetInt() : INDEX_NONE;
			}
		}
		
		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			check(BaseKeyframe->Pose.GetNumBones() == AdditiveKeyframe->Pose.GetNumBones());
			
			const FTransformArrayView BaseTransformsView = BaseKeyframe->Pose.LocalTransforms.GetView();

			if (AdditiveKeyframe->Pose.IsMeshSpaceAdditive())
			{
				BlendWithIdentityAndAccumulatePerBoneWeightMesh(
					BaseTransformsView, AdditiveKeyframe->Pose.LocalTransforms.GetConstView(),
					AdditiveKeyframe->Pose.GetLODBoneIndexToParentLODBoneIndexMap(), LODBoneIndexToWeightIndexMap, BoneMaskWeights, AdditiveWeight);
			}
			else
			{
				BlendWithIdentityAndAccumulatePerBoneWeight(BaseTransformsView, AdditiveKeyframe->Pose.LocalTransforms.GetConstView(), LODBoneIndexToWeightIndexMap, BoneMaskWeights, AdditiveWeight);
			}

			NormalizeRotations(BaseTransformsView);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			if (FAnimWeight::IsRelevant(AdditiveWeight))
			{
				FBlendedCurve& BaseCurve = BaseKeyframe->Curves;
				FBlendedCurve& AdditiveCurve = AdditiveKeyframe->Curves;
				
				// Apply the mask weight to matching additive curves
				FBlendedCurve MaskedAdditiveCurve;
				UE::Anim::FNamedValueArrayUtils::Intersection(AdditiveCurve, CurveMaskWeights, [&MaskedAdditiveCurve] (const UE::Anim::FCurveElement& InAdditiveElement, const UE::Anim::FCurveElement& InMaskElement) mutable
				{
					const float MaskedValue = InAdditiveElement.Value * (InMaskElement.Value);
					MaskedAdditiveCurve.Add(InAdditiveElement.Name, MaskedValue);
				});
				
				AdditiveCurve.Combine(MaskedAdditiveCurve);
				
				// Accumulate curves values by the additive weight 
				UE::Anim::FNamedValueArrayUtils::Union(
					BaseCurve, 
					AdditiveCurve, 
					[AdditiveWeight](UE::Anim::FCurveElement& InOutThisElement, const UE::Anim::FCurveElement& InAdditiveCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
				{
					if (InFlags == UE::Anim::ENamedValueUnionFlags::ValidArg0 || InFlags == UE::Anim::ENamedValueUnionFlags::ValidArg1)
					{
						InOutThisElement.Value += InAdditiveCurveElement.Value * AdditiveWeight;
						InOutThisElement.Flags |= InAdditiveCurveElement.Flags;
						
					}
				});
			}
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			// we could accumulate all attributes per additive weight and then just correct the masked ones manually 
			UE::Anim::FStackAttributeContainer& BaseAttributes = BaseKeyframe->Attributes;
			UE::Anim::FStackAttributeContainer& AdditiveAttributes = AdditiveKeyframe->Attributes;
			
			// Build up override weight map for masked attributes 
			TMap<UE::Anim::FAttributeId, float> AttributeMaskTotalWeights;
			for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : AttributeMaskWeights)
			{
				AttributeMaskTotalWeights.Emplace(AttributeWeight.Attribute, AttributeWeight.Weight * AdditiveWeight);
			}
			
			// Accumulates all attributes with additive weight, writes into BaseAttributes
			const EAdditiveAnimationType AdditiveAnimationType = AdditiveKeyframe->Pose.IsMeshSpaceAdditive() ? AAT_RotationOffsetMeshSpace : AAT_LocalSpaceBase;
			UE::Anim::Attributes::AccumulateAttributesWithOverrideWeights(AdditiveAttributes, BaseAttributes, AttributeMaskTotalWeights, AdditiveWeight, AdditiveAnimationType);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(BaseKeyframe));
	}
}

