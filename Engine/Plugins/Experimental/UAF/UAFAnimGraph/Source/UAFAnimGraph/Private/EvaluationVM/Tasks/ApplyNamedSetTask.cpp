// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/ApplyNamedSetTask.h"

#include "EvaluationVM/EvaluationVM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ApplyNamedSetTask)

namespace UE::UAF
{
	FPushNamedSetTask FPushNamedSetTask::Make(FName SetName)
	{
		FPushNamedSetTask Task;
		Task.SetName = SetName;
		return Task;
	}

	void FPushNamedSetTask::Execute(UE::UAF::FEvaluationVM& VM) const
	{
		// If we use the old runtime, do nothing
		if (!VM.GetActiveNamedSet())
		{
			return;
		}

		if (!VM.PushEvaluationContext(SetName))
		{
			UE_LOGF(LogAnimation, Warning, "FPushNamedSetTask::Execute: Could not push new Named Set, set was not found in current binding.");
		}
	}

	FPopNamedSetTask FPopNamedSetTask::Make(FName ExpectedSetName)
	{
		FPopNamedSetTask Task;
		Task.ExpectedSetName = ExpectedSetName;
		return Task;
	}

	void FPopNamedSetTask::Execute(UE::UAF::FEvaluationVM& VM) const
	{
		// If we use the old runtime, do nothing
		if (!VM.GetActiveNamedSet())
		{
			return;
		}

		const FName ActiveSetName = VM.GetActiveEvaluationContext().GetNamedSet()->GetName();
		if (ActiveSetName != ExpectedSetName)
		{
			UE_LOGF(LogAnimation, Warning, "FPopNamedSetTask::Execute: Could not pop active Named Set, the expected set to pop was not active.");
			return;
		}

		VM.PopEvaluationContext();
	}
}
