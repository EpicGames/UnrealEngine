// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/BlendKeyframes.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"
#include "Animation/InputScaleBias.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendKeyframes)

FAnimNextBlendTwoKeyframesTask FAnimNextBlendTwoKeyframesTask::Make(float InterpolationAlpha)
{
	FAnimNextBlendTwoKeyframesTask Task;
	Task.InterpolationAlpha = InterpolationAlpha;

	return Task;
}

FAnimNextBlendTwoKeyframesTask FAnimNextBlendTwoKeyframesTask::Make(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn)
{
	FAnimNextBlendTwoKeyframesTask Task;
	Task.AlphaSourceCurveName = AlphaSourceCurveName;
	Task.AlphaCurveInputIndex = AlphaCurveInputIndex;
	Task.InputScaleBiasClampFn = MoveTemp(InputScaleBiasClampFn);
	return Task;
}

void FAnimNextBlendTwoKeyframesTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FValueBundle> CollectionB;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, CollectionB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FValueBundle> CollectionA;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, CollectionA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(CollectionB));
			return;
		}

		const float WeightOfPoseB = GetInterpolationAlpha(*CollectionA, *CollectionB);

		if (!FAnimWeight::IsRelevant(WeightOfPoseB))
		{
			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(CollectionA));
		}
		else if (FAnimWeight::IsFullWeight(WeightOfPoseB))
		{
			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(CollectionB));
		}
		else
		{
			// Interpolate in place the result within A
			// Named sets must match as it ensures that our inputs have the same sizes/shapes
			check(CollectionA->GetNamedSet() == CollectionB->GetNamedSet());

			Transformers::FInterpolate::Apply(VM.GetTransformerMap(), *CollectionA, *CollectionB, WeightOfPoseB, *CollectionA);
			Transformers::FSanitize::Apply(VM.GetTransformerMap(), *CollectionA);

			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(CollectionA));
		}
	}
	else
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FKeyframeState> KeyframeB;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> KeyframeA;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
			return;
		}

		const float WeightOfPoseB = GetInterpolationAlpha(KeyframeA.Get(), KeyframeB.Get());

		if (!FAnimWeight::IsRelevant(WeightOfPoseB))
		{
			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeA));
		}
		else if (FAnimWeight::IsFullWeight(WeightOfPoseB))
		{
			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
		}
		else
		{
			const float WeightOfPoseA = 1.0f - WeightOfPoseB;

			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
			{
				check(KeyframeA->Pose.GetNumBones() == KeyframeB->Pose.GetNumBones());

				const FTransformArrayView KeyframeBTransformsView = KeyframeB->Pose.LocalTransforms.GetView();

				BlendOverwriteWithScale(KeyframeBTransformsView, KeyframeB->Pose.LocalTransforms.GetConstView(), WeightOfPoseB);
				BlendAddWithScale(KeyframeBTransformsView, KeyframeA->Pose.LocalTransforms.GetConstView(), WeightOfPoseA);

				// Ensure that all of the resulting rotations are normalized
				NormalizeRotations(KeyframeBTransformsView);
			}

			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
			{
				// Curves cannot blend in place
				FBlendedCurve Result;
				Result.Lerp(KeyframeA->Curves, KeyframeB->Curves, WeightOfPoseB);

				KeyframeB->Curves = MoveTemp(Result);
			}

			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
			{
				UE::Anim::FStackAttributeContainer OutputAttributes;
				UE::Anim::Attributes::BlendAttributes({ KeyframeA->Attributes, KeyframeB->Attributes }, { WeightOfPoseA, WeightOfPoseB }, { 0, 1 }, OutputAttributes);
				KeyframeB->Attributes.MoveFrom(OutputAttributes);
			}

			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
		}
	}
}

float FAnimNextBlendTwoKeyframesTask::GetInterpolationAlpha(const UE::UAF::FKeyframeState* KeyframeA, const UE::UAF::FKeyframeState* KeyframeB) const
{
	float Alpha = InterpolationAlpha;

	if (AlphaSourceCurveName != NAME_None && AlphaCurveInputIndex != INDEX_NONE)
	{
		if (ensure(KeyframeA != nullptr && KeyframeB != nullptr))
		{
			const FBlendedCurve& Curves = AlphaCurveInputIndex == 0 ? KeyframeA->Curves : KeyframeB->Curves;
			Alpha = Curves.Get(AlphaSourceCurveName); // if the curve does not exist, it returns 0.f

			if (InputScaleBiasClampFn.IsSet())
			{
				Alpha = InputScaleBiasClampFn(Alpha);
			}
		}
	}

	return FMath::Clamp(Alpha, 0.0f, 1.0f);
}

float FAnimNextBlendTwoKeyframesTask::GetInterpolationAlpha(const UE::UAF::FValueBundle& CollectionA, const UE::UAF::FValueBundle& CollectionB) const
{
	using namespace UE::UAF;

	float Alpha = InterpolationAlpha;

	if (AlphaSourceCurveName != NAME_None && AlphaCurveInputIndex != INDEX_NONE)
	{
		const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>();
		const FValueBundle& AlphaCurveCollection = AlphaCurveInputIndex == 0 ? CollectionA : CollectionB;
		if (const TBoundValueMap<FFloatAnimationAttribute>* FloatMap = AlphaCurveCollection.GetBoundValueMaps().Find<FFloatAnimationAttribute>(MappingKey))
		{
			const FAttributeTypedSetPtr& TypedSet = FloatMap->GetTypedSet();
			if (FAttributeSetIndex CurveIndex = TypedSet->FindIndex(AlphaSourceCurveName))
			{
				Alpha = (*FloatMap)[CurveIndex].Value;
			}
			else
			{
				// Curve name wasn't found
				Alpha = 0.0f;
			}
		}
		else
		{
			// No float curves found
			Alpha = 0.0f;
		}

		if (InputScaleBiasClampFn.IsSet())
		{
			Alpha = InputScaleBiasClampFn(Alpha);
		}
	}

	return FMath::Clamp(Alpha, 0.0f, 1.0f);
}

// --- FAnimNextBlendOverwriteKeyframeWithScaleTask ---

FAnimNextBlendOverwriteKeyframeWithScaleTask FAnimNextBlendOverwriteKeyframeWithScaleTask::Make(float ScaleFactor)
{
	FAnimNextBlendOverwriteKeyframeWithScaleTask Task;
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

void FAnimNextBlendOverwriteKeyframeWithScaleTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		TUniquePtr<FValueBundle> Collection;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, Collection))
		{
			// We have no inputs, nothing to do
			return;
		}

		// Overwrite in place

		Transformers::FOverwrite::Apply(VM.GetTransformerMap(), *Collection, ScaleFactor, *Collection);

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(Collection));
	}
	else
	{
		TUniquePtr<FKeyframeState> Keyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, Keyframe))
		{
			// We have no inputs, nothing to do
			return;
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			BlendOverwriteWithScale(Keyframe->Pose.LocalTransforms.GetView(), Keyframe->Pose.LocalTransforms.GetConstView(), ScaleFactor);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			// Curves cannot override in place
			FBlendedCurve Result;
			Result.Override(Keyframe->Curves, ScaleFactor);

			Keyframe->Curves = MoveTemp(Result);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			UE::Anim::Attributes::OverrideAttributes(Keyframe->Attributes, Keyframe->Attributes, ScaleFactor);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
	}
}

// --- FAnimNextBlendAddKeyframeWithScaleTask  ---

FAnimNextBlendAddKeyframeWithScaleTask FAnimNextBlendAddKeyframeWithScaleTask::Make(float ScaleFactor)
{
	FAnimNextBlendAddKeyframeWithScaleTask Task;
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

void FAnimNextBlendAddKeyframeWithScaleTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FValueBundle> CollectionB;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, CollectionB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FValueBundle> CollectionA;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, CollectionA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(CollectionB));
			return;
		}

		// Accumulate and place the result within B
		// Named sets must match as it ensures that our inputs have the same sizes/shapes
		check(CollectionA->GetNamedSet() == CollectionB->GetNamedSet());

		Transformers::FAccumulate::Apply(VM.GetTransformerMap(), *CollectionB, *CollectionA, ScaleFactor, *CollectionB);

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(CollectionB));
	}
	else
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FKeyframeState> KeyframeB;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> KeyframeA;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
			return;
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			check(KeyframeA->Pose.GetNumBones() == KeyframeB->Pose.GetNumBones());

			BlendAddWithScale(KeyframeB->Pose.LocalTransforms.GetView(), KeyframeA->Pose.LocalTransforms.GetConstView(), ScaleFactor);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			KeyframeB->Curves.Accumulate(KeyframeA->Curves, ScaleFactor);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			UE::Anim::Attributes::AccumulateAttributes(KeyframeA->Attributes, KeyframeB->Attributes, ScaleFactor, AAT_None);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
	}
}
