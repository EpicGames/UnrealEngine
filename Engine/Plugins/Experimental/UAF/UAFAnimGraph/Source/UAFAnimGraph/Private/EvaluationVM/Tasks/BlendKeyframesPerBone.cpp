// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/BlendKeyframesPerBone.h"

#include "AnimationRuntime.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AttributeTypes.h"
#include "Animation/BlendProfile.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/Layer.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendKeyframesPerBone)

namespace UE::UAF::Private
{
	static FValueBundleStack BuildPerValueWeights(
		const FValueBundle& Input,
		UE::UAF::FEvaluationVM& VM,
		const FBlendSampleData* BlendData,
		const USkeleton* Skeleton)
	{
		const float BlendWeight = BlendData->GetClampedWeight();

		UScriptStruct* FloatAttributeType = FFloatAnimationAttribute::StaticStruct();

		// Build our per-value weights (only for bound values)
		FValueBundleStack PerValueWeights(VM.GetActiveNamedSet());
		FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();

		for (auto It = Input.GetBoundValueMaps().CreateConstIterator(); It; ++It)
		{
			UScriptStruct* AttributeType = It.GetAttributeType();

			// Every attribute type maps to a float, our weight
			FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo(AttributeType, FloatAttributeType);

			TBoundValueMap<FFloatAnimationAttribute>* SetWeights = PerValueWeightMaps.Add<FFloatAnimationAttribute>(MappingKey);

			if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneMap = Cast<FBoneTransformAnimationAttribute>(It.GetMap()))
			{
				const FAttributeTypedSetPtr& BoneSet = BoneMap->GetTypedSet();
				const int32 NumBonesInSet = BoneMap->Num();

				if (Skeleton != nullptr)
				{
					const USkeleton* TargetSkeleton = VM.GetActiveEvaluationContext().GetSkeleton();

					const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(Skeleton, TargetSkeleton);

					if (SkeletonRemapping.IsValid())
					{
						for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
						{
							const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);

							const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;
							const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));

							const int32 WeightIndex = SourceSkeletonBoneIndex.GetInt();
							(*SetWeights)[SetIndex].Value = WeightIndex != INDEX_NONE ? BlendData->PerBoneBlendData[WeightIndex] : BlendWeight;
						}
					}
					else
					{
						for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
						{
							const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);

							const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;

							const int32 WeightIndex = TargetSkeletonBoneIndex.GetInt();
							(*SetWeights)[SetIndex].Value = WeightIndex != INDEX_NONE ? BlendData->PerBoneBlendData[WeightIndex] : BlendWeight;
						}
					}
				}
				else
				{
					// If no skeleton is provided we access the per bone blend data with the assumption that it covers a full pose and without performing any remapping
					check(BlendData->PerBoneBlendData.Num() >= NumBonesInSet);

					for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
					{
						const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);

						const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;
						(*SetWeights)[SetIndex].Value = TargetSkeletonBoneIndex.IsValid() ? BlendData->PerBoneBlendData[TargetSkeletonBoneIndex.GetInt()] : BlendWeight;
					}
				}
			}
			else
			{
				// Everything else defaults to full weight
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 1.0f });
			}
		}

		return PerValueWeights;
	}

	static FValueBundleStack BuildPerValueWeights(
		const FValueBundle& Input,
		UE::UAF::FEvaluationVM& VM,
		const USkeleton* SourceSkeleton,
		const UUAFBlendMask::FSkeletonBoneWeightArray* BoneMaskWeights,
		const UUAFBlendMask::FSkeletonCurveWeightArray* CurveMaskWeights,
		const UUAFBlendMask::FSkeletonAttributeWeightArray* AttributeMaskWeights,
		float DefaultWeight)
	{
		UScriptStruct* FloatAttributeType = FFloatAnimationAttribute::StaticStruct();

		// Build our per-value weights (only for bound values)
		// By default, the layered values are ignored unless they have weight
		FValueBundleStack PerValueWeights(VM.GetActiveNamedSet());
		FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();

		for (auto It = Input.GetBoundValueMaps().CreateConstIterator(); It; ++It)
		{
			UScriptStruct* AttributeType = It.GetAttributeType();

			// Every attribute type maps to a float, our weight
			FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo(AttributeType, FloatAttributeType);

			TBoundValueMap<FFloatAnimationAttribute>* SetWeights = PerValueWeightMaps.Add<FFloatAnimationAttribute>(MappingKey);

			if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneMap = Cast<FBoneTransformAnimationAttribute>(It.GetMap()))
			{
				const FAttributeTypedSetPtr& BoneSet = BoneMap->GetTypedSet();
				const int32 NumBonesInSet = BoneMap->Num();

				bool bUsedSkeletonRemapping = false;

				if (SourceSkeleton)
				{
					const USkeleton* TargetSkeleton = VM.GetActiveEvaluationContext().GetSkeleton();
					const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

					if (SkeletonRemapping.IsValid())
					{
						for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
						{
							const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);

							const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;
							const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));

							if (BoneMaskWeights->IsValidIndex(SourceSkeletonBoneIndex.GetInt()))
							{
								(*SetWeights)[SetIndex].Value = SourceSkeletonBoneIndex.IsValid() ? (*BoneMaskWeights)[SourceSkeletonBoneIndex.GetInt()] : 0.0f;
							}
							else
							{
								(*SetWeights)[SetIndex].Value = 0.0f;
							}
							(*SetWeights)[SetIndex].Value *= DefaultWeight;
						}

						bUsedSkeletonRemapping = true;
					}
				}

				if (!bUsedSkeletonRemapping)
				{
					for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
					{
						const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);

						const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;

						if (BoneMaskWeights->IsValidIndex(TargetSkeletonBoneIndex.GetInt()))
						{
							(*SetWeights)[SetIndex].Value = TargetSkeletonBoneIndex.IsValid() ? (*BoneMaskWeights)[TargetSkeletonBoneIndex.GetInt()] : 0.0f;
						}
						else
						{
							(*SetWeights)[SetIndex].Value = 0.0f;
						}
						(*SetWeights)[SetIndex].Value *= DefaultWeight;
					}
				}
			}
			else if (const TBoundValueMap<FFloatAnimationAttribute>* FloatMap = Cast<FFloatAnimationAttribute>(It.GetMap()))
			{
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 0.0f });

				const FAttributeTypedSetPtr& FloatSet = FloatMap->GetTypedSet();
				CurveMaskWeights->ForEachElement([&FloatSet, SetWeights, DefaultWeight](const UE::Anim::FCurveElement& Curve)
					{
						const FAttributeSetIndex SetIndex = FloatSet->FindIndex(Curve.Name);
						if (SetIndex.IsValid())
						{
							(*SetWeights)[SetIndex].Value = Curve.Value;
							(*SetWeights)[SetIndex].Value *= DefaultWeight;
						}
					});

				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : *AttributeMaskWeights)
				{
					const FAttributeSetIndex SetIndex = FloatSet->FindIndex(AttributeWeight.Attribute.GetName());
					if (SetIndex.IsValid())
					{
						(*SetWeights)[SetIndex].Value = AttributeWeight.Weight;
						(*SetWeights)[SetIndex].Value *= DefaultWeight;
					}
				}
			}
			else
			{
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 0.0f });

				const FAttributeTypedSetPtr& TypedSet = It.GetMap()->GetTypedSet();
				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : *AttributeMaskWeights)
				{
					const FAttributeSetIndex SetIndex = TypedSet->FindIndex(AttributeWeight.Attribute.GetName());
					if (SetIndex.IsValid())
					{
						(*SetWeights)[SetIndex].Value = AttributeWeight.Weight;
					}
				}
			}
		}

		return PerValueWeights;
	}
}

FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask::Make(const USkeleton* Skeleton, const FBlendSampleData& BlendData, float ScaleFactor)
{
	FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask Task;
	Task.SourceSkeleton = Skeleton;
	Task.BlendData = &BlendData;
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

void FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (BlendData->PerBoneBlendData.Num() == 0)
	{
		// If we don't have a blend profile and per bone blend data, we blend the whole pose
		Super::Execute(VM);
		return;
	}

	if (VM.GetActiveNamedSet())
	{
		TUniquePtr<FValueBundle> Input;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, Input))
		{
			// We have no inputs, nothing to do
			return;
		}

		const float BlendWeight = BlendData->GetClampedWeight();

		const FValueBundleStack PerValueWeights = Private::BuildPerValueWeights(*Input, VM, BlendData, SourceSkeleton);

		// Modify our input in place
		Transformers::FOverwrite::Apply(VM.GetTransformerMap(), *Input, PerValueWeights, BlendWeight, *Input);

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(Input));
	}
	else
	{
		TUniquePtr<FKeyframeState> Keyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, Keyframe))
		{
			// We have no inputs, nothing to do
			return;
		}

		const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = Keyframe->Pose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const int32 NumLODBones = LODBoneIndexToSkeletonBoneIndexMap.Num();

		TArray<int32> LODBoneIndexToWeightIndexMap;
		LODBoneIndexToWeightIndexMap.AddUninitialized(NumLODBones);

		const USkeleton* TargetSkeleton = Keyframe->Pose.GetSkeletonAsset();

		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

		if (SkeletonRemapping.IsValid())
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SourceSkeletonBoneIndex.GetInt();
			}
		}
		else
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SkeletonBoneIndex.GetInt();
			}
		}

		const float BlendWeight = BlendData->GetClampedWeight();

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			BlendOverwritePerBoneWithScale(
				Keyframe->Pose.LocalTransforms.GetView(), Keyframe->Pose.LocalTransforms.GetConstView(),
				LODBoneIndexToWeightIndexMap, BlendData->PerBoneBlendData, BlendWeight);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			// Curves cannot override in place
			FBlendedCurve Result;
			Result.Override(Keyframe->Curves, BlendWeight);

			Keyframe->Curves = MoveTemp(Result);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			UE::Anim::FStackAttributeContainer OutputAttributes;
			UE::Anim::Attributes::BlendAttributesPerBone({ Keyframe->Attributes }, { LODBoneIndexToWeightIndexMap }, { *BlendData }, { 0 }, { OutputAttributes });
			Keyframe->Attributes.MoveFrom(OutputAttributes);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
	}
}

FAnimNextBlendAddKeyframePerBoneWithScaleTask FAnimNextBlendAddKeyframePerBoneWithScaleTask::Make(const USkeleton* Skeleton, const FBlendSampleData& BlendDataA, const FBlendSampleData& BlendDataB, float ScaleFactor)
{
	FAnimNextBlendAddKeyframePerBoneWithScaleTask Task;
	Task.SourceSkeleton = Skeleton;
	Task.BlendDataA = &BlendDataA;
	Task.BlendDataB = &BlendDataB;
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

void FAnimNextBlendAddKeyframePerBoneWithScaleTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if ((BlendDataA->PerBoneBlendData.Num() == 0 || BlendDataB->PerBoneBlendData.Num() == 0)
		|| (BlendDataA->PerBoneBlendData.Num() != BlendDataB->PerBoneBlendData.Num()))
	{
		// If we don't have a blend profile, we blend the whole pose
		Super::Execute(VM);
		return;
	}

	if (VM.GetActiveNamedSet())
	{
		TUniquePtr<FValueBundle> InputB;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, InputB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FValueBundle> InputA;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, InputA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(InputB));
			return;
		}

		const float BlendWeight = BlendDataA->GetClampedWeight();

		const FValueBundleStack PerValueWeightsA = Private::BuildPerValueWeights(*InputA, VM, BlendDataA, SourceSkeleton);

		// Modify input B in place
		// We flip A and B here because state is pushed in reversed order
		Transformers::FAccumulate::Apply(VM.GetTransformerMap(), *InputB, *InputA, PerValueWeightsA, BlendWeight, *InputB);

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(InputB));
	}
	else
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FKeyframeState> KeyframeB;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> KeyframeA;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
			return;
		}

		const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = KeyframeB->Pose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const int32 NumLODBones = LODBoneIndexToSkeletonBoneIndexMap.Num();

		TArray<int32> LODBoneIndexToWeightIndexMap;
		LODBoneIndexToWeightIndexMap.AddUninitialized(NumLODBones);

		bool bUsedSkeletonRemapping = false;

		if (SourceSkeleton)
		{
			const USkeleton* TargetSkeleton = KeyframeB->Pose.GetSkeletonAsset();
			const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

			if (SkeletonRemapping.IsValid())
			{
				for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
				{
					const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
					const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));
					LODBoneIndexToWeightIndexMap[LODBoneIndex] = SourceSkeletonBoneIndex.GetInt();
				}

				bUsedSkeletonRemapping = true;
			}
		}

		if (!bUsedSkeletonRemapping)
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SkeletonBoneIndex.GetInt();
			}
		}

		const float BlendWeightA = BlendDataA->GetClampedWeight();

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			check(KeyframeA->Pose.GetNumBones() == KeyframeB->Pose.GetNumBones());

			BlendAddPerBoneWithScale(
				KeyframeB->Pose.LocalTransforms.GetView(), KeyframeA->Pose.LocalTransforms.GetConstView(),
				LODBoneIndexToWeightIndexMap, BlendDataA->PerBoneBlendData, BlendWeightA);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			KeyframeB->Curves.Accumulate(KeyframeA->Curves, BlendWeightA);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			// TODO: Might need to be revisited, might not work as intended
			UE::Anim::FStackAttributeContainer OutputAttributes;
			UE::Anim::Attributes::BlendAttributesPerBone(
				{ KeyframeA->Attributes, KeyframeB->Attributes },
				{ LODBoneIndexToWeightIndexMap },
				{ *BlendDataA, *BlendDataB },
				{ 0, 1 },
				{ OutputAttributes });

			KeyframeB->Attributes.MoveFrom(OutputAttributes);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
	}
}

FAnimNextBlendKeyframePerBoneWithScaleTask FAnimNextBlendKeyframePerBoneWithScaleTask::Make(
	const UUAFBlendMask* BlendMask,
	float ScaleFactor)
{
	FAnimNextBlendKeyframePerBoneWithScaleTask Task;
	Task.SourceSkeleton = BlendMask->GetSkeleton();
	Task.BoneMaskWeights = &BlendMask->GetSkeletonBoneWeights();
	Task.CurveMaskWeights = &BlendMask->GetSkeletonCurveWeights();
	Task.AttributeMaskWeights = &BlendMask->GetSkeletonAttributeWeights();
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

FAnimNextBlendKeyframePerBoneWithScaleTask FAnimNextBlendKeyframePerBoneWithScaleTask::Make(
	const USkeleton* Skeleton,
	const UUAFBlendMask::FSkeletonBoneWeightArray* InBoneWeights,
	const UUAFBlendMask::FSkeletonCurveWeightArray* InCurveWeights,
	const UUAFBlendMask::FSkeletonAttributeWeightArray* InAttributeWeights,
	float ScaleFactor)
{
	FAnimNextBlendKeyframePerBoneWithScaleTask Task;
	Task.SourceSkeleton = Skeleton;
	Task.BoneMaskWeights = InBoneWeights;
	Task.CurveMaskWeights = InCurveWeights;
	Task.AttributeMaskWeights = InAttributeWeights;
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

void FAnimNextBlendKeyframePerBoneWithScaleTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (!BoneMaskWeights || BoneMaskWeights->IsEmpty())
	{
		// If we don't have a blend profile, we blend the whole pose
		Super::Execute(VM);
		return;
	}

	if (VM.GetActiveNamedSet())
	{
		TUniquePtr<FValueBundle> Layer;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, Layer))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FValueBundle> Base;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, Base))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(Layer));
			return;
		}

		const FValueBundleStack PerValueWeightsLayer = Private::BuildPerValueWeights(*Base, VM, SourceSkeleton, BoneMaskWeights, CurveMaskWeights, AttributeMaskWeights, ScaleFactor);

		// Apply our layer onto our base and modify it in-place
		Transformers::FLayer::Apply(VM.GetTransformerMap(), *Base, *Layer, PerValueWeightsLayer, ScaleFactor, *Base);
		Transformers::FSanitize::Apply(VM.GetTransformerMap(), *Base);

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(Base));
	}
	else
	{
		// Pop our top two poses, we'll re-use the top keyframe for our result

		TUniquePtr<FKeyframeState> KeyframeB;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeB))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> KeyframeA;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeA))
		{
			// We have a single input, leave it on top of the stack
			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
			return;
		}

		const USkeleton* TargetSkeleton = KeyframeB->Pose.GetSkeletonAsset();

		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);

		const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = KeyframeB->Pose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const int32 NumLODBones = LODBoneIndexToSkeletonBoneIndexMap.Num();

		TArray<int32, TMemStackAllocator<>> LODBoneIndexToWeightIndexMap;
		LODBoneIndexToWeightIndexMap.AddUninitialized(NumLODBones);

		if (SkeletonRemapping.IsValid())
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SourceSkeletonBoneIndex.IsValid() ? SourceSkeletonBoneIndex.GetInt() : INDEX_NONE;
			}
		}
		else
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIndex(LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex]);
				LODBoneIndexToWeightIndexMap[LODBoneIndex] = SkeletonBoneIndex.GetInt();
			}
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			check(KeyframeA->Pose.GetNumBones() == KeyframeB->Pose.GetNumBones());

			BlendOverwritePerBoneWithScale(
				KeyframeA->Pose.LocalTransforms.GetView(), KeyframeA->Pose.LocalTransforms.GetConstView(),
				LODBoneIndexToWeightIndexMap, *BoneMaskWeights, 1.0f, true);

			BlendAddPerBoneWithScale(
				KeyframeA->Pose.LocalTransforms.GetView(), KeyframeB->Pose.LocalTransforms.GetConstView(),
				LODBoneIndexToWeightIndexMap, *BoneMaskWeights, ScaleFactor);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			FBlendedCurve& OutCurve = KeyframeA->Curves;
			FBlendedCurve& TargetCurve = KeyframeB->Curves;

			if (FAnimWeight::IsRelevant(ScaleFactor))
			{
				FBlendedCurve FilteredCurves;

				if (CurveMaskWeights)
				{
					// Multiply per-curve blend weights by matching blend pose curves
					UE::Anim::FNamedValueArrayUtils::Intersection(
						TargetCurve,
						*CurveMaskWeights,
						[this, &FilteredCurves](const UE::Anim::FCurveElement& InBlendElement, const UE::Anim::FCurveElement& InMaskElement) mutable
						{
							FilteredCurves.Add(InBlendElement.Name, InBlendElement.Value * InMaskElement.Value);
						});

					// Override blend curve values with premultipled curves
					TargetCurve.Combine(FilteredCurves);

					// Remove curves that have been filtered by the mask, curves with no mask value defined remain, even with 0.0 value
					UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(
						TargetCurve,
						*CurveMaskWeights,
						[](const UE::Anim::FCurveElement& InBaseElement, const UE::Anim::FCurveElement& InMaskElement)
						{
							const bool bKeep = InMaskElement.Value == 0.0;
							return bKeep;
						}
					);
				}

				// Combine base and filtered pre-multiplied blend curves
				UE::Anim::FNamedValueArrayUtils::Union(
					OutCurve,
					TargetCurve,
					[this](UE::Anim::FCurveElement& InOutBaseElement, const UE::Anim::FCurveElement& InBlendElement, UE::Anim::ENamedValueUnionFlags InFlags)
					{
						if (InFlags == UE::Anim::ENamedValueUnionFlags::BothArgsValid || InFlags == UE::Anim::ENamedValueUnionFlags::ValidArg1)
						{
							InOutBaseElement.Value = FMath::Lerp(InOutBaseElement.Value, InBlendElement.Value, ScaleFactor);
							InOutBaseElement.Flags |= InBlendElement.Flags;
						}
					});
			}
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			using namespace UE::Anim;

			FStackAttributeContainer& BaseAttributes = KeyframeA->Attributes;
			FStackAttributeContainer& BlendAttributes = KeyframeB->Attributes;

			FStackAttributeContainer OutputAttributes;

			// Attributes are to be masked out according to the mask weights in AttributeMaskWeights, if an attribute has no mask weight set then it
			// inherits the weight of whatever bone it is attached to. Below are possible configurations that we need to account for:

			// Root 0.0						Root set to 0.0 therefore a RootMotionDelta attribute will also be masked out without having to set an explicit entry in AttributeMaskWeights

			// Root 0.0						RootMotionDelta is set to 1.0 in AttributeMaskWeights despite the parent bone being masked out.
			//  \ RootMotionDelta 1.0

			// Root 1.0						RootMotionDelta is being masked out in AttributeMaskWeights despite the parent bone being kept.
			//  \ RootMotionDelta 0.0


			// Below is a table of the possible permutations of base/blend attributes being present/absent along with the possible mask values
			// k denotes some value in the range (0, 1) exclusive.
			// - denotes an absent attribute

			// Base | Blend | Weight | Output
			// ------------------------------
			// a    | b     | 1.0    | b
			// a    | b     | k      | lerp(a, b, k)
			// a    | b     | 0.0    | a
			// - - - - - - - - - - - - - - -
			// a    | -     | 1.0    | a
			// a    | -     | k      | a
			// a    | -     | 0.0    | a
			// - - - - - - - - - - - - - - -
			// -    | b     | 1.0    | b
			// -    | b     | k      | lerp(default, b, k)
			// -    | b     | 0.0    | -


			if (AttributeMaskWeights)
			{
				// 1. Blend attributes according to the bone blend weights,
				// i.e. an attribute's weight is determined by the weight of its attached bone.
				UE::Anim::Attributes::BlendAttributesPerBone(BaseAttributes, BlendAttributes, *BoneMaskWeights, OutputAttributes);


				// 2. For each attribute that has a custom weight, i.e. one's that shouldn't be weighted by its attached bone, go and correct the blended value.
				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& MaskedAttribute : *AttributeMaskWeights)
				{
					const TConstArrayView<TWeakObjectPtr<UScriptStruct>> UniqueTypes = OutputAttributes.GetUniqueTypes();

					for (int32 UniqueTypeIndex = 0; UniqueTypeIndex < UniqueTypes.Num(); ++UniqueTypeIndex)
					{
						const TConstArrayView<FAttributeId> AttributeKeys = OutputAttributes.GetKeys(UniqueTypeIndex);
						const TConstArrayView<TWrappedAttribute<FAnimStackAllocator>> AttributeValues = OutputAttributes.GetValues(UniqueTypeIndex);

						const TWeakObjectPtr<UScriptStruct> AttributeType = UniqueTypes[UniqueTypeIndex];
						const IAttributeBlendOperator* Operator = UE::Anim::AttributeTypes::GetTypeOperator(AttributeType);

						TWrappedAttribute<FAnimStackAllocator> DefaultData(AttributeType.Get());
						AttributeType->InitializeStruct(DefaultData.GetPtr<void>());

						uint8* OutputData = OutputAttributes.Find(AttributeType.Get(), MaskedAttribute.Attribute);
						if (OutputData)
						{
							uint8* BaseData = BaseAttributes.Find(AttributeType.Get(), MaskedAttribute.Attribute);
							uint8* BlendData = BlendAttributes.Find(AttributeType.Get(), MaskedAttribute.Attribute);

							if (BaseData && BlendData)
							{
								// Base | Blend | Weight | Output
								// ------------------------------
								// a    | b     | 1.0    | b
								// a    | b     | k      | lerp(a, b, k)
								// a    | b     | 0.0    | a

								Operator->Interpolate(BaseData, BlendData, MaskedAttribute.Weight, OutputData);
							}
							else if (BaseData && !BlendData)
							{
								// Base | Blend | Weight | Output
								// ------------------------------
								// a    | -     | 1.0    | a
								// a    | -     | k      | a
								// a    | -     | 0.0    | a

								*OutputData = *BaseData;
							}
							else if (!BaseData && BlendData)
							{
								// Base | Blend | Weight | Output
								// ------------------------------
								// -    | b     | 1.0    | b
								// -    | b     | k      | lerp(default, b, k)
								// -    | b     | 0.0    | -

								if (MaskedAttribute.Weight != 0.0f)
								{
									Operator->Interpolate(DefaultData.GetPtr<void>(), BlendData, MaskedAttribute.Weight, OutputData);
								}
								else
								{
									OutputAttributes.Remove(AttributeType.Get(), MaskedAttribute.Attribute);
								}
							}
						}
					}
				}
			}

			BaseAttributes.MoveFrom(OutputAttributes);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeA));
	}
}
