// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Quat.isph"

export void ConvertRotationsToMeshSpace(uniform FVector4 LocalPose[], const uniform unsigned int16 ParentBones[], const uniform int NumBones)
{
	for (uniform int BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
	{
		const uniform unsigned int16 ParentIndex = ParentBones[BoneIndex];
		const uniform FVector4 LocalPoseRotation = LocalPose[BoneIndex];
		const uniform FVector4 LocalParentRotation = LocalPose[ParentIndex];

		const uniform FVector4 MeshSpaceRotation = VectorQuaternionMultiply2(LocalParentRotation, LocalPoseRotation);
		LocalPose[BoneIndex] = MeshSpaceRotation;
	}
}

export void ConvertRotationsToLocalSpace(uniform FVector4 MeshPose[], const uniform unsigned int16 ParentBones[], const uniform int NumBones)
{
	for (uniform int BoneIndex = NumBones - 1; BoneIndex > 0; --BoneIndex)
	{
		const uniform unsigned int16 ParentIndex = ParentBones[BoneIndex];
		const uniform FVector4 PoseRotation = MeshPose[BoneIndex];
		const uniform FVector4 ParentRotation = MeshPose[ParentIndex];

		const uniform FVector4 LocalSpaceRotation = VectorQuaternionMultiply2(QuatInverse(ParentRotation), PoseRotation);
		MeshPose[BoneIndex] = LocalSpaceRotation;
	}
}