// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef UE_ANIM_CONVERT_ROTATION_SPACE_TASKS_ISPC
#define UE_ANIM_CONVERT_ROTATION_SPACE_TASKS_ISPC INTEL_ISPC
#endif