// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/ConvertRotationsToMeshSpace.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/Tasks/ConvertRotationSpaceDefs.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"

#if UE_ANIM_CONVERT_ROTATION_SPACE_TASKS_ISPC
#include "ConvertRotationSpace.ispc.generated.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConvertRotationsToMeshSpace)

FAnimNextConvertRotationsToMeshSpaceTask FAnimNextConvertRotationsToMeshSpaceTask::Make(const int32 NumPoses)
{
	FAnimNextConvertRotationsToMeshSpaceTask Task;
	Task.NumPoses = NumPoses;

	return Task;
}

void FAnimNextConvertRotationsToMeshSpaceTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
		{
			TUniquePtr<FValueBundle>* Collection = VM.PeekValueMutable<TUniquePtr<FValueBundle>>(ATTRIBUTE_STACK_NAME, PoseIndex);
			if (Collection == nullptr || !Collection->IsValid())
			{
				// No pose to convert, we are done
				UE_LOGF(LogAnimation, Warning, "FAnimNextConvertRotationsToMeshSpaceTask::Execute: Failed to fully execute - No pose data to convert.");
				break;
			}

			Transformers::FBoneSpace::LocalToMeshRotation(FPoseValueBundle::From(**Collection), FPoseValueBundle::From(**Collection));
		}
	}
	else
	{
		if (!EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			return;
		}

		for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
		{
			TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, PoseIndex);
			if (!Keyframe || !Keyframe->IsValid())
			{
				// No pose to convert
				UE_LOGF(LogAnimation, Warning, "FAnimNextConvertRotationsToMeshSpaceTask::Execute: Failed to fully execute - No pose data to convert.");
				return;
			}

			FLODPoseStack& Pose = (*Keyframe)->Pose;
			TArrayView<FQuat> PoseRotations = Pose.LocalTransforms.Rotations;

#if UE_ANIM_CONVERT_ROTATION_SPACE_TASKS_ISPC
			static_assert(sizeof(FBoneIndexType) == sizeof(uint16)); // ISPC needs updating if assert fails

			const TArrayView<const FBoneIndexType> ParentIndices = Pose.GetRefPose().GetLODBoneIndexToParentLODBoneIndexMap(Pose.LODLevel);

			ispc::ConvertRotationsToMeshSpace(
				reinterpret_cast<ispc::FVector4*>(PoseRotations.GetData()),
				reinterpret_cast<const uint16*>(ParentIndices.GetData()),
				Pose.GetNumBones());
#else
			for (FBoneIndexType Index = 1; Index < PoseRotations.Num(); ++Index)
			{
				const FBoneIndexType ParentIndex = Pose.GetLODBoneParentIndex(Index);
				PoseRotations[Index] = PoseRotations[ParentIndex] * PoseRotations[Index];
			}
#endif // UE_ANIM_CONVERT_ROTATION_SPACE_TASKS_ISPC
		}
	}
}
