// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/CopyBones.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CopyBones)

FAnimNextCopyBonesComponentSpaceTask FAnimNextCopyBonesComponentSpaceTask::Make(const TArray<FAnimNextBoneMapping>& Mapping)
{
	FAnimNextCopyBonesComponentSpaceTask Task;
	Task.Mapping = Mapping;
	return Task;
}

void FAnimNextCopyBonesComponentSpaceTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
		{
			// Get the number of bones to copy

			const int32 MappingNum = Mapping.Num();

			// Loop over each pair of bones to copy

			for (int32 MappingIdx = 0; MappingIdx < MappingNum; MappingIdx++)
			{
				// Find LOD indices for both bones

				const FBoneIndexType SourceBoneIdx = Keyframe->Get()->Pose.GetRefPose().FindLODBoneIndexFromBoneName(Mapping[MappingIdx].SourceBone);
				const FBoneIndexType DestinationBoneIdx = Keyframe->Get()->Pose.GetRefPose().FindLODBoneIndexFromBoneName(Mapping[MappingIdx].TargetBone);

				// If we can't find them or they are not in the current LOD then skip

				if (SourceBoneIdx == INVALID_BONE_INDEX ||
					DestinationBoneIdx == INVALID_BONE_INDEX ||
					SourceBoneIdx >= Keyframe->Get()->Pose.GetNumBones() ||
					DestinationBoneIdx >= Keyframe->Get()->Pose.GetNumBones())
				{
					continue;
				}

				// Compute the Source Component Space Transform

				FTransform SourceComponentTransform = Keyframe->Get()->Pose.LocalTransforms[SourceBoneIdx];
				FBoneIndexType SourceParentIdx = Keyframe->Get()->Pose.GetRefPose().GetLODParentBoneIndex(Keyframe->Get()->Pose.LODLevel, SourceBoneIdx);
				while (SourceParentIdx != INVALID_BONE_INDEX)
				{
					FTransform ParentLocalTransform = Keyframe->Get()->Pose.LocalTransforms[SourceParentIdx];
					SourceComponentTransform = SourceComponentTransform * ParentLocalTransform;
					SourceParentIdx = Keyframe->Get()->Pose.GetRefPose().GetLODParentBoneIndex(Keyframe->Get()->Pose.LODLevel, SourceParentIdx);
				}

				// Compute the Destination Parent Component Space Transform

				FTransform DestinationComponentParentTransform = FTransform::Identity;
				FBoneIndexType DestinationParentIdx = Keyframe->Get()->Pose.GetRefPose().GetLODParentBoneIndex(Keyframe->Get()->Pose.LODLevel, DestinationBoneIdx);
				while (DestinationParentIdx != INVALID_BONE_INDEX)
				{
					FTransform ParentLocalTransform = Keyframe->Get()->Pose.LocalTransforms[DestinationParentIdx];
					DestinationComponentParentTransform = DestinationComponentParentTransform * ParentLocalTransform;
					DestinationParentIdx = Keyframe->Get()->Pose.GetRefPose().GetLODParentBoneIndex(Keyframe->Get()->Pose.LODLevel, DestinationParentIdx);
				}

				// Set the Destination Bone Transform

				Keyframe->Get()->Pose.LocalTransforms[DestinationBoneIdx] = SourceComponentTransform.GetRelativeTransform(DestinationComponentParentTransform);
			}
		}
	}
}
