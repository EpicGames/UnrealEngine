// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Traits/BoneMapping.h"

#include "CopyBones.generated.h"

#define UE_API UAFANIMGRAPH_API

/**
 * FAnimNextCopyBonesComponentSpaceTask
 * 
 * Copies bones in component space
 */
USTRUCT()
struct FAnimNextCopyBonesComponentSpaceTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextCopyBonesComponentSpaceTask)

	// Make a FAnimNextCopyBonesComponentSpaceTask from a source to destination bone mapping
	static UE_API FAnimNextCopyBonesComponentSpaceTask Make(const TArray<FAnimNextBoneMapping>& Mapping);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	TArray<FAnimNextBoneMapping> Mapping;
};

#undef UE_API
