// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/DeadBlending.h"

#include "Animation/AnimTrace.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"

#include "Traits/Inertialization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeadBlending)

#ifndef UE_ANIM_NEXT_DEAD_BLENDING_ISPC
#define UE_ANIM_NEXT_DEAD_BLENDING_ISPC INTEL_ISPC
//#define UE_ANIM_NEXT_DEAD_BLENDING_ISPC 0
#endif

#if UE_ANIM_NEXT_DEAD_BLENDING_ISPC
#include "DeadBlending.ispc.generated.h"
#endif

namespace UE::UAF::DeadBlending::Private
{
#if !UE_ANIM_NEXT_DEAD_BLENDING_ISPC
	static constexpr float Ln2 = 0.69314718056f;

	static inline FVector3f VectorDivMax(const float V, const FVector3f W, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector3f(
			V / FMath::Max(W.X, Epsilon),
			V / FMath::Max(W.Y, Epsilon),
			V / FMath::Max(W.Z, Epsilon));
	}

	static inline FVector3f VectorDivMax(const FVector3f V, const FVector3f W, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector3f(
			V.X / FMath::Max(W.X, Epsilon),
			V.Y / FMath::Max(W.Y, Epsilon),
			V.Z / FMath::Max(W.Z, Epsilon));
	}

	static inline FVector VectorDivMax(const FVector V, const FVector W, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector(
			V.X / FMath::Max(W.X, Epsilon),
			V.Y / FMath::Max(W.Y, Epsilon),
			V.Z / FMath::Max(W.Z, Epsilon));
	}

	static inline FVector3f VectorInvExpApprox(const FVector3f V)
	{
		return FVector3f(
			FMath::InvExpApprox(V.X),
			FMath::InvExpApprox(V.Y),
			FMath::InvExpApprox(V.Z));
	}

	static inline FVector VectorEerp(const FVector V, const FVector W, const float Alpha, const float Epsilon = UE_SMALL_NUMBER)
	{
		if (FVector::DistSquared(V, W) < Epsilon)
		{
			return FVector(
				FMath::Lerp(FMath::Max(V.X, Epsilon), FMath::Max(W.X, Epsilon), Alpha),
				FMath::Lerp(FMath::Max(V.Y, Epsilon), FMath::Max(W.Y, Epsilon), Alpha),
				FMath::Lerp(FMath::Max(V.Z, Epsilon), FMath::Max(W.Z, Epsilon), Alpha));
		}
		else
		{
			return FVector(
				FMath::Pow(FMath::Max(V.X, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.X, Epsilon), Alpha),
				FMath::Pow(FMath::Max(V.Y, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.Y, Epsilon), Alpha),
				FMath::Pow(FMath::Max(V.Z, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.Z, Epsilon), Alpha));
		}
	}

	static inline FVector VectorExp(const FVector V)
	{
		return FVector(
			FMath::Exp(V.X),
			FMath::Exp(V.Y),
			FMath::Exp(V.Z));
	}

	static inline FVector VectorLogMax(const FVector V, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector(
			FMath::Loge(FMath::Max(V.X, Epsilon)),
			FMath::Loge(FMath::Max(V.Y, Epsilon)),
			FMath::Loge(FMath::Max(V.Z, Epsilon)));
	}

	static inline FVector ExtrapolateTranslation(
		const FVector Translation,
		const FVector3f Velocity,
		const FVector3f DecayHalflife,
		const float Time,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(Ln2, DecayHalflife, Epsilon);
			return Translation + (FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)));
		}
		else
		{
			return Translation;
		}
	}

	static inline FQuat ExtrapolateRotation(
		const FQuat Rotation,
		const FVector3f Velocity,
		const FVector3f DecayHalflife,
		const float Time,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(Ln2, DecayHalflife, Epsilon);
			return FQuat::MakeFromRotationVector((FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)))) * Rotation;
		}
		else
		{
			return Rotation;
		}
	}

	static inline FVector ExtrapolateScale(
		const FVector Scale,
		const FVector3f Velocity,
		const FVector3f DecayHalflife,
		const float Time,
		const float Epsilon = UE_SMALL_NUMBER)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(Ln2, DecayHalflife, Epsilon);
			return VectorExp((FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)))) * Scale;
		}
		else
		{
			return Scale;
		}
	}

	static inline float ClipMagnitudeToGreaterThanEpsilon(const float X, const float Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		return
			X >= 0.0f && X < Epsilon ? Epsilon :
			X <  0.0f && X > -Epsilon ? -Epsilon : X;
	}

	static inline float ComputeDecayHalfLifeFromDiffAndVelocity(
		const float SrcDstDiff,
		const float SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		// Essentially what this function does is compute a half-life based on the ratio between the velocity vector and
		// the vector from the source to the destination. This is then clamped to some min and max. If the signs are
		// different (i.e. the velocity and the vector from source to destination are in opposite directions) this will
		// produce a negative number that will get clamped to HalfLifeMin. If the signs match, this will produce a large
		// number when the velocity is small and the vector from source to destination is large, and a small number when
		// the velocity is large and the vector from source to destination is small. This will be clamped either way to 
		// be in the range given by HalfLifeMin and HalfLifeMax. Finally, since the velocity can be close to zero we 
		// have to clamp it to always be greater than some given magnitude (preserving the sign).

		return FMath::Clamp(HalfLife * (SrcDstDiff / ClipMagnitudeToGreaterThanEpsilon(SrcVelocity, Epsilon)), HalfLifeMin, HalfLifeMax);
	}

	static inline FVector3f ComputeDecayHalfLifeFromDiffAndVelocity(
		const FVector SrcDstDiff,
		const FVector3f SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		return FVector3f(
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.X, SrcVelocity.X, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.Y, SrcVelocity.Y, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.Z, SrcVelocity.Z, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon));
	}
#endif

	void Transition(
		const TArrayView<FQuat4f>& BoneRotationDirections,
		const FTransformArraySoAView& Source,
		const TArrayView<FVector3f>& SourceBoneTranslationVelocities,
		const TArrayView<FVector3f>& SourceBoneRotationVelocities,
		const TArrayView<FVector3f>& SourceBoneScaleVelocities,
		const TArrayView<FVector3f>& SourceBoneTranslationDecayHalfLives,
		const TArrayView<FVector3f>& SourceBoneRotationDecayHalfLives,
		const TArrayView<FVector3f>& SourceBoneScaleDecayHalfLives,
		const FTransformArraySoAConstView& Dest,
		const FTransformArraySoAConstView& Curr,
		const FTransformArraySoAConstView& Prev,
		const float DeltaTime,
		const UE::UAF::FDeadBlendTransitionTaskParameters& Parameters)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAF::DeadBlending::Transition);

		const int32 LODBoneNum = Source.Num();
		check(LODBoneNum <= Dest.Num());
		check(LODBoneNum <= Curr.Num());
		check(LODBoneNum <= Prev.Num());

#if UE_ANIM_NEXT_DEAD_BLENDING_ISPC
		ispc::AnimNextDeadBlendingTransition(
			(float*)BoneRotationDirections.GetData(),
			(double*)Source.Translations.GetData(),
			(double*)Source.Rotations.GetData(),
			(double*)Source.Scales3D.GetData(),
			(float*)SourceBoneTranslationVelocities.GetData(),
			(float*)SourceBoneRotationVelocities.GetData(),
			(float*)SourceBoneScaleVelocities.GetData(),
			(float*)SourceBoneTranslationDecayHalfLives.GetData(),
			(float*)SourceBoneRotationDecayHalfLives.GetData(),
			(float*)SourceBoneScaleDecayHalfLives.GetData(),
			(double*)Dest.Translations.GetData(),
			(double*)Dest.Rotations.GetData(),
			(double*)Dest.Scales3D.GetData(),
			(double*)Curr.Translations.GetData(),
			(double*)Curr.Rotations.GetData(),
			(double*)Curr.Scales3D.GetData(),
			(double*)Prev.Translations.GetData(),
			(double*)Prev.Rotations.GetData(),
			(double*)Prev.Scales3D.GetData(),
			LODBoneNum,
			DeltaTime,
			Parameters.ExtrapolationHalfLife,
			Parameters.ExtrapolationHalfLifeMin,
			Parameters.ExtrapolationHalfLifeMax,
			Parameters.MaximumTranslationVelocity,
			Parameters.MaximumRotationVelocity,
			Parameters.MaximumScaleVelocity);
#else
		for (FBoneIndexType LODBoneIdx = 0; LODBoneIdx < LODBoneNum; LODBoneIdx++)
		{
			BoneRotationDirections[LODBoneIdx] = FQuat4f::Identity;

			// Get Source Animation Transform

			const FVector SrcTranslationCurr = Curr.Translations[LODBoneIdx];
			const FQuat SrcRotationCurr = Curr.Rotations[LODBoneIdx];
			const FVector SrcScaleCurr = Curr.Scales3D[LODBoneIdx];

			Source.Translations[LODBoneIdx] = SrcTranslationCurr;
			Source.Rotations[LODBoneIdx] = SrcRotationCurr;
			Source.Scales3D[LODBoneIdx] = SrcScaleCurr;

			// Get Source Animation Velocity

			const FVector SrcTranslationPrev = Prev.Translations[LODBoneIdx];
			const FQuat SrcRotationPrev = Prev.Rotations[LODBoneIdx];
			const FVector SrcScalePrev = Prev.Scales3D[LODBoneIdx];

			const FVector TranslationDiff = SrcTranslationCurr - SrcTranslationPrev;

			const FQuat RotationDiff = (SrcRotationCurr * SrcRotationPrev.Inverse()).GetShortestArcWith(FQuat::Identity);
			const FVector ScaleDiff = Private::VectorDivMax(SrcScaleCurr, SrcScalePrev);

			SourceBoneTranslationVelocities[LODBoneIdx] = (FVector3f)(TranslationDiff / DeltaTime).GetClampedToMaxSize(Parameters.MaximumTranslationVelocity);
			SourceBoneRotationVelocities[LODBoneIdx] = (FVector3f)(RotationDiff.ToRotationVector() / DeltaTime).GetClampedToMaxSize(Parameters.MaximumRotationVelocity);
			SourceBoneScaleVelocities[LODBoneIdx] = (FVector3f)(Private::VectorLogMax(ScaleDiff) / DeltaTime).GetClampedToMaxSize(Parameters.MaximumScaleVelocity);

			// Compute Decay HalfLives

			const FVector DstTranslation = Dest.Translations[LODBoneIdx];
			const FQuat DstRotation = Dest.Rotations[LODBoneIdx];
			const FVector DstScale = Dest.Scales3D[LODBoneIdx];

			const FVector TranslationSrcDstDiff = DstTranslation - SrcTranslationCurr;

			const FQuat RotationSrcDstDiff = DstRotation * SrcRotationCurr.Inverse().GetShortestArcWith(FQuat::Identity);
			const FVector ScaleSrcDstDiff = Private::VectorDivMax(DstScale, SrcScaleCurr);

			SourceBoneTranslationDecayHalfLives[LODBoneIdx] = Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				TranslationSrcDstDiff,
				SourceBoneTranslationVelocities[LODBoneIdx],
				Parameters.ExtrapolationHalfLife,
				Parameters.ExtrapolationHalfLifeMin,
				Parameters.ExtrapolationHalfLifeMax);

			SourceBoneRotationDecayHalfLives[LODBoneIdx] = Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				RotationSrcDstDiff.ToRotationVector(),
				SourceBoneRotationVelocities[LODBoneIdx],
				Parameters.ExtrapolationHalfLife,
				Parameters.ExtrapolationHalfLifeMin,
				Parameters.ExtrapolationHalfLifeMax);

			SourceBoneScaleDecayHalfLives[LODBoneIdx] = Private::ComputeDecayHalfLifeFromDiffAndVelocity(
				ScaleSrcDstDiff,
				SourceBoneScaleVelocities[LODBoneIdx],
				Parameters.ExtrapolationHalfLife,
				Parameters.ExtrapolationHalfLifeMin,
				Parameters.ExtrapolationHalfLifeMax);
		}
#endif
	}


	void Transition(
		const TArrayView<FQuat4f>& BoneRotationDirections,
		const FTransformArraySoAView& Source,
		const TArrayView<FVector3f>& SourceBoneTranslationVelocities,
		const TArrayView<FVector3f>& SourceBoneRotationVelocities,
		const TArrayView<FVector3f>& SourceBoneScaleVelocities,
		const TArrayView<FVector3f>& SourceBoneTranslationDecayHalfLives,
		const TArrayView<FVector3f>& SourceBoneRotationDecayHalfLives,
		const TArrayView<FVector3f>& SourceBoneScaleDecayHalfLives,
		const FTransformArraySoAConstView& Curr,
		const UE::UAF::FDeadBlendTransitionTaskParameters& Parameters)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAF::DeadBlending::Transition);

		const int32 LODBoneNum = Source.Num();
		check(LODBoneNum <= Curr.Num());

#if UE_ANIM_NEXT_DEAD_BLENDING_ISPC
		ispc::AnimNextDeadBlendingTransitionStatic(
			(float*)BoneRotationDirections.GetData(),
			(double*)Source.Translations.GetData(),
			(double*)Source.Rotations.GetData(),
			(double*)Source.Scales3D.GetData(),
			(float*)SourceBoneTranslationVelocities.GetData(),
			(float*)SourceBoneRotationVelocities.GetData(),
			(float*)SourceBoneScaleVelocities.GetData(),
			(float*)SourceBoneTranslationDecayHalfLives.GetData(),
			(float*)SourceBoneRotationDecayHalfLives.GetData(),
			(float*)SourceBoneScaleDecayHalfLives.GetData(),
			(double*)Curr.Translations.GetData(),
			(double*)Curr.Rotations.GetData(),
			(double*)Curr.Scales3D.GetData(),
			LODBoneNum,
			Parameters.ExtrapolationHalfLifeMin);
#else
		for (FBoneIndexType LODBoneIdx = 0; LODBoneIdx < LODBoneNum; LODBoneIdx++)
		{
			BoneRotationDirections[LODBoneIdx] = FQuat4f::Identity;

			Source.Translations[LODBoneIdx] = Curr.Translations[LODBoneIdx];
			Source.Rotations[LODBoneIdx] = Curr.Rotations[LODBoneIdx];
			Source.Scales3D[LODBoneIdx] = Curr.Scales3D[LODBoneIdx];

			SourceBoneTranslationVelocities[LODBoneIdx] = FVector3f::ZeroVector;
			SourceBoneRotationVelocities[LODBoneIdx] = FVector3f::ZeroVector;
			SourceBoneScaleVelocities[LODBoneIdx] = FVector3f::ZeroVector;

			SourceBoneTranslationDecayHalfLives[LODBoneIdx] = Parameters.ExtrapolationHalfLifeMin * FVector3f::OneVector;
			SourceBoneRotationDecayHalfLives[LODBoneIdx] = Parameters.ExtrapolationHalfLifeMin * FVector3f::OneVector;
			SourceBoneScaleDecayHalfLives[LODBoneIdx] = Parameters.ExtrapolationHalfLifeMin * FVector3f::OneVector;
		}
#endif
	}

	void Apply(
		const FTransformArraySoAView& Dest,
		const TArrayView<FQuat4f>& BoneRotationDirections,
		const FTransformArraySoAConstView& Source,
		const TArrayView<const FVector3f>& SourceBoneTranslationVelocities,
		const TArrayView<const FVector3f>& SourceBoneRotationVelocities,
		const TArrayView<const FVector3f>& SourceBoneScaleVelocities,
		const TArrayView<const FVector3f>& SourceBoneTranslationDecayHalfLives,
		const TArrayView<const FVector3f>& SourceBoneRotationDecayHalfLives,
		const TArrayView<const FVector3f>& SourceBoneScaleDecayHalfLives,
		const float BlendDuration,
		const float TimeSinceTransition,
		const EAlphaBlendOption BlendMode,
		UCurveFloat* CustomBlendCurve)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAF::DeadBlending::Apply)

		const float Alpha = 1.0f - FAlphaBlend::AlphaToBlendOption(
			FMath::Clamp(TimeSinceTransition / FMath::Max(BlendDuration, UE_SMALL_NUMBER), 0.0f, 1.0f),
			BlendMode, CustomBlendCurve);

		if (Alpha == 0.0f)
		{
			return;
		}

		const int32 LODBoneNum = FMath::Min(Dest.Num(), Source.Num());

#if UE_ANIM_NEXT_DEAD_BLENDING_ISPC
		ispc::AnimNextDeadBlendingApply(
			(double*)Dest.Translations.GetData(),
			(double*)Dest.Rotations.GetData(),
			(double*)Dest.Scales3D.GetData(),
			(float*)BoneRotationDirections.GetData(),
			(double*)Source.Translations.GetData(),
			(double*)Source.Rotations.GetData(),
			(double*)Source.Scales3D.GetData(),
			(float*)SourceBoneTranslationVelocities.GetData(),
			(float*)SourceBoneRotationVelocities.GetData(),
			(float*)SourceBoneScaleVelocities.GetData(),
			(float*)SourceBoneTranslationDecayHalfLives.GetData(),
			(float*)SourceBoneRotationDecayHalfLives.GetData(),
			(float*)SourceBoneScaleDecayHalfLives.GetData(),
			LODBoneNum,
			Alpha,
			TimeSinceTransition);
#else
		for (FBoneIndexType LODBoneIdx = 0; LODBoneIdx < LODBoneNum; LODBoneIdx++)
		{
			// Extrapolate and Blend Translation

			const FVector ExtrapolatedTranslation = DeadBlending::Private::ExtrapolateTranslation(
				Source.Translations[LODBoneIdx],
				SourceBoneTranslationVelocities[LODBoneIdx],
				SourceBoneTranslationDecayHalfLives[LODBoneIdx],
				TimeSinceTransition);

			Dest.Translations[LODBoneIdx] = FMath::Lerp(Dest.Translations[LODBoneIdx], ExtrapolatedTranslation, Alpha);

			// Extrapolate and Blend Rotation

			const FQuat ExtrapolatedRotation = DeadBlending::Private::ExtrapolateRotation(
				Source.Rotations[LODBoneIdx],
				SourceBoneRotationVelocities[LODBoneIdx],
				SourceBoneRotationDecayHalfLives[LODBoneIdx],
				TimeSinceTransition);

			// We need to enforce that the blend of the rotation doesn't suddenly "switch sides"
			// given that the extrapolated rotation can become quite far from the destination
			// animation. To do this we keep track of the blend "direction" and ensure that the
			// delta we are applying to the destination animation always remains on the same
			// side of this rotation.

			FQuat RotationDiff = ExtrapolatedRotation * Dest.Rotations[LODBoneIdx].Inverse();
			RotationDiff.EnforceShortestArcWith((FQuat)BoneRotationDirections[LODBoneIdx]);

			// Update BoneRotationDirections to match our current path
			BoneRotationDirections[LODBoneIdx] = (FQuat4f)RotationDiff;

			Dest.Rotations[LODBoneIdx] = FQuat::MakeFromRotationVector(RotationDiff.ToRotationVector() * Alpha) * Dest.Rotations[LODBoneIdx];

			// Extrapolate and Blend Scale

			const FVector ExtrapolatedScale = Private::ExtrapolateScale(
				Source.Scales3D[LODBoneIdx],
				SourceBoneScaleVelocities[LODBoneIdx],
				SourceBoneScaleDecayHalfLives[LODBoneIdx],
				TimeSinceTransition);

			Dest.Scales3D[LODBoneIdx] = Private::VectorEerp(Dest.Scales3D[LODBoneIdx], ExtrapolatedScale, Alpha);
		}
#endif
	}
}

FAnimNextDeadBlendingTransitionTask FAnimNextDeadBlendingTransitionTask::Make(
	UE::UAF::FDeadBlendingState* State,
	const UE::UAF::FTransformArraySoAHeap* CurrPose,
	const UE::UAF::FTransformArraySoAHeap* PrevPose,
	const float DeltaTime,
	const UE::UAF::FDeadBlendTransitionTaskParameters& Parameters)
{
	FAnimNextDeadBlendingTransitionTask Task;
	Task.State = State;
	Task.CurrPose = CurrPose;
	Task.PrevPose = PrevPose;
	Task.DeltaTime = DeltaTime;
	Task.Parameters = Parameters;
	return Task;
}

FAnimNextDeadBlendingTransitionTask FAnimNextDeadBlendingTransitionTask::Make(
	UE::UAF::FDeadBlendingState* State,
	const UE::UAF::FTransformArraySoAHeap* CurrPose,
	const UE::UAF::FDeadBlendTransitionTaskParameters& Parameters)
{
	FAnimNextDeadBlendingTransitionTask Task;
	Task.State = State;
	Task.CurrPose = CurrPose;
	Task.Parameters = Parameters;
	return Task;
}

void FAnimNextDeadBlendingTransitionTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
		{
			if (PrevPose)
			{
				// Transition with two previous poses

				const int32 SourceLODBoneNum = FMath::Min((*Keyframe)->Pose.LocalTransformsView.Num(), FMath::Min(CurrPose->Num(), PrevPose->Num()));
				State->SetNumUninitialized(SourceLODBoneNum);

				DeadBlending::Private::Transition(
					State->BoneRotationDirections,
					State->SourcePose.GetView(),
					State->SourceBoneTranslationVelocities,
					State->SourceBoneRotationVelocities,
					State->SourceBoneScaleVelocities,
					State->SourceBoneTranslationDecayHalfLives,
					State->SourceBoneRotationDecayHalfLives,
					State->SourceBoneScaleDecayHalfLives,
					(*Keyframe)->Pose.LocalTransformsView,
					CurrPose->GetConstView(),
					PrevPose->GetConstView(),
					DeltaTime,
					Parameters);
			}
			else
			{
				// Transition with a single previous pose, assuming zero velocity

				const int32 SourceLODBoneNum = CurrPose->Num();
				State->SetNumUninitialized(SourceLODBoneNum);

				DeadBlending::Private::Transition(
					State->BoneRotationDirections,
					State->SourcePose.GetView(),
					State->SourceBoneTranslationVelocities,
					State->SourceBoneRotationVelocities,
					State->SourceBoneScaleVelocities,
					State->SourceBoneTranslationDecayHalfLives,
					State->SourceBoneRotationDecayHalfLives,
					State->SourceBoneScaleDecayHalfLives,
					CurrPose->GetConstView(),
					Parameters);
			}
		}
	}
}

FAnimNextDeadBlendingApplyTask FAnimNextDeadBlendingApplyTask::Make(
	UE::UAF::FDeadBlendingState* State,
	const float BlendDuration,
	const float TimeSinceTransition,
	const EAlphaBlendOption& BlendMode,
	const TWeakObjectPtr<UCurveFloat> CustomBlendCurve)
{
	FAnimNextDeadBlendingApplyTask Task;
	Task.State = State;
	Task.BlendDuration = BlendDuration;
	Task.TimeSinceTransition = TimeSinceTransition;
	Task.BlendMode = BlendMode;
	Task.CustomBlendCurve = CustomBlendCurve;
	return Task;
}

void FAnimNextDeadBlendingApplyTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
		{
			DeadBlending::Private::Apply(
				(*Keyframe)->Pose.LocalTransforms.GetView(),
				State->BoneRotationDirections,
				State->SourcePose.GetConstView(),
				State->SourceBoneTranslationVelocities,
				State->SourceBoneRotationVelocities,
				State->SourceBoneScaleVelocities,
				State->SourceBoneTranslationDecayHalfLives,
				State->SourceBoneRotationDecayHalfLives,
				State->SourceBoneScaleDecayHalfLives,
				BlendDuration,
				TimeSinceTransition,
				BlendMode,
				CustomBlendCurve.Get());
		}
	}
}

#undef UE_ANIM_NEXT_DEAD_BLENDING_ISPC
