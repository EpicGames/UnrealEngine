// Copyright Epic Games, Inc. All Rights Reserved.

#define UE_LN2 (0.69314718056f)
#define UE_SMALL_NUMBER (1.e-8f)
#define UE_KINDA_SMALL_NUMBER (1.e-4f)

typedef float<3> Vector3f;
typedef float<4> Quat4f;
typedef double<3> Vector3d;
typedef double<4> Quat4d;

inline Vector3f Vector3fLoad(uniform const float Data[], const int Index)
{
	const Vector3f X = {
#pragma ignore warning(perf)
		Data[Index * 3 + 0],
#pragma ignore warning(perf)
		Data[Index * 3 + 1],
#pragma ignore warning(perf)
		Data[Index * 3 + 2],
	};

	return X;
}

inline Quat4f Quat4fLoad(uniform const float Data[], const int Index)
{
	const Quat4f X = {
#pragma ignore warning(perf)
		Data[Index * 4 + 0],
#pragma ignore warning(perf)
		Data[Index * 4 + 1],
#pragma ignore warning(perf)
		Data[Index * 4 + 2],
#pragma ignore warning(perf)
		Data[Index * 4 + 3],
	};

	return X;
}

inline Vector3d Vector3dLoad(uniform const double Data[], const int Index)
{
	const Vector3d X = {
#pragma ignore warning(perf)
		Data[Index * 3 + 0],
#pragma ignore warning(perf)
		Data[Index * 3 + 1],
#pragma ignore warning(perf)
		Data[Index * 3 + 2],
	};

	return X;
}

inline Quat4d Quat4dLoad(uniform const double Data[], const int Index)
{
	const Quat4d X = {
#pragma ignore warning(perf)
		Data[Index * 4 + 0],
#pragma ignore warning(perf)
		Data[Index * 4 + 1],
#pragma ignore warning(perf)
		Data[Index * 4 + 2],
#pragma ignore warning(perf)
		Data[Index * 4 + 3],
	};

	return X;
}

inline void Vector3fSave(uniform float Data[], const int Index, const Vector3f X)
{
#pragma ignore warning(perf)
	Data[Index * 3 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 3 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 3 + 2] = X[2];
}

inline void Quat4fSave(uniform float Data[], const int Index, const Quat4f X)
{
#pragma ignore warning(perf)
	Data[Index * 4 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 4 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 4 + 2] = X[2];
#pragma ignore warning(perf)
	Data[Index * 4 + 3] = X[3];
}

inline void Vector3dSave(uniform double Data[], const int Index, const Vector3d X)
{
#pragma ignore warning(perf)
	Data[Index * 3 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 3 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 3 + 2] = X[2];
}

inline void Quat4dSave(uniform double Data[], const int Index, const Quat4d X)
{
#pragma ignore warning(perf)
	Data[Index * 4 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 4 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 4 + 2] = X[2];
#pragma ignore warning(perf)
	Data[Index * 4 + 3] = X[3];
}

inline float Vector3fDot(const Vector3f V, const Vector3f W)
{
	return V.x * W.x + V.y * W.y + V.z * W.z;
}

inline float Vector3fLengthSquared(const Vector3f V)
{
	return Vector3fDot(V, V);
}

inline float Vector3fLength(const Vector3f V)
{
	return sqrt(Vector3fLengthSquared(V));
}

inline Vector3f Vector3fInvExpApprox(const Vector3f V)
{
	const uniform float A = 1.00746054f;
	const uniform float B = 0.45053901f;
	const uniform float C = 0.25724632f;
	return 1.0f / (1.0f + A * V + B * V * V + C * V * V * V);
}

inline Vector3f Vector3fDivMax(const Vector3f V, const Vector3f W, const uniform float Epsilon = UE_SMALL_NUMBER)
{
	const Vector3f O = {
		V.x / max(W.x, Epsilon),
		V.y / max(W.y, Epsilon),
		V.z / max(W.z, Epsilon),
	};

	return O;
}

inline Vector3f Vector3fExp(const Vector3f V)
{
	const Vector3f O = {
		exp(V.x),
		exp(V.y),
		exp(V.z),
	};

	return O;
}

inline double Vector3dDot(const Vector3d V, const Vector3d W)
{
	return V.x * W.x + V.y * W.y + V.z * W.z;
}

inline double Vector3dLengthSquared(const Vector3d V)
{
	return Vector3dDot(V, V);
}

inline double Vector3dLength(const Vector3d V)
{
	return sqrt(Vector3dLengthSquared(V));
}

inline Vector3d Vector3dLerp(const Vector3d V, const Vector3d W, const uniform double Alpha)
{
	return V * (1.0 - Alpha) + W * Alpha;
}

inline Vector3d Vector3dEerp(const Vector3d V, const Vector3d W, const uniform double Alpha, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	const Vector3d Out = {
		pow(max(V.x, Epsilon), (1.0 - Alpha)) * pow(max(W.x, Epsilon), Alpha),
		pow(max(V.y, Epsilon), (1.0 - Alpha)) * pow(max(W.y, Epsilon), Alpha),
		pow(max(V.z, Epsilon), (1.0 - Alpha)) * pow(max(W.z, Epsilon), Alpha),
	};

	return Out;
}

inline Vector3d Vector3dDivMax(const Vector3d V, const Vector3d W, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	const Vector3d O = {
		V.x / max(W.x, Epsilon),
		V.y / max(W.y, Epsilon),
		V.z / max(W.z, Epsilon),
	};

	return O;
}

inline Vector3d Vector3dLogMax(const Vector3d V, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	const Vector3d O = {
		log(max(V.x, Epsilon)),
		log(max(V.y, Epsilon)),
		log(max(V.z, Epsilon)),
	};

	return O;
}

inline Vector3d Vector3dClampToMaxSize(const Vector3d V, const uniform double MaxSize)
{
	const double L = Vector3dLengthSquared(V);
	return L > (MaxSize * MaxSize) ? V * (MaxSize * rsqrt(L)) : V;
}

inline uniform Quat4f Quat4fIdentity()
{
	const uniform Quat4f O = { 0.0f, 0.0f, 0.0f, 1.0f };
	return O;
}

inline Quat4d Quat4dAbs(const Quat4d Q)
{
	const Quat4d O = {
		select(Q.w > 0.0, Q.x, -Q.x),
		select(Q.w > 0.0, Q.y, -Q.y),
		select(Q.w > 0.0, Q.z, -Q.z),
		select(Q.w > 0.0, Q.w, -Q.w),
	};

	return O;
}

inline Quat4d Quat4dMul(const Quat4d Q, const Quat4d P)
{
	const Quat4d O = {
		P.w * Q.x + P.x * Q.w - P.y * Q.z + P.z * Q.y,
		P.w * Q.y + P.x * Q.z + P.y * Q.w - P.z * Q.x,
		P.w * Q.z - P.x * Q.y + P.y * Q.x + P.z * Q.w,
		P.w * Q.w - P.x * Q.x - P.y * Q.y - P.z * Q.z,
	};

	return O;
}

inline Quat4d Quat4dInv(const Quat4d Q)
{
	const Quat4d O = { -Q.x, -Q.y, -Q.z, Q.w };
	return O;
}

inline Quat4d Quat4dMulInv(const Quat4d Q, const Quat4d P)
{
	return Quat4dMul(Q, Quat4dInv(P));
}

inline double Quat4dDot(const Quat4d Q, const Quat4d P)
{
	return Q.x * P.x + Q.y * P.y + Q.z * P.z + Q.w * P.w;
}

inline double Quat4dLengthSquared(const Quat4d Q)
{
	return Quat4dDot(Q, Q);
}

inline double Quat4dLength(const Quat4d Q)
{
	return sqrt(Quat4dLengthSquared(Q));
}

inline Quat4d Quat4dNormalized(const Quat4d Q)
{
	return Q / Quat4dLength(Q);
}

inline Quat4d Quat4dExp(const Vector3d V, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	const double HalfAngleSq = Vector3dLengthSquared(V);

	cif(HalfAngleSq < Epsilon)
	{
		const Quat4d O = { V.x, V.y, V.z, 1.0 };
		return Quat4dNormalized(O);
	}
	else
	{
		const double HalfAngle = sqrt(HalfAngleSq);
		const double C = cos(HalfAngle);
		const double S = sin(HalfAngle) / HalfAngle;

		const Quat4d O = { S * V.x, S * V.y, S * V.z, C };
		return O;
	}
}

inline Vector3d Quat4dLog(const Quat4d Q, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	const Vector3d V = { Q.x, Q.y, Q.z };
	const double LengthSq = Vector3dLengthSquared(V);

	cif(LengthSq < Epsilon)
	{
		return V;
	}
	else
	{
		const double Length = sqrt(LengthSq);
		const double HalfAngle = acos(clamp(Q.w, -1.0d, 1.0d));
		return HalfAngle * (V / Length);
	}
}

inline Vector3d Quat4dToRotationVector(const Quat4d Q, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	return 2.0 * Quat4dLog(Q, Epsilon);
}

inline Quat4d Quat4dFromRotationVector(const Vector3d V, const uniform double Epsilon = UE_SMALL_NUMBER)
{
	return Quat4dExp(V * 0.5, Epsilon);
}

inline Quat4d Quat4dGetShortestArcWith(const Quat4d Q, const Quat4d P)
{
	const double D = Quat4dDot(Q, P);

	const Quat4d O = {
		select(D > 0.0, Q.x, -Q.x),
		select(D > 0.0, Q.y, -Q.y),
		select(D > 0.0, Q.z, -Q.z),
		select(D > 0.0, Q.w, -Q.w),
	};

	return O;
}

inline Vector3d ExtrapolateTranslation(
	const Vector3d Translation,
	const Vector3f Velocity,
	const Vector3f DecayHalflife,
	const uniform float Time,
	const uniform float Epsilon = UE_SMALL_NUMBER)
{
	const Vector3f C = Vector3fDivMax(UE_LN2, DecayHalflife, Epsilon);
	return Translation + Vector3fDivMax(Velocity, C, Epsilon) * (1.0f - Vector3fInvExpApprox(C * Time));
}

inline Quat4d ExtrapolateRotation(
	const Quat4d Rotation,
	const Vector3f Velocity,
	const Vector3f DecayHalflife,
	const uniform float Time,
	const uniform float Epsilon = UE_SMALL_NUMBER)
{
	const Vector3f C = Vector3fDivMax(UE_LN2, DecayHalflife, Epsilon);
	return Quat4dMul(Quat4dFromRotationVector(Vector3fDivMax(Velocity, C, Epsilon) * (1.0f - Vector3fInvExpApprox(C * Time))), Rotation);
}

inline Vector3d ExtrapolateScale(
	const Vector3d Scale,
	const Vector3f Velocity,
	const Vector3f DecayHalflife,
	const uniform float Time,
	const uniform float Epsilon = UE_SMALL_NUMBER)
{
	const Vector3f C = Vector3fDivMax(UE_LN2, DecayHalflife, Epsilon);
	return Vector3fExp(Vector3fDivMax(Velocity, C, Epsilon) * (1.0f - Vector3fInvExpApprox(C * Time))) * Scale;
}

inline float ClipMagnitudeToGreaterThanEpsilon(const float X, const uniform float Epsilon = UE_KINDA_SMALL_NUMBER)
{
	return
		X >= 0.0f && X < +Epsilon ? +Epsilon :
		X <  0.0f && X > -Epsilon ? -Epsilon : X;
}

inline float ComputeDecayHalfLifeFromDiffAndVelocity(
	const float SrcDstDiff,
	const float SrcVelocity,
	const uniform float HalfLife,
	const uniform float HalfLifeMin,
	const uniform float HalfLifeMax,
	const uniform float Epsilon = UE_KINDA_SMALL_NUMBER)
{
	return clamp(HalfLife * (SrcDstDiff / ClipMagnitudeToGreaterThanEpsilon(SrcVelocity, Epsilon)), HalfLifeMin, HalfLifeMax);
}

inline Vector3f ComputeDecayHalfLifeFromDiffAndVelocity(
	const Vector3d SrcDstDiff,
	const Vector3f SrcVelocity,
	const uniform float HalfLife,
	const uniform float HalfLifeMin,
	const uniform float HalfLifeMax,
	const uniform float Epsilon = UE_KINDA_SMALL_NUMBER)
{
	const Vector3f O = {
		ComputeDecayHalfLifeFromDiffAndVelocity((float)SrcDstDiff.x, SrcVelocity.x, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
		ComputeDecayHalfLifeFromDiffAndVelocity((float)SrcDstDiff.y, SrcVelocity.y, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
		ComputeDecayHalfLifeFromDiffAndVelocity((float)SrcDstDiff.z, SrcVelocity.z, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
	};

	return O;
}

export void AnimNextDeadBlendingTransition(
	uniform float BoneRotationDirections[],
	uniform double SourceBoneTranslations[],
	uniform double SourceBoneRotations[],
	uniform double SourceBoneScales3D[],
	uniform float SourceBoneTranslationVelocities[],
	uniform float SourceBoneRotationVelocities[],
	uniform float SourceBoneScaleVelocities[],
	uniform float SourceBoneTranslationDecayHalfLives[],
	uniform float SourceBoneRotationDecayHalfLives[],
	uniform float SourceBoneScaleDecayHalfLives[],
	const uniform double DestBoneTranslations[],
	const uniform double DestBoneRotations[],
	const uniform double DestBoneScales3D[],
	const uniform double CurrBoneTranslations[],
	const uniform double CurrBoneRotations[],
	const uniform double CurrBoneScales3D[],
	const uniform double PrevBoneTranslations[],
	const uniform double PrevBoneRotations[],
	const uniform double PrevBoneScales3D[],
	const uniform int LODBoneNum,
	const uniform float DeltaTime,
	const uniform float ExtrapolationHalfLife,
	const uniform float ExtrapolationHalfLifeMin,
	const uniform float ExtrapolationHalfLifeMax,
	const uniform float MaximumTranslationVelocity,
	const uniform float MaximumRotationVelocity,
	const uniform float MaximumScaleVelocity)
{
	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		Quat4fSave(BoneRotationDirections, BoneIdx, Quat4fIdentity());
	}

	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		const Vector3d SrcTranslationCurr = Vector3dLoad(CurrBoneTranslations, BoneIdx);
		Vector3dSave(SourceBoneTranslations, BoneIdx, SrcTranslationCurr);

		const Vector3d TranslationDiff = SrcTranslationCurr - Vector3dLoad(PrevBoneTranslations, BoneIdx);
		const Vector3f TranslationVelocity = Vector3dClampToMaxSize(TranslationDiff / DeltaTime, MaximumTranslationVelocity);
		Vector3fSave(SourceBoneTranslationVelocities, BoneIdx, TranslationVelocity);

		Vector3fSave(SourceBoneTranslationDecayHalfLives, BoneIdx, ComputeDecayHalfLifeFromDiffAndVelocity(
			Vector3dLoad(DestBoneTranslations, BoneIdx) - SrcTranslationCurr,
			TranslationVelocity,
			ExtrapolationHalfLife,
			ExtrapolationHalfLifeMin,
			ExtrapolationHalfLifeMax));
	}

	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		const Quat4d SrcRotationCurr = Quat4dLoad(CurrBoneRotations, BoneIdx);
		Quat4dSave(SourceBoneRotations, BoneIdx, SrcRotationCurr);

		const Quat4d RotationDiff = Quat4dAbs(Quat4dMulInv(SrcRotationCurr, Quat4dLoad(PrevBoneRotations, BoneIdx)));
		const Vector3f RotationVelocity = Vector3dClampToMaxSize(Quat4dToRotationVector(RotationDiff) / DeltaTime, MaximumRotationVelocity);
		Vector3fSave(SourceBoneRotationVelocities, BoneIdx, RotationVelocity);

		Vector3fSave(SourceBoneRotationDecayHalfLives, BoneIdx, ComputeDecayHalfLifeFromDiffAndVelocity(
			Quat4dToRotationVector(Quat4dAbs(Quat4dMulInv(Quat4dLoad(DestBoneRotations, BoneIdx), SrcRotationCurr))),
			RotationVelocity,
			ExtrapolationHalfLife,
			ExtrapolationHalfLifeMin,
			ExtrapolationHalfLifeMax));
	}

	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		const Vector3d SrcScaleCurr = Vector3dLoad(CurrBoneScales3D, BoneIdx);
		Vector3dSave(SourceBoneScales3D, BoneIdx, SrcScaleCurr);

		const Vector3d ScaleDiff = Vector3dDivMax(SrcScaleCurr, Vector3dLoad(PrevBoneScales3D, BoneIdx));
		const Vector3f ScaleVelocity = Vector3dClampToMaxSize(Vector3dLogMax(ScaleDiff) / DeltaTime, MaximumScaleVelocity);
		Vector3fSave(SourceBoneScaleVelocities, BoneIdx, ScaleVelocity);

		Vector3fSave(SourceBoneScaleDecayHalfLives, BoneIdx, ComputeDecayHalfLifeFromDiffAndVelocity(
			Vector3dDivMax(Vector3dLoad(DestBoneScales3D, BoneIdx), SrcScaleCurr),
			ScaleVelocity,
			ExtrapolationHalfLife,
			ExtrapolationHalfLifeMin,
			ExtrapolationHalfLifeMax));
	}
}


export void AnimNextDeadBlendingTransitionStatic(
	uniform float BoneRotationDirections[],
	uniform double SourceBoneTranslations[],
	uniform double SourceBoneRotations[],
	uniform double SourceBoneScales3D[],
	uniform float SourceBoneTranslationVelocities[],
	uniform float SourceBoneRotationVelocities[],
	uniform float SourceBoneScaleVelocities[],
	uniform float SourceBoneTranslationDecayHalfLives[],
	uniform float SourceBoneRotationDecayHalfLives[],
	uniform float SourceBoneScaleDecayHalfLives[],
	const uniform double CurrBoneTranslations[],
	const uniform double CurrBoneRotations[],
	const uniform double CurrBoneScales[],
	const uniform int LODBoneNum,
	const uniform float ExtrapolationHalfLifeMin)
{
	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		Quat4fSave(BoneRotationDirections, BoneIdx, Quat4fIdentity());
		Vector3dSave(SourceBoneTranslations, BoneIdx, Vector3dLoad(CurrBoneTranslations, BoneIdx));
		Quat4dSave(SourceBoneRotations, BoneIdx, Quat4dLoad(CurrBoneRotations, BoneIdx));
		Vector3dSave(SourceBoneScales3D, BoneIdx, Vector3dLoad(CurrBoneScales, BoneIdx));

		Vector3fSave(SourceBoneTranslationVelocities, BoneIdx, 0.0f);
		Vector3fSave(SourceBoneRotationVelocities, BoneIdx, 0.0f);
		Vector3fSave(SourceBoneScaleVelocities, BoneIdx, 0.0f);
		Vector3fSave(SourceBoneTranslationDecayHalfLives, BoneIdx, ExtrapolationHalfLifeMin);
		Vector3fSave(SourceBoneRotationDecayHalfLives, BoneIdx, ExtrapolationHalfLifeMin);
		Vector3fSave(SourceBoneScaleDecayHalfLives, BoneIdx, ExtrapolationHalfLifeMin);
	}
}

export void AnimNextDeadBlendingApply(
	uniform double DestTranslations[],
	uniform double DestRotations[],
	uniform double DestScales3D[],
	uniform float BoneRotationDirections[],
	const uniform double SourceTranslations[],
	const uniform double SourceRotations[],
	const uniform double SourceScales3D[],
	const uniform float SourceBoneTranslationVelocities[],
	const uniform float SourceBoneRotationVelocities[],
	const uniform float SourceBoneScaleVelocities[],
	const uniform float SourceBoneTranslationDecayHalfLives[],
	const uniform float SourceBoneRotationDecayHalfLives[],
	const uniform float SourceBoneScaleDecayHalfLives[],
	const uniform int LODBoneNum,
	const uniform float Alpha,
	const uniform float TimeSinceTransition)
{
	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		const Vector3d ExtrapolatedTranslation = ExtrapolateTranslation(
			Vector3dLoad(SourceTranslations, BoneIdx),
			Vector3fLoad(SourceBoneTranslationVelocities, BoneIdx),
			Vector3fLoad(SourceBoneTranslationDecayHalfLives, BoneIdx),
			TimeSinceTransition);

		Vector3dSave(DestTranslations, BoneIdx, Vector3dLerp(Vector3dLoad(DestTranslations, BoneIdx), ExtrapolatedTranslation, Alpha));
	}

	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		const Quat4d ExtrapolatedRotation = ExtrapolateRotation(
			Quat4dLoad(SourceRotations, BoneIdx),
			Vector3fLoad(SourceBoneRotationVelocities, BoneIdx),
			Vector3fLoad(SourceBoneRotationDecayHalfLives, BoneIdx),
			TimeSinceTransition);

		const Quat4d RotationDiff = Quat4dGetShortestArcWith(
			Quat4dMulInv(ExtrapolatedRotation, Quat4dLoad(DestRotations, BoneIdx)),
			Quat4fLoad(BoneRotationDirections, BoneIdx));

		Quat4fSave(BoneRotationDirections, BoneIdx, RotationDiff);

		Quat4dSave(DestRotations, BoneIdx,
			Quat4dMul(
				Quat4dFromRotationVector(Quat4dToRotationVector(RotationDiff) * Alpha),
				Quat4dLoad(DestRotations, BoneIdx)));
	}

	foreach(BoneIdx = 0 ... LODBoneNum)
	{
		const Vector3d ExtrapolatedScale = ExtrapolateScale(
			Vector3dLoad(SourceScales3D, BoneIdx),
			Vector3fLoad(SourceBoneScaleVelocities, BoneIdx),
			Vector3fLoad(SourceBoneScaleDecayHalfLives, BoneIdx),
			TimeSinceTransition);

		Vector3dSave(DestScales3D, BoneIdx, Vector3dEerp(Vector3dLoad(DestScales3D, BoneIdx), ExtrapolatedScale, Alpha));
	}
}