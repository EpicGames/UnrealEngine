// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/DecompressionTools.h"

#include "Animation/AnimSequenceDecompressionContext.h"
#include "Animation/AnimCompressionTypes.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimCurveCompressionCodec.h"

#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "Animation/AnimBoneDecompressionData.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequence.h"
#include "HAL/ConsoleManager.h"
#include "Logging/StructuredLog.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "EvaluationVM/EvaluationTaskContext.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "RetargetingTools.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

DECLARE_CYCLE_STAT(TEXT("Build Anim Track Pairs"), STAT_BuildAnimTrackPairs, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("Extract Pose From Anim Data"), STAT_ExtractPoseFromAnimData, STATGROUP_Anim);

DECLARE_CYCLE_STAT(TEXT("AnimSeq GetBonePose"), STAT_AnimSeq_GetBonePose, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("AnimSeq EvalCurveData"), STAT_AnimSeq_EvalCurveData, STATGROUP_Anim);

namespace UE::UAF
{

static IConsoleVariable* CVarForceEvalRawData = IConsoleManager::Get().FindConsoleVariable(TEXT("a.ForceEvalRawData"));
static bool GetForceRawData()
{
	return (CVarForceEvalRawData != nullptr) ? CVarForceEvalRawData->GetBool() : false;
}


//****************************************************************************
// This file contains code extracted from AnimSequence and AnimationDecompression,
// in order to be able to decompress an anim sequence using AnimNext format
//****************************************************************************

struct FGetBonePoseScratchArea : public TThreadSingleton<FGetBonePoseScratchArea>
{
	BoneTrackArray SkeletonPairs;
	BoneTrackArray RotationScalePairs;
	BoneTrackArray TranslationPairs;
	BoneTrackArray AnimScaleRetargetingPairs;
	BoneTrackArray AnimRelativeRetargetingPairs;
	BoneTrackArray OrientAndScaleRetargetingPairs;

	// A bit set that specifies whether a compact bone index has its rotation animated by the sequence or not
	TBitArray<> AnimatedCompactRotations;
};


static bool CanEvaluateRawAnimationData(const UAnimSequence* AnimSequence)
{
#if WITH_EDITOR
	return AnimSequence->IsDataModelValid();
#else
	return false;
#endif
}

static bool UseRawDataForPoseExtraction(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext)
{
	return CanEvaluateRawAnimationData(AnimSequence) && (
#if WITH_EDITOR
		!AnimSequence->IsCompressedDataValid() ||
#endif
		// TODO
		(AnimSequence->GetSkeletonVirtualBoneGuid() != AnimSequence->GetSkeleton()->GetVirtualBoneGuid())// || ExtractionContext.bDisableRetargeting || ExtractionContext.bUseRawData
#if WITH_EDITOR
		|| GetForceRawData()
#endif // WITH_EDITOR
		);
}

bool FDecompressionTools::ShouldUseRawData(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext)
{
	const bool bUseRawData =
#if WITH_EDITOR
		GetForceRawData() ||
#endif // WITH_EDITOR
		UE::UAF::UseRawDataForPoseExtraction(AnimSequence, ExtractionContext) ||
		// TODO
		false; //ExtractionContext.bUseSourceData;

	return bUseRawData;
}

// --- ---
namespace Private
{
	static void DecompressBoneTransforms(
		TBoundValueMap<FBoneTransformAnimationAttribute>& OutBoneTransforms,
		const FCompressedAnimSequence& CompressedData,
		const FAnimExtractContext& ExtractionContext,
		const FEvaluationTaskContext& VMContext,
		FAnimSequenceDecompressionContext& DecompressionContext,
		const TArray<FTransform>& RetargetTransforms,
		const FRootMotionReset& RootMotionReset)
	{
		const FAttributeTypedSetPtr& BoneSet = OutBoneTransforms.GetTypedSet();
		const int32 NumBoneTransforms = OutBoneTransforms.Num();
		const int32 NumTracks = CompressedData.CompressedTrackToSkeletonMapTable.Num();

		const USkeleton* SourceSkeleton = DecompressionContext.GetSourceSkeleton();
		const USkeleton* TargetSkeleton = VMContext.GetSkeleton();
		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
		const bool bIsSkeletonRemappingValid = SkeletonRemapping.IsValid();

		const bool bUseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();
		// TODO
		const bool bDisableRetargeting = false;// ExtractionContext.bDisableRetargeting;

		FGetBonePoseScratchArea& ScratchArea = FGetBonePoseScratchArea::Get();
		BoneTrackArray& RotationScalePairs = ScratchArea.RotationScalePairs;
		BoneTrackArray& TranslationPairs = ScratchArea.TranslationPairs;
		BoneTrackArray& AnimScaleRetargetingPairs = ScratchArea.AnimScaleRetargetingPairs;
		BoneTrackArray& AnimRelativeRetargetingPairs = ScratchArea.AnimRelativeRetargetingPairs;
		BoneTrackArray& OrientAndScaleRetargetingPairs = ScratchArea.OrientAndScaleRetargetingPairs;

		// build a list of desired bones
		RotationScalePairs.Reset();
		TranslationPairs.Reset();
		AnimScaleRetargetingPairs.Reset();
		AnimRelativeRetargetingPairs.Reset();
		OrientAndScaleRetargetingPairs.Reset();

		const bool bIsMeshSpaceAdditive = DecompressionContext.GetAdditiveType() == AAT_RotationOffsetMeshSpace;
		TBitArray<>& AnimatedCompactRotations = ScratchArea.AnimatedCompactRotations;
		if (bIsMeshSpaceAdditive)
		{
			AnimatedCompactRotations.Init(false, NumBoneTransforms);
		}

		// this is not guaranteed for AnimSequences though... If Root is not animated, Track will not exist.
		const bool bIsFirstBoneRoot = BoneSet->GetBindingIndex(FAttributeSetIndex(0)).IsRootBone();

		{
			SCOPE_CYCLE_COUNTER(STAT_BuildAnimTrackPairs);

			for (int32 TrackIndex = 0; TrackIndex < NumTracks; TrackIndex++)
			{
				const int32 SourceSkeletonBoneIndex = CompressedData.GetSkeletonIndexFromTrackIndex(TrackIndex);
				const int32 TargetSkeletonBoneIndex = bIsSkeletonRemappingValid ? SkeletonRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex) : SourceSkeletonBoneIndex;

				if (TargetSkeletonBoneIndex != INDEX_NONE)
				{
					const FAttributeSetIndex BoneSetIndex = BoneSet->GetIndex(FAttributeBindingIndex(TargetSkeletonBoneIndex));

					// Skip bones not in current set
					if (BoneSetIndex.IsValid())
					{
						RotationScalePairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), TrackIndex));

						if (bIsMeshSpaceAdditive)
						{
							AnimatedCompactRotations[BoneSetIndex.GetInt()] = true;
						}

						// Check what retarget mode to use for the translational retargeting for this specific bone.
						const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
							bUseSourceRetargetModes,
							SourceSkeletonBoneIndex,
							TargetSkeletonBoneIndex,
							SourceSkeleton,
							TargetSkeleton,
							bDisableRetargeting);

						// Skip extracting translation component for EBoneTranslationRetargetingMode::Skeleton.
						switch (RetargetMode)
						{
						case EBoneTranslationRetargetingMode::Animation:
							TranslationPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), TrackIndex));
							break;
						case EBoneTranslationRetargetingMode::AnimationScaled:
							TranslationPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), TrackIndex));
							AnimScaleRetargetingPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), SourceSkeletonBoneIndex));
							break;
						case EBoneTranslationRetargetingMode::AnimationRelative:
							TranslationPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), TrackIndex));

							// With baked additives, we can skip 'AnimationRelative' tracks, as the relative transform gets canceled out.
							// (A1 + Rel) - (A2 + Rel) = A1 - A2.
							if (!DecompressionContext.IsAdditiveAnimation())
							{
								AnimRelativeRetargetingPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), SourceSkeletonBoneIndex));
							}
							break;
						case EBoneTranslationRetargetingMode::OrientAndScale:
							TranslationPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), TrackIndex));

							// Additives remain additives, they're not retargeted.
							if (!DecompressionContext.IsAdditiveAnimation())
							{
								OrientAndScaleRetargetingPairs.Add(BoneTrackPair(BoneSetIndex.GetInt(), SourceSkeletonBoneIndex));
							}
							break;
						}
					}
				}
			}
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_ExtractPoseFromAnimData);
			CSV_SCOPED_TIMING_STAT(Animation, ExtractPoseFromAnimData);
			CSV_CUSTOM_STAT(Animation, NumberOfExtractedAnimations, 1, ECsvCustomStatOp::Accumulate);

			DecompressionContext.Seek(ExtractionContext.CurrentTime);

			if (!RotationScalePairs.IsEmpty())
			{
				static_assert(sizeof(FBoneTransformAnimationAttribute) == sizeof(FTransform), "Sizes should match since we coerce between the two");

				TArrayView<FTransform> TransformsView(reinterpret_cast<FTransform*>(OutBoneTransforms.GetData()), NumBoneTransforms);
				CompressedData.BoneCompressionCodec->DecompressPose(DecompressionContext, RotationScalePairs, TranslationPairs, RotationScalePairs, TransformsView);
			}
		}

		// Retarget the pose onto the target skeleton (correcting for differences in rest poses)
		if (SkeletonRemapping.RequiresReferencePoseRetarget())
		{
			if (DecompressionContext.IsAdditiveAnimation())
			{
				// Skip root since it might contain root motion
				for (FAttributeSetIndex BoneSetIndex(bIsFirstBoneRoot ? 1 : 0); BoneSetIndex < NumBoneTransforms; ++BoneSetIndex)
				{
					const int32 TargetSkeletonBoneIndex = BoneSet->GetBindingIndex(BoneSetIndex).GetInt();

					// Mesh space additives do not require fix-up
					if (DecompressionContext.GetAdditiveType() == AAT_LocalSpaceBase)
					{
						OutBoneTransforms[BoneSetIndex].Value.SetRotation(SkeletonRemapping.RetargetAdditiveRotationToTargetSkeleton(TargetSkeletonBoneIndex, OutBoneTransforms[BoneSetIndex].Value.GetRotation()));
					}

					// Check what retarget mode to use for the translational retargeting for this specific bone.
					const int32 SourceSkeletonBoneIndex = SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex);
					const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
						bUseSourceRetargetModes,
						SourceSkeletonBoneIndex,
						TargetSkeletonBoneIndex,
						SourceSkeleton,
						TargetSkeleton,
						bDisableRetargeting);

					if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
					{
						OutBoneTransforms[BoneSetIndex].Value.SetTranslation(SkeletonRemapping.RetargetAdditiveTranslationToTargetSkeleton(TargetSkeletonBoneIndex, OutBoneTransforms[BoneSetIndex].Value.GetTranslation()));
					}
				}
			}
			else
			{
				// Skip root since it might contain root motion
				for (FAttributeSetIndex BoneSetIndex(bIsFirstBoneRoot ? 1 : 0); BoneSetIndex < NumBoneTransforms; ++BoneSetIndex)
				{
					const int32 TargetSkeletonBoneIndex = BoneSet->GetBindingIndex(BoneSetIndex).GetInt();

					OutBoneTransforms[BoneSetIndex].Value.SetRotation(SkeletonRemapping.RetargetBoneRotationToTargetSkeleton(TargetSkeletonBoneIndex, OutBoneTransforms[BoneSetIndex].Value.GetRotation()));

					// Check what retarget mode to use for the translational retargeting for this specific bone.
					const int32 SourceSkeletonBoneIndex = SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex);
					const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
						bUseSourceRetargetModes,
						SourceSkeletonBoneIndex,
						TargetSkeletonBoneIndex,
						SourceSkeleton,
						TargetSkeleton,
						bDisableRetargeting);

					if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
					{
						OutBoneTransforms[BoneSetIndex].Value.SetTranslation(SkeletonRemapping.RetargetBoneTranslationToTargetSkeleton(TargetSkeletonBoneIndex, OutBoneTransforms[BoneSetIndex].Value.GetTranslation()));
					}
				}
			}
		}

		if (bIsFirstBoneRoot)
		{
			// Once pose has been extracted, snap root bone back to first frame if we are extracting root motion.
			if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
			{
				// TODO: Speculatively fetching the reference pose isn't ideal, it might not even be needed
				// It also doesn't make sense if the animation is additive, the reference root transform isn't in additive space
				// and we have nothing to subtract it from, we should probably use the additive identity
				const FAttributeNamedSetPtr& NamedSet = OutBoneTransforms.GetTypedSet()->GetNamedSet();
				const FValueBundlePtr& ReferenceValues = NamedSet->GetDefaultAttributeValues();
				const TBoundValueMap<FBoneTransformAnimationAttribute>* ReferenceBoneTransforms = ReferenceValues->GetBoundValueMaps().Find<FBoneTransformAnimationAttribute>(OutBoneTransforms.GetMappingKey());
				check(ReferenceBoneTransforms != nullptr && !ReferenceBoneTransforms->IsEmpty());

				const FTransform& RootRefTransform = ReferenceBoneTransforms->GetData()->Value;

				FTransform RootTransform = OutBoneTransforms.GetData()->Value;
				RootMotionReset.ResetRootBoneForRootMotion(RootTransform, RootRefTransform);
				OutBoneTransforms.GetData()->Value = RootTransform;
			}
		}

		// Anim Scale Retargeting
		if (!AnimScaleRetargetingPairs.IsEmpty())
		{
			const TArray<FTransform>& AuthoredOnRefSkeleton = RetargetTransforms;
			const TArray<FTransform>& TargetRefSkeleton = TargetSkeleton->GetRefLocalPoses();

			for (const BoneTrackPair& BonePair : AnimScaleRetargetingPairs)
			{
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				if (!AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
				{
					continue;
				}

				// @todo - precache that in FBoneContainer when we have SkeletonIndex->TrackIndex mapping. So we can just apply scale right away.
				const double SourceTranslationLengthSq = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation().SizeSquared();
				if (SourceTranslationLengthSq > (UE_KINDA_SMALL_NUMBER * UE_KINDA_SMALL_NUMBER))
				{
					const FAttributeSetIndex BoneSetIndex(BonePair.AtomIndex);
					const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(BoneSetIndex);
					const double TargetTranslationLength = TargetRefSkeleton[BindingIndex.GetInt()].GetTranslation().Size();
					OutBoneTransforms[BoneSetIndex].Value.ScaleTranslation(TargetTranslationLength / FMath::Sqrt(SourceTranslationLengthSq));
				}
			}
		}

		// Anim Relative Retargeting
		if (!AnimRelativeRetargetingPairs.IsEmpty())
		{
			const TArray<FTransform>& AuthoredOnRefSkeleton = RetargetTransforms;
			const TArray<FTransform>& TargetRefSkeleton = TargetSkeleton->GetRefLocalPoses();

			for (const BoneTrackPair& BonePair : AnimRelativeRetargetingPairs)
			{
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				if (!AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
				{
					continue;
				}

				// Remap the base pose onto the target skeleton so that we are working entirely in target space
				FTransform BaseTransform = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex];
				if (SkeletonRemapping.RequiresReferencePoseRetarget())
				{
					const int32 TargetSkeletonBoneIndex = SkeletonRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex);
					BaseTransform = SkeletonRemapping.RetargetBoneTransformToTargetSkeleton(TargetSkeletonBoneIndex, BaseTransform);
				}

				const FAttributeSetIndex BoneSetIndex(BonePair.AtomIndex);
				const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(BoneSetIndex);
				const FTransform& RefPoseTransform = TargetRefSkeleton[BindingIndex.GetInt()];

				// Apply the retargeting as if it were an additive difference between the current skeleton and the retarget skeleton. 
				OutBoneTransforms[BoneSetIndex].Value.SetRotation(OutBoneTransforms[BoneSetIndex].Value.GetRotation() * BaseTransform.GetRotation().Inverse() * RefPoseTransform.GetRotation());
				OutBoneTransforms[BoneSetIndex].Value.SetTranslation(OutBoneTransforms[BoneSetIndex].Value.GetTranslation() + (RefPoseTransform.GetTranslation() - BaseTransform.GetTranslation()));
				OutBoneTransforms[BoneSetIndex].Value.SetScale3D(OutBoneTransforms[BoneSetIndex].Value.GetScale3D() * (RefPoseTransform.GetScale3D() * BaseTransform.GetSafeScaleReciprocal(BaseTransform.GetScale3D())));
				OutBoneTransforms[BoneSetIndex].Value.NormalizeRotation();
			}
		}

		// TODO : Have to recreate GetRetargetSourceCachedData for optimization
		const int32 NumBonesToOrientAndScaleRetarget = OrientAndScaleRetargetingPairs.Num();
		if (NumBonesToOrientAndScaleRetarget > 0)
		{
			const TArray<FTransform>& AuthoredOnRefSkeleton = RetargetTransforms;
			
			// TODO need to use Reference pose from SkeletalMesh its ref-skeleton (when available) to prevent incorrect proportion changes
			const TArray<FTransform>& TargetRefSkeleton = TargetSkeleton->GetRefLocalPoses();
			
			for (const BoneTrackPair& BonePair : OrientAndScaleRetargetingPairs)
			{
				const FAttributeSetIndex BoneSetIndex(BonePair.AtomIndex);
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				const int32 TargetSkeletonBoneIndex = bIsSkeletonRemappingValid ? SkeletonRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex) : SourceSkeletonBoneIndex;
				if(AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex) && TargetRefSkeleton.IsValidIndex(TargetSkeletonBoneIndex))
				{
					const FVector SourceSkelTrans = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation();
					const FVector TargetSkelTrans = TargetRefSkeleton[TargetSkeletonBoneIndex].GetTranslation();
					
					// If translations are identical, we don't need to do any retargeting
					if (!SourceSkelTrans.Equals(TargetSkelTrans, BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION))
					{
						const float SourceSkelTransLength = SourceSkelTrans.Size();
						const float TargetSkelTransLength = TargetSkelTrans.Size();

						// this only works on non zero vectors.
						if (!FMath::IsNearlyZero(SourceSkelTransLength * TargetSkelTransLength))
						{
							const FVector SourceSkelTransDir = SourceSkelTrans / SourceSkelTransLength;
							const FVector TargetSkelTransDir = TargetSkelTrans / TargetSkelTransLength;

							const FQuat DeltaRotation = FQuat::FindBetweenNormals(SourceSkelTransDir, TargetSkelTransDir);
							const float Scale = TargetSkelTransLength / SourceSkelTransLength;

							FTransform& BoneTransform = OutBoneTransforms[BoneSetIndex].Value;
							const FVector AnimatedTranslation = BoneTransform.GetTranslation();
							// If Translation is not animated, we can just copy the TargetTranslation. No retargeting needs to be done.
							const FVector NewTranslation = (AnimatedTranslation - SourceSkelTrans).IsNearlyZero(BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION) ?
								TargetSkelTrans :
								DeltaRotation.RotateVector(AnimatedTranslation) * Scale;

							BoneTransform.SetTranslation(NewTranslation);
						}
					}
				}
			}
		}

		if (bIsMeshSpaceAdditive)
		{
			// When an animation is a mesh-space additive, bones that aren't animated will end up with some non-identity
			// delta relative to the base used to create the additive. This is because the delta is calculated in mesh-space
			// unlike regular additive animations where bones that aren't animated has an identity delta. For rotations,
			// this mesh-space delta will be the parent bone rotation.
			// However, if a bone isn't animated in the sequence but present on the target skeleton, we have no data for it
			// and the output pose will contain an identity delta which isn't what we want. As such, bones missing from
			// the sequence have their rotation set to their parent.

			for (FAttributeSetIndex BoneSetIndex(0); BoneSetIndex < NumBoneTransforms; ++BoneSetIndex)
			{
				if (!AnimatedCompactRotations[BoneSetIndex.GetInt()])
				{
					// This bone wasn't animated in the sequence, fix it up
					const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
					if (ParentBoneSetIndex.IsValid())
					{
						// Propagate the rotation of our parent to maintain the same orientation in mesh space
						const FQuat ParentRotation = OutBoneTransforms[ParentBoneSetIndex].Value.GetRotation();
						OutBoneTransforms[BoneSetIndex].Value.SetRotation(ParentRotation);
					}
				}
			}
		}
	}

	static void GetBoneTransforms(
		const UAnimSequence* AnimSequence,
		const FAnimExtractContext& ExtractionContext,
		const FEvaluationTaskContext& VMContext,
		TBoundValueMap<FBoneTransformAnimationAttribute>& OutBoneTransforms,
		bool bForceUseRawData)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimSeq_GetBonePose);
		CSV_SCOPED_TIMING_STAT(Animation, AnimSeq_GetBonePose);

		check(!bForceUseRawData || CanEvaluateRawAnimationData(AnimSequence));

		const bool bIsBakedAdditive = !bForceUseRawData && AnimSequence->IsValidAdditive();

		const USkeleton* MySkeleton = AnimSequence->GetSkeleton();
		if (!MySkeleton)
		{
			return;
		}

		// TODO
		const bool bDisableRetargeting = false;// ExtractionContext.bDisableRetargeting;

		// if retargeting is disabled, we initialize pose with 'Retargeting Source' ref pose.
		if (!bIsBakedAdditive && bDisableRetargeting)
		{
			const TArray<FTransform>& AuthoredOnRefSkeleton = AnimSequence->GetRetargetTransforms();

			const int32 NumBoneTransforms = OutBoneTransforms.Num();
			const int32 NumRawSkeletonBones = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetRawBoneNum();
			const FAttributeTypedSetPtr& BoneSet = OutBoneTransforms.GetTypedSet();

			for (FAttributeSetIndex AttributeSetIndex(0); AttributeSetIndex < NumBoneTransforms; ++AttributeSetIndex)
			{
				const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(AttributeSetIndex);
				const int32 SkeletonBoneIndex = BindingIndex.GetInt();

				// Virtual bones are part of the retarget transform pose, so if the pose has not been updated (recently) there might be a mismatch
				if (SkeletonBoneIndex < NumRawSkeletonBones || AuthoredOnRefSkeleton.IsValidIndex(SkeletonBoneIndex))
				{
					OutBoneTransforms[AttributeSetIndex].Value = AuthoredOnRefSkeleton[SkeletonBoneIndex];
				}
			}
		}

		UAnimSequence::FScopedCompressedAnimSequence PlatformCompressedData = AnimSequence->GetCompressedData(ExtractionContext);
		int32 NumTracks = 0;
		{
#if WITH_EDITOR
			NumTracks = bForceUseRawData ? AnimSequence->GetDataModelInterface()->GetNumBoneTracks() : PlatformCompressedData.Get().CompressedTrackToSkeletonMapTable.Num();
#else
			NumTracks = PlatformCompressedData.Get().CompressedTrackToSkeletonMapTable.Num();
#endif
		}
		const bool bTreatAnimAsAdditive = AnimSequence->IsValidAdditive() && !bForceUseRawData; // Raw data is never additive
		const FRootMotionReset RootMotionReset(
			AnimSequence->bEnableRootMotion,
			AnimSequence->RootMotionRootLock,
			AnimSequence->bForceRootLock,
			AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(0.0), /*&RequiredBones*/nullptr),
			bTreatAnimAsAdditive);

#if WITH_EDITOR
		// Evaluate raw (source) curve and bone data
		if (bForceUseRawData)
		{
			// TODO : Curves support
			//{
			//	FSkeletonRemappingCurve RemappedCurve(OutAnimationPoseData.GetCurve(), RequiredBones, AnimSequence->GetSkeleton());
			//	FAnimationPoseData RemappedPoseData(OutAnimationPoseData.GetPose(), RemappedCurve.GetCurve(), OutAnimationPoseData.GetAttributes());

			//	const UE::Anim::DataModel::FEvaluationContext EvaluationContext(ExtractionContext.CurrentTime, DataModelInterface->GetFrameRate(), GetRetargetTransformsSourceName(), GetRetargetTransforms(), Interpolation);
			//	DataModelInterface->Evaluate(RemappedPoseData, EvaluationContext);
			//}

			//if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
			//{
			//	RootMotionReset.ResetRootBoneForRootMotion(OutPose[FCompactPoseBoneIndex(0)], RequiredBones);
			//}

			return;
		}
		else
#endif // WITH_EDITOR
		{
			// Only try and evaluate compressed bone data if the animation contains any bone tracks
			if (NumTracks != 0)
			{
				// Evaluate compressed bone data
				FAnimSequenceDecompressionContext DecompContext(AnimSequence->GetSamplingFrameRate()
					, AnimSequence->GetSamplingFrameRate().AsFrameTime(AnimSequence->GetPlayLength()).RoundToFrame().Value
					, AnimSequence->Interpolation
					, AnimSequence->GetRetargetTransformsSourceName()
					, *PlatformCompressedData.Get().CompressedDataStructure
					, AnimSequence->GetSkeleton()->GetRefLocalPoses()
					, PlatformCompressedData.Get().CompressedTrackToSkeletonMapTable
					, AnimSequence->GetSkeleton()
					, AnimSequence->IsValidAdditive()
					, AnimSequence->GetAdditiveAnimType());

				DecompressBoneTransforms(OutBoneTransforms, PlatformCompressedData.Get(), ExtractionContext, VMContext, DecompContext, AnimSequence->GetRetargetTransforms(), RootMotionReset);
			}
		}
	}

	struct FBlendedCurveWrapper : public FBlendedCurve
	{
		UE::Anim::FCurveElement& operator[](int32 Index)
		{
			return Elements[Index];
		}

		const UE::Anim::FCurveElement& operator[](int32 Index) const
		{
			return Elements[Index];
		}

		bool IsSorted() const
		{
			return bSorted;
		}
	};
}

void FDecompressionTools::GetAnimationPose(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData, bool bForceUseRawData)
{
	if (OutAnimationPoseData.GetRefPose().IsValid() == false)
	{
		return;
	}

	// @todo anim: if compressed and baked in the future, we don't have to do this 
	if (bForceUseRawData && AnimSequence->IsValidAdditive())
	{
		switch (AnimSequence->GetAdditiveAnimType())
		{
			case AAT_LocalSpaceBase:
			{
				GetBonePose_Additive(AnimSequence, ExtractionContext, OutAnimationPoseData);
			}
			break;

			case AAT_RotationOffsetMeshSpace:
			{
				GetBonePose_AdditiveMeshRotationOnly(AnimSequence, ExtractionContext, OutAnimationPoseData);
			}
			break;

			default:
				break;
		}
	}
	else
	{
		GetBonePose(AnimSequence, ExtractionContext, OutAnimationPoseData, bForceUseRawData);
	}

	// Check that all bone atoms coming from animation are normalized
#if DO_CHECK && WITH_EDITORONLY_DATA
	check(OutAnimationPoseData.LocalTransformsView.IsValid());
#endif

}

void FDecompressionTools::GetAnimationPose(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FEvaluationTaskContext& VMContext, FValueBundle& OutCollection, bool bForceUseRawData)
{
	using namespace UE::UAF;

	const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute>();
	TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutCollection.GetBoundValueMaps().Find<FBoneTransformAnimationAttribute>(MappingKey);
	if (BoneTransforms == nullptr || BoneTransforms->IsEmpty())
	{
		// Not requesting bone transforms
		return;
	}

	check(VMContext.IsValid());

	// @todo anim: if compressed and baked in the future, we don't have to do this 
	if (bForceUseRawData && AnimSequence->IsValidAdditive())
	{
		switch (AnimSequence->GetAdditiveAnimType())
		{
		case AAT_LocalSpaceBase:
		{
			//GetBonePose_Additive(AnimSequence, ExtractionContext, OutAnimationPoseData);
		}
		break;

		case AAT_RotationOffsetMeshSpace:
		{
			//GetBonePose_AdditiveMeshRotationOnly(AnimSequence, ExtractionContext, OutAnimationPoseData);
		}
		break;

		default:
			break;
		}
	}
	else
	{
		Private::GetBoneTransforms(AnimSequence, ExtractionContext, VMContext, *BoneTransforms, bForceUseRawData);
	}

	// Check that all bone atoms coming from animation are normalized
#if DO_CHECK && WITH_EDITORONLY_DATA
	//check(OutAnimationPoseData.LocalTransformsView.IsValid());
#endif
}

void FDecompressionTools::GetBonePose(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData, bool bForceUseRawData /*= false*/)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimSeq_GetBonePose);
	CSV_SCOPED_TIMING_STAT(Animation, AnimSeq_GetBonePose);

	const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = OutAnimationPoseData.GetLODBoneIndexToSkeletonBoneIndexMap();

	check(!bForceUseRawData || CanEvaluateRawAnimationData(AnimSequence));

	const bool bIsBakedAdditive = !bForceUseRawData && AnimSequence->IsValidAdditive();

	const USkeleton* MySkeleton = AnimSequence->GetSkeleton();
	if (!MySkeleton)
	{
		OutAnimationPoseData.SetRefPose(bIsBakedAdditive);
		return;
	}

	const bool bDisableRetargeting = OutAnimationPoseData.GetDisableRetargeting();

	// initialize with ref-pose
	if (bIsBakedAdditive)
	{
		//When using baked additive ref pose is identity
		OutAnimationPoseData.SetRefPose(bIsBakedAdditive);
	}
	else
	{
		// if retargeting is disabled, we initialize pose with 'Retargeting Source' ref pose.
		if (bDisableRetargeting)
		{
			const TArray<FTransform>& AuthoredOnRefSkeleton = AnimSequence->GetRetargetTransforms();

			const int32 NumLODBones = LODBoneIndexToSkeletonBoneIndexMap.Num();
			const int32 NumRawSkeletonBones = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetRawBoneNum();

			for (int LODBoneIndex = 0; LODBoneIndex < NumLODBones; LODBoneIndex++)
			{
				const int32 SkeletonBoneIndex = LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex];

				// Virtual bones are part of the retarget transform pose, so if the pose has not been updated (recently) there might be a mismatch
				if (SkeletonBoneIndex < NumRawSkeletonBones || AuthoredOnRefSkeleton.IsValidIndex(SkeletonBoneIndex))
				{
					OutAnimationPoseData.LocalTransformsView[LODBoneIndex] = AuthoredOnRefSkeleton[SkeletonBoneIndex];
				}
			}
		}
		else
		{
			OutAnimationPoseData.SetRefPose();
		}
	}

	UAnimSequence::FScopedCompressedAnimSequence PlatformCompressedData = AnimSequence->GetCompressedData(ExtractionContext);
	int32 NumTracks = 0;
	{
#if WITH_EDITOR
		NumTracks = bForceUseRawData ? AnimSequence->GetDataModelInterface()->GetNumBoneTracks() : PlatformCompressedData.Get().CompressedTrackToSkeletonMapTable.Num();
#else
		NumTracks = PlatformCompressedData.Get().CompressedTrackToSkeletonMapTable.Num();
#endif
	}
	const bool bTreatAnimAsAdditive = (AnimSequence->IsValidAdditive() && !bForceUseRawData); // Raw data is never additive
	const bool bApplyForceRootLock =
#if WITH_EDITOR
		!ExtractionContext.bIgnoreRootLock &&
#endif
		AnimSequence->bForceRootLock;
	const FRootMotionReset RootMotionReset(AnimSequence->bEnableRootMotion, AnimSequence->RootMotionRootLock, bApplyForceRootLock, AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(0.0), /*&RequiredBones*/nullptr), bTreatAnimAsAdditive);

#if WITH_EDITOR
	// Evaluate raw (source) curve and bone data
	if (bForceUseRawData)
	{
		// TODO : Curves support
		//{
		//	FSkeletonRemappingCurve RemappedCurve(OutAnimationPoseData.GetCurve(), RequiredBones, AnimSequence->GetSkeleton());
		//	FAnimationPoseData RemappedPoseData(OutAnimationPoseData.GetPose(), RemappedCurve.GetCurve(), OutAnimationPoseData.GetAttributes());

		//	const UE::Anim::DataModel::FEvaluationContext EvaluationContext(ExtractionContext.CurrentTime, DataModelInterface->GetFrameRate(), GetRetargetTransformsSourceName(), GetRetargetTransforms(), Interpolation);
		//	DataModelInterface->Evaluate(RemappedPoseData, EvaluationContext);
		//}

		//if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
		//{
		//	RootMotionReset.ResetRootBoneForRootMotion(OutPose[FCompactPoseBoneIndex(0)], RequiredBones);
		//}

		return;
	}
	else
#endif // WITH_EDITOR
	{
		// Only try and evaluate compressed bone data if the animation contains any bone tracks
		if (NumTracks != 0)
		{
			// Evaluate compressed bone data
			FAnimSequenceDecompressionContext DecompContext(AnimSequence->GetSamplingFrameRate()
				, AnimSequence->GetSamplingFrameRate().AsFrameTime(AnimSequence->GetPlayLength()).RoundToFrame().Value
				, AnimSequence->Interpolation
				, AnimSequence->GetRetargetTransformsSourceName()
				, *PlatformCompressedData.Get().CompressedDataStructure
				, AnimSequence->GetSkeleton()->GetRefLocalPoses()
				, PlatformCompressedData.Get().CompressedTrackToSkeletonMapTable
				, AnimSequence->GetSkeleton()
				, AnimSequence->IsValidAdditive()
				, AnimSequence->GetAdditiveAnimType());

			DecompressPose(OutAnimationPoseData, PlatformCompressedData.Get(), ExtractionContext, DecompContext, AnimSequence->GetRetargetTransforms(), RootMotionReset);
		}
	}
}

void FDecompressionTools::GetBonePose_Additive(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData)
{
	// @TODO: Implement now that we have additive, only used for 'bForceUseRawData' for now.
	// Below exists to prevent reading garbage / warnings about non-normalized rotations
	GetBonePose(AnimSequence, ExtractionContext, OutAnimationPoseData);
}

void FDecompressionTools::GetBonePose_AdditiveMeshRotationOnly(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData)
{
	// @TODO: Implement now that we have additive, only used for 'bForceUseRawData' for now.
	// Below exists to prevent reading garbage / warnings about non-normalized rotations
	GetBonePose(AnimSequence, ExtractionContext, OutAnimationPoseData);
}

void FDecompressionTools::GetAnimationCurves(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FBlendedCurve& OutCurves, bool bForceUseRawData)
{
	AnimSequence->EvaluateCurveData(OutCurves, ExtractionContext.CurrentTime, bForceUseRawData);
}

void FDecompressionTools::GetAnimationCurves(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FEvaluationTaskContext& VMContext, FValueBundle& OutCollection, bool bForceUseRawData)
{
	using namespace UE::UAF;

	SCOPE_CYCLE_COUNTER(STAT_AnimSeq_EvalCurveData);

	if (!VMContext.IsValid())
	{
		return;
	}

	UAnimSequence::FScopedCompressedAnimSequence PlatformCompressedData = AnimSequence->GetCompressedData(ExtractionContext);
	const FCompressedAnimSequence& CompressedData = PlatformCompressedData.Get();

	const bool bEvaluateRawData = bForceUseRawData || !CompressedData.IsCurveDataValid(AnimSequence);
	check(!bForceUseRawData || AnimSequence->CanEvaluateRawAnimationData());
	if (AnimSequence->CanEvaluateRawAnimationData() && bEvaluateRawData)
	{
		// TODO: Raw curve support
#if WITH_EDITOR
		//UE::Anim::EvaluateFloatCurvesFromModel(DataModelInterface.GetInterface(), OutCurve, ExtractionContext.CurrentTime);
#else
		//Super::EvaluateCurveData(OutCurve, ExtractionContext, bForceUseRawData);
#endif
	}
	else if (CompressedData.IsCurveDataValid(AnimSequence))
	{
		if (CompressedData.CurveCompressionCodec)
		{
			CSV_SCOPED_TIMING_STAT(Animation, EvaluateCurveData);

			Private::FBlendedCurveWrapper Curves;
			CompressedData.CurveCompressionCodec->DecompressCurves(CompressedData, Curves, ExtractionContext.CurrentTime);

			const int32 NumDecompressedCurves = Curves.Num();
			if (NumDecompressedCurves != 0)
			{
				ensureMsgf(Curves.IsSorted(), TEXT("Decompressed curves must be sorted"));

				const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>();
				TBoundValueMap<FFloatAnimationAttribute>* FloatAttributes = OutCollection.GetBoundValueMaps().Find<FFloatAnimationAttribute>(MappingKey);
				TUnboundValueMap<FFloatAnimationAttribute>* FloatMap = nullptr;

				if (FloatAttributes != nullptr && !FloatAttributes->IsEmpty())
				{
					// Extract our set defined floats out into our set map
					const FAttributeTypedSetPtr& FloatSet = FloatAttributes->GetTypedSet();

					// TODO: We can perform a tape merge since both inputs are in sorted order (in reverse order since we remove values from our temp curve array)

					for (int32 DecompressedCurveIndex = 0; DecompressedCurveIndex < NumDecompressedCurves; ++DecompressedCurveIndex)
					{
						const UE::Anim::FCurveElement& CurveElement = Curves[DecompressedCurveIndex];

						if (const FAttributeBindingIndex BindingIndex = FloatSet->FindBindingIndex(CurveElement.Name))
						{
							// This curve is mapped within our binding, it is statically defined
							if (const FAttributeSetIndex AttributeIndex = FloatSet->GetIndex(BindingIndex))
							{
								// This curve is within our current named set, keep it
								(*FloatAttributes)[AttributeIndex].Value = CurveElement.Value;
							}
							else
							{
								// This curve is stripped and isn't within our current named set
							}
						}
						else
						{
							// This curve isn't mapped within our binding, it is dynamically defined
							if (FloatMap == nullptr)
							{
								FloatMap = OutCollection.GetUnboundValueMaps().Add<FFloatAnimationAttribute>();
							}

							FloatMap->Append(CurveElement.Name, FFloatAnimationAttribute{ CurveElement.Value });
						}
					}
				}
				else
				{
					// We have no float attributes within our named set, everything is dynamically defined
					FloatMap = OutCollection.GetUnboundValueMaps().Add<FFloatAnimationAttribute>();

					for (int32 DecompressedCurveIndex = 0; DecompressedCurveIndex < NumDecompressedCurves; ++DecompressedCurveIndex)
					{
						FloatMap->Append(Curves[DecompressedCurveIndex].Name, FFloatAnimationAttribute{ Curves[DecompressedCurveIndex].Value });
					}
				}
			}
		}
	}
}

void FDecompressionTools::GetAnimationAttributes(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FReferencePose& RefPose, UE::Anim::FStackAttributeContainer& OutAttributes, bool bForceUseRawData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EvaluateAttributes);

#if WITH_EDITOR
	if (bForceUseRawData)
	{
		AnimSequence->ValidateModel();

		for (const FAnimatedBoneAttribute& Attribute : AnimSequence->GetDataModel()->GetAttributes())
		{
			const int32 LODBoneIndex = RefPose.GetLODBoneIndexFromSkeletonBoneIndex(Attribute.Identifier.GetBoneIndex());
			// Only add attribute if the bone its tied to exists in the currently evaluated set of bones
			if (LODBoneIndex != INDEX_NONE)
			{
				UE::Anim::Attributes::GetAttributeValue(OutAttributes, FCompactPoseBoneIndex(LODBoneIndex), Attribute, ExtractionContext.CurrentTime);
			}
		}
	}
	else
#endif // WITH_EDITOR
	{
		for (const TPair<FAnimationAttributeIdentifier, FAttributeCurve>& BakedAttribute : AnimSequence->AttributeCurves)
		{
			const int32 LODBoneIndex = RefPose.GetLODBoneIndexFromSkeletonBoneIndex(BakedAttribute.Key.GetBoneIndex());
			// Only add attribute if the bone its tied to exists in the currently evaluated set of bones
			if (LODBoneIndex != INDEX_NONE)
			{
				UE::Anim::FAttributeId Info(BakedAttribute.Key.GetName(), FCompactPoseBoneIndex(LODBoneIndex));
				uint8* AttributePtr = OutAttributes.FindOrAdd(BakedAttribute.Key.GetType(), Info);
				BakedAttribute.Value.EvaluateToPtr(BakedAttribute.Key.GetType(), ExtractionContext.CurrentTime, AttributePtr);
			}
		}
	}
}

void FDecompressionTools::GetAnimationAttributes(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FEvaluationTaskContext& VMContext, FValueBundle& OutCollection, bool bForceUseRawData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EvaluateAttributes);

	if (!VMContext.IsValid())
	{
		return;
	}

#if WITH_EDITOR && 0	// TODO: Add support for raw attributes
	if (bForceUseRawData)
	{
		AnimSequence->ValidateModel();

		for (const FAnimatedBoneAttribute& Attribute : AnimSequence->GetDataModel()->GetAttributes())
		{
			const int32 LODBoneIndex = RefPose.GetLODBoneIndexFromSkeletonBoneIndex(Attribute.Identifier.GetBoneIndex());
			// Only add attribute if the bone its tied to exists in the currently evaluated set of bones
			if (LODBoneIndex != INDEX_NONE)
			{
				UE::Anim::Attributes::GetAttributeValue(OutAttributes, FCompactPoseBoneIndex(LODBoneIndex), Attribute, ExtractionContext.CurrentTime);
			}
		}
	}
	else
#endif // WITH_EDITOR
	{
		if (AnimSequence->AttributeCurves.IsEmpty())
		{
			return;
		}

		const FAttributeNamedSetPtr& NamedSet = VMContext.GetNamedSet();
		const FAttributeTypedSetPtr BoneTypedSet = NamedSet->FindTypedSet<FBoneTransformAnimationAttribute>();

		for (const TPair<FAnimationAttributeIdentifier, FAttributeCurve>& BakedAttribute : AnimSequence->AttributeCurves)
		{
			UScriptStruct* AttributeType = BakedAttribute.Key.GetType();

			if (const FSkeletonPoseBoneIndex SkeletonBoneIndex = FSkeletonPoseBoneIndex(BakedAttribute.Key.GetBoneIndex()))
			{
				// This attribute is attached to a bone

				const FAttributeBindingIndex BoneBindingIndex(SkeletonBoneIndex);
				if (BoneTypedSet && BoneTypedSet->GetIndex(BoneBindingIndex).IsValid())
				{
					// Our parent bone is present within the current named set
					const FName AttributeName = BakedAttribute.Key.GetName();

					if (const FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet(AttributeType))
					{
						if (const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(AttributeName))
						{
							// This attribute is present within the current named set

							FBoundValueMap* Map = OutCollection.GetBoundValueMaps().Find(FAttributeMappingKey::MakeFromTo(AttributeType));
							checkf(Map != nullptr, TEXT("This attribute exists within our named set but isn't present in the output collection"));

							Map->SetValueWithSetter(SetIndex, [&ExtractionContext, &BakedAttribute](UScriptStruct* ValueType, uint8* OutValuePtr)
								{
									BakedAttribute.Value.EvaluateToPtr(ValueType, ExtractionContext.CurrentTime, OutValuePtr);
								});
						}
						else
						{
							// This attribute is stripped within the current named set
						}
					}
					else
					{
						// This attribute type is unknown within our binding, it is dynamic
						FUnboundValueMap* AttributeMap = OutCollection.GetUnboundValueMaps().FindOrAdd(AttributeType);

						AttributeMap->AddWithSetter(AttributeName, [&ExtractionContext, &BakedAttribute](UScriptStruct* ValueType, uint8* OutValuePtr)
							{
								BakedAttribute.Value.EvaluateToPtr(ValueType, ExtractionContext.CurrentTime, OutValuePtr);
							});
					}
				}
				else
				{
					// Our parent bone is stripped within the current named set
				}
			}
			else
			{
				// This attribute isn't attached to a bone
				// These are not currently supported by AnimBP nor UAF
#if 0
				const FName AttributeName = BakedAttribute.Key.GetName();

				if (const FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet(AttributeType))
				{
					if (const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(AttributeName))
					{
						// This attribute is present within the current named set

						FBoundValueMap* Map = OutCollection.FindSetMap(FAttributeMappingKey::MakeFromTo(AttributeType));
						checkf(Map != nullptr, TEXT("This attribute exists within our named set but isn't present in the output collection"));

						Map->SetValueWithSetter(SetIndex, [&ExtractionContext, &BakedAttribute](UScriptStruct* ValueType, uint8* OutValuePtr)
							{
								BakedAttribute.Value.EvaluateToPtr(ValueType, ExtractionContext.CurrentTime, OutValuePtr);
							});
					}
					else
					{
						// This attribute is stripped within the current named set
					}
				}
				else
				{
					// This attribute type is unknown within our binding, it is dynamic
					FUnboundValueMap* AttributeMap = OutCollection.FindOrAddMap(AttributeType);

					AttributeMap->AddWithSetter(AttributeName, [&ExtractionContext, &BakedAttribute](UScriptStruct* ValueType, uint8* OutValuePtr)
						{
							BakedAttribute.Value.EvaluateToPtr(ValueType, ExtractionContext.CurrentTime, OutValuePtr);
						});
				}
#endif
			}
		}
	}
}

// --- ---

void FDecompressionTools::DecompressPose(FLODPose& OutAnimationPoseData,
										const FCompressedAnimSequence& CompressedData,
										const FAnimExtractContext& ExtractionContext,
										FAnimSequenceDecompressionContext& DecompressionContext,
										FName RetargetSource,
										const FRootMotionReset& RootMotionReset)
{
	const TArray<FTransform>& RetargetTransforms = DecompressionContext.GetSourceSkeleton()->GetRefLocalPoses(RetargetSource);
	DecompressPose(OutAnimationPoseData, CompressedData, ExtractionContext, DecompressionContext, RetargetTransforms, RootMotionReset);
}

void FDecompressionTools::DecompressPose(FLODPose& OutAnimationPoseData,
										const FCompressedAnimSequence& CompressedData,
										const FAnimExtractContext& ExtractionContext,
										FAnimSequenceDecompressionContext& DecompressionContext,
										const TArray<FTransform>& RetargetTransforms,
										const FRootMotionReset& RootMotionReset)
{
	const FReferencePose& ReferencePose = OutAnimationPoseData.GetRefPose();
	const TArrayView<const FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap = OutAnimationPoseData.GetLODBoneIndexToSkeletonBoneIndexMap();
	const TArrayView<const FBoneIndexType> SkeletonToLODBoneIndexes = ReferencePose.GetSkeletonBoneIndexToLODBoneIndexMap(); // Full list of Skeleton to LOD conversion
	const int32 NumLODBoneIndexes = LODBoneIndexToSkeletonBoneIndexMap.Num();

	const int32 NumTracks = CompressedData.CompressedTrackToSkeletonMapTable.Num();

	const USkeleton* SourceSkeleton = DecompressionContext.GetSourceSkeleton();
	const USkeleton* TargetSkeleton = OutAnimationPoseData.GetSkeletonAsset();
	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
	const bool bIsSkeletonRemappingValid = SkeletonRemapping.IsValid();

	const bool bUseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();
	const bool bDisableRetargeting = OutAnimationPoseData.GetDisableRetargeting();

	FGetBonePoseScratchArea& ScratchArea = FGetBonePoseScratchArea::Get();
	BoneTrackArray& SkeletonPairs = ScratchArea.SkeletonPairs;
	BoneTrackArray& RotationScalePairs = ScratchArea.RotationScalePairs;
	BoneTrackArray& TranslationPairs = ScratchArea.TranslationPairs;
	BoneTrackArray& AnimScaleRetargetingPairs = ScratchArea.AnimScaleRetargetingPairs;
	BoneTrackArray& AnimRelativeRetargetingPairs = ScratchArea.AnimRelativeRetargetingPairs;
	BoneTrackArray& OrientAndScaleRetargetingPairs = ScratchArea.OrientAndScaleRetargetingPairs;

	// build a list of desired bones
	SkeletonPairs.Reset();
	RotationScalePairs.Reset();
	TranslationPairs.Reset();
	AnimScaleRetargetingPairs.Reset();
	AnimRelativeRetargetingPairs.Reset();
	OrientAndScaleRetargetingPairs.Reset();

	const bool bIsMeshSpaceAdditive = DecompressionContext.GetAdditiveType() == AAT_RotationOffsetMeshSpace;
	TBitArray<>& AnimatedCompactRotations = ScratchArea.AnimatedCompactRotations;
	if (bIsMeshSpaceAdditive)
	{
		AnimatedCompactRotations.Init(false, NumLODBoneIndexes);
		OutAnimationPoseData.Flags |= ELODPoseFlags::MeshSpaceAdditive;
	}
	else if (DecompressionContext.GetAdditiveType() == AAT_LocalSpaceBase)
	{
		OutAnimationPoseData.Flags |= ELODPoseFlags::LocalSpaceAdditive;
	}

	// Optimization: assuming first index is root bone. That should always be the case in Skeletons.
	checkSlow((LODBoneIndexToSkeletonBoneIndexMap[0] == FMeshPoseBoneIndex(0).GetInt()));
	// this is not guaranteed for AnimSequences though... If Root is not animated, Track will not exist.
	const bool bFirstTrackIsRootBone = (CompressedData.GetSkeletonIndexFromTrackIndex(0) == 0);

	{
		SCOPE_CYCLE_COUNTER(STAT_BuildAnimTrackPairs);

		// Handle root bone separately if it is track 0. so we start w/ Index 1.
		for (int32 TrackIndex = (bFirstTrackIsRootBone ? 1 : 0); TrackIndex < NumTracks; TrackIndex++)
		{
			const int32 SourceSkeletonBoneIndex = CompressedData.GetSkeletonIndexFromTrackIndex(TrackIndex);
			const int32 TargetSkeletonBoneIndex = bIsSkeletonRemappingValid ? SkeletonRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex) : SourceSkeletonBoneIndex;

			if (TargetSkeletonBoneIndex != INDEX_NONE)
			{
				const int32 LODBoneIndex = TargetSkeletonBoneIndex < SkeletonToLODBoneIndexes.Num() ? SkeletonToLODBoneIndexes[TargetSkeletonBoneIndex] : INDEX_NONE;

				if (LODBoneIndex != INDEX_NONE && LODBoneIndex < NumLODBoneIndexes) // skip bones not in current LOD
				{
					RotationScalePairs.Add(BoneTrackPair(LODBoneIndex, TrackIndex));

					if (bIsMeshSpaceAdditive)
					{
						AnimatedCompactRotations[LODBoneIndex] = true;
					}

					// Check what retarget mode to use for the translational retargeting for this specific bone.
					const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
						bUseSourceRetargetModes,
						SourceSkeletonBoneIndex,
						TargetSkeletonBoneIndex,
						SourceSkeleton,
						TargetSkeleton,
						bDisableRetargeting);

					// Skip extracting translation component for EBoneTranslationRetargetingMode::Skeleton.
					switch (RetargetMode)
					{
					case EBoneTranslationRetargetingMode::Skeleton:
						SkeletonPairs.Add(BoneTrackPair(LODBoneIndex, TrackIndex));
						break;
					case EBoneTranslationRetargetingMode::Animation:
						TranslationPairs.Add(BoneTrackPair(LODBoneIndex, TrackIndex));
						break;
					case EBoneTranslationRetargetingMode::AnimationScaled:
						TranslationPairs.Add(BoneTrackPair(LODBoneIndex, TrackIndex));
						AnimScaleRetargetingPairs.Add(BoneTrackPair(LODBoneIndex, SourceSkeletonBoneIndex));
						break;
					case EBoneTranslationRetargetingMode::AnimationRelative:
						TranslationPairs.Add(BoneTrackPair(LODBoneIndex, TrackIndex));

						// With baked additives, we can skip 'AnimationRelative' tracks, as the relative transform gets canceled out.
						// (A1 + Rel) - (A2 + Rel) = A1 - A2.
						if (!DecompressionContext.IsAdditiveAnimation())
						{
							AnimRelativeRetargetingPairs.Add(BoneTrackPair(LODBoneIndex, SourceSkeletonBoneIndex));
						}
						break;
					case EBoneTranslationRetargetingMode::OrientAndScale:
						TranslationPairs.Add(BoneTrackPair(LODBoneIndex, TrackIndex));

						// Additives remain additives, they're not retargeted.
						if (!DecompressionContext.IsAdditiveAnimation())
						{
							OrientAndScaleRetargetingPairs.Add(BoneTrackPair(LODBoneIndex, SourceSkeletonBoneIndex));
						}
						break;
					}
				}
			}
		}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_ExtractPoseFromAnimData);
		CSV_SCOPED_TIMING_STAT(Animation, ExtractPoseFromAnimData);
		CSV_CUSTOM_STAT(Animation, NumberOfExtractedAnimations, 1, ECsvCustomStatOp::Accumulate);

		DecompressionContext.Seek(ExtractionContext.CurrentTime);

		// Handle Root Bone separately
		if (bFirstTrackIsRootBone)
		{
			const int32 TrackIndex = 0;
			const int32 LODRootBone = 0;
			FTransform RootAtom = OutAnimationPoseData.LocalTransformsView[0];

			CompressedData.BoneCompressionCodec->DecompressBone(DecompressionContext, TrackIndex, RootAtom);

			// Retarget the root onto the target skeleton (correcting for differences in rest poses)
			if (SkeletonRemapping.RequiresReferencePoseRetarget())
			{
				// Root bone does not require fix-up for additive animations as there is no parent delta rotation to account for
				if (!DecompressionContext.IsAdditiveAnimation())
				{
					constexpr int32 SkeletonBoneIndex = 0;

					// Check what retarget mode to use for the translational retargeting for this specific bone.
					const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
						bUseSourceRetargetModes,
						SkeletonBoneIndex,
						SkeletonBoneIndex,
						SourceSkeleton,
						TargetSkeleton,
						bDisableRetargeting);

					RootAtom.SetRotation(SkeletonRemapping.RetargetBoneRotationToTargetSkeleton(SkeletonBoneIndex, RootAtom.GetRotation()));
					if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
					{
						RootAtom.SetTranslation(SkeletonRemapping.RetargetBoneTranslationToTargetSkeleton(SkeletonBoneIndex, RootAtom.GetTranslation()));
					}
				}
			}

			// @laurent - we should look into splitting rotation and translation tracks, so we don't have to process translation twice.
			FRetargetingTools::RetargetBoneTransform(OutAnimationPoseData.GetRefPose()
				, DecompressionContext.GetSourceSkeleton()
				, OutAnimationPoseData.GetSkeletonAsset()
				, DecompressionContext.AnimName
				, RetargetTransforms
				, RootAtom
				, 0
				, LODRootBone
				, DecompressionContext.IsAdditiveAnimation()
				, bDisableRetargeting);

			OutAnimationPoseData.LocalTransformsView[0] = RootAtom;
		}

		if (RotationScalePairs.Num() > 0)
		{
#if DEFAULT_SOA
			// get the remaining bone atoms
			CompressedData.BoneCompressionCodec->DecompressPose(DecompressionContext, UE::Anim::FAnimPoseDecompressionData(RotationScalePairs, TranslationPairs, RotationScalePairs, OutAnimationPoseData.LocalTransformsView.Rotations, OutAnimationPoseData.LocalTransformsView.Translations, OutAnimationPoseData.LocalTransformsView.Scales3D));
#else
			// get the remaining bone atoms
			TArrayView<FTransform> OutPoseBones = OutAnimationPoseData.LocalTransforms.Transforms;
			CompressedData.BoneCompressionCodec->DecompressPose(DecompressionContext, RotationScalePairs, TranslationPairs, RotationScalePairs, OutPoseBones);
#endif
		}
	}

	// Retarget the pose onto the target skeleton (correcting for differences in rest poses)
	if (SkeletonRemapping.RequiresReferencePoseRetarget())
	{
		if (DecompressionContext.IsAdditiveAnimation())
		{
			for (int32 LODBoneIndex = (bFirstTrackIsRootBone ? 1 : 0); LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
			{
				const int32 TargetSkeletonBoneIndex = LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex];

				// Mesh space additives do not require fix-up
				if (DecompressionContext.GetAdditiveType() == AAT_LocalSpaceBase)
				{
					OutAnimationPoseData.LocalTransformsView[LODBoneIndex].SetRotation(SkeletonRemapping.RetargetAdditiveRotationToTargetSkeleton(TargetSkeletonBoneIndex, OutAnimationPoseData.LocalTransformsView[LODBoneIndex].GetRotation()));
				}

				FRetargetingTools::TranslationRetargetReferencePoseBone</*bIsAdditive=*/true>(
					SourceSkeleton,
					TargetSkeleton,
					SkeletonRemapping,
					bUseSourceRetargetModes,
					bDisableRetargeting,
					LODBoneIndex,
					TargetSkeletonBoneIndex,
					OutAnimationPoseData);
			}
		}
		else
		{
			for (int32 LODBoneIndex = (bFirstTrackIsRootBone ? 1 : 0); LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
			{
				const int32 TargetSkeletonBoneIndex = LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex];
				OutAnimationPoseData.LocalTransformsView[LODBoneIndex].SetRotation(SkeletonRemapping.RetargetBoneRotationToTargetSkeleton(TargetSkeletonBoneIndex, OutAnimationPoseData.LocalTransformsView[LODBoneIndex].GetRotation()));

				FRetargetingTools::TranslationRetargetReferencePoseBone</*bIsAdditive=*/false>(
					SourceSkeleton,
					TargetSkeleton,
					SkeletonRemapping,
					bUseSourceRetargetModes,
					bDisableRetargeting,
					LODBoneIndex,
					TargetSkeletonBoneIndex,
					OutAnimationPoseData);				
			}
		}
	}

	// Once pose has been extracted, snap root bone back to first frame if we are extracting root motion.
	if ((ExtractionContext.bExtractRootMotion && RootMotionReset.bEnableRootMotion) || RootMotionReset.bForceRootLock)
	{
		FTransform RootTransform = OutAnimationPoseData.LocalTransformsView[0];
		RootMotionReset.ResetRootBoneForRootMotion(RootTransform, ReferencePose.GetRefPoseTransform(0));
		OutAnimationPoseData.LocalTransformsView[0] = RootTransform;
	}

	FRetargetingTools::RetargetPose(
		ReferencePose,
		SkeletonRemapping,
		SkeletonPairs,
		AnimScaleRetargetingPairs,
		AnimRelativeRetargetingPairs,
		OrientAndScaleRetargetingPairs,
		RetargetTransforms,
		OutAnimationPoseData);

	if (bIsMeshSpaceAdditive)
	{
		// When an animation is a mesh-space additive, bones that aren't animated will end up with some non-identity
		// delta relative to the base used to create the additive. This is because the delta is calculated in mesh-space
		// unlike regular additive animations where bones that aren't animated has an identity delta. For rotations,
		// this mesh-space delta will be the parent bone rotation.
		// However, if a bone isn't animated in the sequence but present on the target skeleton, we have no data for it
		// and the output pose will contain an identity delta which isn't what we want. As such, bones missing from
		// the sequence have their rotation set to their parent.

		const TArrayView<const FBoneIndexType> LODBoneIndexToParentLODBoneIndexMap = OutAnimationPoseData.GetLODBoneIndexToParentLODBoneIndexMap();

		// We always skip the root since it has no parent (its delta value is fine as the identity)
		for (int32 LODBoneIndex = 1; LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
		{
			if (!AnimatedCompactRotations[LODBoneIndex])
			{
				// This bone wasn't animated in the sequence, fix it up
				const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
				const FQuat ParentRotation = OutAnimationPoseData.LocalTransformsView[ParentLODBoneIndex].GetRotation();
				OutAnimationPoseData.LocalTransformsView[LODBoneIndex].SetRotation(ParentRotation);
			}
		}
	}
}

} // end namespace UE::UAF
