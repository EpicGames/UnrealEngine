// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/ExecuteProgram.h"

#include "EvaluationVM/EvaluationProgram.h"
#include "EvaluationVM/EvaluationVM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExecuteProgram)

FAnimNextExecuteProgramTask FAnimNextExecuteProgramTask::Make(TSharedPtr<const UE::UAF::FEvaluationProgram> Program)
{
	FAnimNextExecuteProgramTask Task;
	Task.Program = Program;
	return Task;
}

void FAnimNextExecuteProgramTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	if (Program.IsValid())
	{
		Program->Execute(VM);
	}
}
