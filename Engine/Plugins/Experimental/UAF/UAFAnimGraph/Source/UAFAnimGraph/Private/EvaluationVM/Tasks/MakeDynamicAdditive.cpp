// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/MakeDynamicAdditive.h"

#include "EvaluationVM/EvaluationVM.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MakeDynamicAdditive)

FUAFMakeDynamicAdditiveTask FUAFMakeDynamicAdditiveTask::Make(const EAdditiveAnimationType InAdditiveType)
{
	FUAFMakeDynamicAdditiveTask Task;
	Task.AdditiveType = InAdditiveType;
	return Task;
}

void FUAFMakeDynamicAdditiveTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	
	if (VM.GetActiveNamedSet())
	{
		// Pop our top two poses, the top one being the pose to be turned additive and the second one the base
		TUniquePtr<FValueBundle> AdditiveInput;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, AdditiveInput))
		{
			// We have no inputs, nothing to do
			return;
		}

		FValueSpace ValueSpace;
		if (AdditiveType == AAT_RotationOffsetMeshSpace)
		{
			ValueSpace = FValueSpace(EMixedSpaceFlags::MeshRotation, true);
		}
		else
		{
			ValueSpace = FValueSpace(EValueSpaceType::Local, true);
		}
		
		FValueBundleStack Output(VM.GetActiveNamedSet());
		Output.InitWithValueSpace(ValueSpace);
		
		TUniquePtr<FValueBundle> BaseInput;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, BaseInput))
		{
			// If we have no base to generate against, we push an additive ref pose in the correct value space 
			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Output)));
			return;
		}
		
		UE::UAF::Transformers::FMakeAdditiveSpace::Apply(VM.GetTransformerMap(), *BaseInput, *AdditiveInput, Output);
		VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Output)));
	}
	else
	{
		// Pop our top two poses, the top one being the pose to be turned additive and the second one the base
		TUniquePtr<FKeyframeState> AdditiveKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, AdditiveKeyframe))
		{
			// We have no inputs, nothing to do
			return;
		}

		TUniquePtr<FKeyframeState> BaseKeyframe;
		if (!VM.PopValue(KEYFRAME_STACK_NAME, BaseKeyframe))
		{
			// If we have no base to generate against, we push an additive ref pose 
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(true)));
			return;
		}
		
		FKeyframeState OutputKeyframe = VM.MakeUninitializedKeyframe(true);
		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			// Build bone data 
			ConvertToAdditive(AdditiveKeyframe->Pose.LocalTransforms.GetConstView(), BaseKeyframe->Pose.LocalTransforms.GetConstView(), OutputKeyframe.Pose.LocalTransforms.GetView());
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			// Build curve data
			// Convert current curves to Additive (Target - BaseCurve) if overlapping entries are found
			OutputKeyframe.Curves.CopyFrom(AdditiveKeyframe->Curves);
			FBlendedCurve& BaseCurve = BaseKeyframe->Curves;
			FBlendedCurve& AdditiveCurve = OutputKeyframe.Curves;
			UE::Anim::FNamedValueArrayUtils::Union(
				AdditiveCurve, 
				BaseCurve,
				[](UE::Anim::FCurveElement& InOutTargetCurveElement, const UE::Anim::FCurveElement& InBaseCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				InOutTargetCurveElement.Value -= InBaseCurveElement.Value;
				InOutTargetCurveElement.Flags |= InBaseCurveElement.Flags;
			});
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			// Build attribute data 
			OutputKeyframe.Attributes.CopyFrom(AdditiveKeyframe->Attributes);
			UE::Anim::Attributes::ConvertToAdditive(BaseKeyframe->Attributes, OutputKeyframe.Attributes);
		}
		
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(OutputKeyframe)));
	}
}
