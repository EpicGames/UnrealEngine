// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/NormalizeRotations.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NormalizeRotations)

void FAnimNextNormalizeKeyframeRotationsTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		TUniquePtr<FValueBundle> Collection;
		if (!VM.PopValue(ATTRIBUTE_STACK_NAME, Collection))
		{
			// We have no inputs, nothing to do
			return;
		}

		Transformers::FSanitize::Apply(VM.GetTransformerMap(), *Collection);

		VM.PushValue(ATTRIBUTE_STACK_NAME, MoveTemp(Collection));
	}
	else
	{
		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			TUniquePtr<FKeyframeState> Keyframe;
			if (!VM.PopValue(KEYFRAME_STACK_NAME, Keyframe))
			{
				// We have no inputs, nothing to do
				return;
			}

			NormalizeRotations(Keyframe->Pose.LocalTransforms.GetView());

			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
		}
	}
}
