// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/PushAnimSequenceKeyframe.h"

#include "Animation/AnimSequence.h"
#include "Animation/AnimRootMotionProvider.h"
#include "BonePose.h"
#include "EvaluationVM/Tasks/DecompressionTools.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PushAnimSequenceKeyframe)

FAnimNextAnimSequenceKeyframeTask FAnimNextAnimSequenceKeyframeTask::MakeFromSampleTime(TWeakObjectPtr<const UAnimSequence> AnimSequence, double SampleTime, bool bInterpolate)
{
	FAnimNextAnimSequenceKeyframeTask Task;
	Task.AnimSequence = AnimSequence;
	Task.SampleTime = SampleTime;
	Task.bInterpolate = bInterpolate;

	return Task;
}

FAnimNextAnimSequenceKeyframeTask FAnimNextAnimSequenceKeyframeTask::MakeFromKeyframeIndex(TWeakObjectPtr<const UAnimSequence> AnimSequence, uint32 KeyframeIndex)
{
	FAnimNextAnimSequenceKeyframeTask Task;
	Task.AnimSequence = AnimSequence;
	Task.KeyframeIndex = KeyframeIndex;

	return Task;
}

void FAnimNextAnimSequenceKeyframeTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	
	const FEvaluationTaskContext& VMContext = VM.GetActiveEvaluationContext();

	const UAnimSequence* AnimSequencePtr = AnimSequence.Get();
	if (AnimSequencePtr && AnimSequencePtr->GetSkeleton()) // Skip in case the anim sequence has no valid skeleton assigned.
	{
		const bool bExtractRootMotion = bExtractTrajectory;
		const FAnimExtractContext ExtractionContext(SampleTime, bExtractRootMotion, DeltaTimeRecord, bLooping);

		const bool bIsAdditive = AnimSequencePtr->IsValidAdditive();
		const EAdditiveAnimationType AdditiveType = AnimSequencePtr->GetAdditiveAnimType();
		const bool bUseRawData = FDecompressionTools::ShouldUseRawData(AnimSequencePtr, ExtractionContext);

		if (VMContext.IsValid())
		{
			FValueSpace ValueSpace;
			if (bIsAdditive && AdditiveType == AAT_RotationOffsetMeshSpace)
			{
				ValueSpace = FValueSpace(EMixedSpaceFlags::MeshRotation, bIsAdditive);
			}
			else
			{
				ValueSpace = FValueSpace(EValueSpaceType::Local, bIsAdditive);
			}

			FValueBundleStack Collection(VMContext.GetNamedSet());
			Collection.InitWithValueSpace(ValueSpace);

			FDecompressionTools::GetAnimationPose(AnimSequencePtr, ExtractionContext, VMContext, Collection, bUseRawData);
			FDecompressionTools::GetAnimationCurves(AnimSequencePtr, ExtractionContext, VMContext, Collection, bUseRawData);
			FDecompressionTools::GetAnimationAttributes(AnimSequencePtr, ExtractionContext, VMContext, Collection, bUseRawData);

			// If the sequence has root motion enabled, extract it into its own attribute
			if (AnimSequence->HasRootMotion())
			{
				FAttributeNamedSetPtr NamedSet = Collection.GetNamedSet();
				if (FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet<FTransformAnimationAttribute>())
				{
					// AttributeName: RootMotionDelta
					if (const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(UE::Anim::IAnimRootMotionProvider::AttributeName))
					{
						// This attribute is present within the current named set

						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FTransformAnimationAttribute>();
						TBoundValueMap<FTransformAnimationAttribute>* Map = Collection.GetBoundValueMaps().Find<FTransformAnimationAttribute>(MappingKey);
						checkf(Map != nullptr, TEXT("This attribute exists within our named set but isn't present in the output collection"));

						const FTransform RootMotionTransform = AnimSequence->ExtractRootMotion(ExtractionContext);
						(*Map)[SetIndex].Value = RootMotionTransform;
					}
				}
			}

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
		}
		else
		{
			FKeyframeState Keyframe = VM.MakeUninitializedKeyframe(bIsAdditive);

			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
			{
				FDecompressionTools::GetAnimationPose(AnimSequencePtr, ExtractionContext, Keyframe.Pose, bUseRawData);
			}

			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
			{
				FDecompressionTools::GetAnimationCurves(AnimSequencePtr, ExtractionContext, Keyframe.Curves, bUseRawData);
			}

			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
			{
				FDecompressionTools::GetAnimationAttributes(AnimSequencePtr, ExtractionContext, Keyframe.Pose.GetRefPose(), Keyframe.Attributes, bUseRawData);
			}

			// Trajectory is currently held as an attribute
			if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes | EEvaluationFlags::Trajectory))
			{
				// If the sequence has root motion enabled, allow sampling of a root motion delta into the custom attribute container of the outgoing pose
				if (AnimSequence->HasRootMotion())
				{
					// TODO: We should cache the provider in the VM
					// We have to grab two locks to get it and it won't change during graph evaluation
					if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
					{
						RootMotionProvider->SampleRootMotion(ExtractionContext.DeltaTimeRecord, *AnimSequence, ExtractionContext.bLooping, Keyframe.Attributes);
					}
				}
			}

			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));
		}
	}
	else
	{
		if (VMContext.IsValid())
		{
			FValueBundleStack Collection(VMContext.GetNamedSet());
			Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

			VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
		}
		else
		{
			constexpr bool bIsAdditive = false;
			FKeyframeState Keyframe = VM.MakeReferenceKeyframe(bIsAdditive);
			VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));
		}
	}
}
