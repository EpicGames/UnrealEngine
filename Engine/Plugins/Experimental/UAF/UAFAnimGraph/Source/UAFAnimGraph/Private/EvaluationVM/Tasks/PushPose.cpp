// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/PushPose.h"
#include "EvaluationVM/EvaluationVM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PushPose)

DEFINE_STAT(STAT_AnimNext_Task_PushPose);

FAnimNextPushPoseTask::FAnimNextPushPoseTask(const FAnimNextGraphLODPose* InGraphPose)
	: GraphPose(InGraphPose)
{
}

FAnimNextPushPoseTask FAnimNextPushPoseTask::Make(const FAnimNextGraphLODPose* InGraphPose)
{
	return FAnimNextPushPoseTask(InGraphPose);
}

FAnimNextPushPoseTask& FAnimNextPushPoseTask::operator=(const FAnimNextPushPoseTask& Other)
{
	GraphPose = Other.GraphPose;
	return *this;
}

void FAnimNextPushPoseTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Task_PushPose);

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		FValueBundleStack Collection(VM.GetActiveNamedSet());
		Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

		VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
		return;
	}

	FKeyframeState Keyframe = VM.MakeUninitializedKeyframe(GraphPose->LODPose.IsAdditive());
	// CopyFrom transparently remaps if the source skeleton differs from the target (cross-skeleton input).
	Keyframe.CopyFrom(*GraphPose);
	VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));
}
