// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/PushReferenceKeyframe.h"

#include "EvaluationVM/EvaluationVM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PushReferenceKeyframe)

FAnimNextPushReferenceKeyframeTask FAnimNextPushReferenceKeyframeTask::MakeFromSkeleton()
{
	return FAnimNextPushReferenceKeyframeTask();
}

FAnimNextPushReferenceKeyframeTask FAnimNextPushReferenceKeyframeTask::MakeFromAdditiveIdentity()
{
	FAnimNextPushReferenceKeyframeTask Task;
	Task.bIsAdditive = true;

	return Task;
}

void FAnimNextPushReferenceKeyframeTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		FValueBundleStack Collection(VM.GetActiveNamedSet());
		Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local, bIsAdditive));

		VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
	}
	else
	{
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(bIsAdditive)));
	}
}
