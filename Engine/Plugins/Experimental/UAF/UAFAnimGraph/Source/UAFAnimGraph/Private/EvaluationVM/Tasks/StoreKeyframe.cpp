// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StoreKeyframe)

FAnimNextSwapTransformsTask FAnimNextSwapTransformsTask::Make(UE::UAF::FTransformArraySoAHeap* A, UE::UAF::FTransformArraySoAHeap* B)
{
	FAnimNextSwapTransformsTask Task;
	Task.A = A;
	Task.B = B;
	return Task;
}

void FAnimNextSwapTransformsTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	Swap(*A, *B);
}

FAnimNextStoreKeyframeTransformsTask FAnimNextStoreKeyframeTransformsTask::Make(UE::UAF::FTransformArraySoAHeap* Dest)
{
	FAnimNextStoreKeyframeTransformsTask Task;
	Task.Dest = Dest;
	return Task;
}

void FAnimNextStoreKeyframeTransformsTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
		{
			Dest->SetNumUninitialized((*Keyframe)->Pose.LocalTransforms.Num());
			CopyTransforms(Dest->GetView(), (*Keyframe)->Pose.LocalTransforms.GetConstView());
		}
	}
}

// --- FAnimNextDuplicateTopKeyframeTask  ---

FAnimNextDuplicateTopKeyframeTask FAnimNextDuplicateTopKeyframeTask::Make()
{
	FAnimNextDuplicateTopKeyframeTask Task;
	return Task;
}

// Task entry point
void FAnimNextDuplicateTopKeyframeTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		FValueBundleStack Collection(VM.GetActiveNamedSet());
		Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

		VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
		return;
	}

	TUniquePtr<FKeyframeState> KeyframeOut;

	// Try to get a Keyframe from the stack without consuming it
	// If there is a Keyframe, create a copy
	if (const TUniquePtr<FKeyframeState>* KeyframeIn = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
	{
		KeyframeOut = MakeUnique<FKeyframeState>(VM.MakeUninitializedKeyframe(false));
		KeyframeOut->Pose.CopyFrom((*KeyframeIn)->Pose);
		KeyframeOut->Curves.CopyFrom((*KeyframeIn)->Curves);
		KeyframeOut->Attributes.CopyFrom((*KeyframeIn)->Attributes);
	}
	else
	{
		// If there is no Keyframe, generate a reference key frame
		KeyframeOut = MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false));
		UE_LOGF(LogAnimation, Log, "FAnimNextDuplicateTopKeyframeTask::Execute: There was no valid keyframe to duplicate. Pushing ref pose keyframe instead.");
	}

	// Push the copy or reference keyframe to the stack
	VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeOut));
}
