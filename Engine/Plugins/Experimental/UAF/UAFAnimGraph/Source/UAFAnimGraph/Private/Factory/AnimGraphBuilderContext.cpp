// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/AnimGraphBuilderContext.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "TraitCore/TraitWriter.h"
#include "UAFAssetInstanceComponent.h"

namespace UE::UAF
{

const UUAFAnimGraph* FAnimGraphBuilderContext::Build()
{
	if(!ensure(TraitWriter.GetGraphSharedData().Num() > 0))
	{
		return nullptr;
	}

	if (!ensure(RootTraitHandle.IsValid()))
	{
		return nullptr;
	}

	{
		FGCScopeGuard GCSCope;

		// Create our anim graph
		UUAFAnimGraph* AnimationGraph = NewObject<UUAFAnimGraph>(GetTransientPackage(), NAME_None, RF_Transient);

		FAnimNextGraphEntryPoint& EntryPoint = AnimationGraph->EntryPoints.AddDefaulted_GetRef();
		EntryPoint.EntryPointName = AnimationGraph->DefaultEntryPoint;
		EntryPoint.RootTraitHandle = RootTraitHandle;
		AnimationGraph->ReferencedVariableStructs = VariableStructs;
		AnimationGraph->GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
		AnimationGraph->GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		ensure(AnimationGraph->LoadFromArchiveBuffer(TraitWriter.GetGraphSharedData()));

		AnimationGraph->Components.Reserve(ComponentStructs.Num());
		for (const UScriptStruct* ComponentStruct : ComponentStructs)
		{
			TInstancedStruct<FUAFAssetInstanceComponent> ComponentInstance;
			ComponentInstance.InitializeAsScriptStruct(ComponentStruct);
			AnimationGraph->Components.Add(MoveTemp(ComponentInstance));
		}

		AnimationGraph->ReferencedVariableRigVMAssets = ReferencedVariableRigVMAssets.Array();
		AnimationGraph->ReferencedVariableAssets = ReferencedVariableAssets.Array();

		// Clear async flag to make the object visible to GC
		(void)AnimationGraph->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);

		return AnimationGraph;
	}

}

}