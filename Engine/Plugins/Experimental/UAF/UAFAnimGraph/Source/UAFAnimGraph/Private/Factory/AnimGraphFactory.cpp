// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/AnimGraphFactory.h"

#include "Asset/UAFAnimGraphAssetData.h"
#include "Factory/AnimGraphBuilderContext.h"
#include "Containers/StripedMap.h"
#include "Factory/AnimNextAnimGraphBuilder.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Misc/CoreDelegates.h"
#include "TraitCore/TraitWriter.h"

namespace UE::UAF::Private
{

static FDelegateHandle GAnimGraphFactoryPreExitHandle;

// References to all generated factory graphs, keyed by method
static TStripedMap<32, uint64, TWeakObjectPtr<const UUAFAnimGraph>> GAnimationGraphMap;
}

namespace UE::UAF
{
uint64 IAnimGraphBuilder::GetKey() const
{
	return 0;
}

TMap<FTopLevelAssetPath, TPair<FAnimNextFactoryParams, FAnimGraphFactory::FParamsInitializer>> FAnimGraphFactory::DefaultParamsForObject;
// All registered object -> param mappings
TArray<TSoftObjectPtr<UScriptStruct>> FAnimGraphFactory::AllFactoryStructs;

const TPair<FAnimNextFactoryParams, FAnimGraphFactory::FParamsInitializer>* FAnimGraphFactory::FindGraphParamsForAssetData(FGraphAssetHandleConstView AssetData)
{
	const UStruct* SearchStruct = AssetData.GetScriptStruct();
	TPair<FAnimNextFactoryParams, FAnimGraphFactory::FParamsInitializer>* FoundItem = nullptr;
	while (SearchStruct != nullptr && FoundItem == nullptr)
	{
		FoundItem = DefaultParamsForObject.Find(SearchStruct->GetStructPathName());
		SearchStruct = SearchStruct->GetSuperStruct();
	}
	return FoundItem;
}

const UUAFAnimGraph* FAnimGraphFactory::GetDefaultGraphHost()
{
	const TPair<FAnimNextFactoryParams, FParamsInitializer>* ParamsTask = FindGraphParamsForAssetData(FUAFGraphFactoryAsset_Graph() );
	return BuildGraph(ParamsTask->Key.GetBuilder());
}

const UUAFAnimGraph* FAnimGraphFactory::GetOrBuildGraph(UE::UAF::FGraphAssetHandleConstView AssetHandle, const FAnimNextFactoryParams& InParams)
{
	if (AssetHandle.IsValid() == false)
	{
		return nullptr;
	}
	// If a graph was provided directly, just use it
	if (const FUAFGraphFactoryAsset_Graph* AnimationGraphAssetData = AssetHandle.GetPtr<FUAFGraphFactoryAsset_Graph>())
	{
		return AnimationGraphAssetData->AnimationGraph;
	}

	FAnimNextFactoryParams Params = InParams.IsValid() ? InParams : GetDefaultParamsForAsset(AssetHandle);

	// Apply default initializer
	InitializeDefaultParamsForAsset(AssetHandle, Params);

	return BuildGraph(Params.GetBuilder());
}

const UUAFAnimGraph* FAnimGraphFactory::BuildGraph(const IAnimGraphBuilder& InBuilder)
{
	uint64 Key = InBuilder.GetKey();
	if (Key == 0)
	{
		return nullptr;
	}

	auto ProduceGraph = [&InBuilder]() -> TWeakObjectPtr<const UUAFAnimGraph>
	{
		FAnimGraphBuilderContext Context;
		if (InBuilder.Build(Context))
		{
			return Context.Build();
		}
		return nullptr;
	};

	TWeakObjectPtr<const UUAFAnimGraph> AnimationGraph;
	auto ApplyWeakGraph = [&ProduceGraph, &AnimationGraph](TWeakObjectPtr<const UUAFAnimGraph>& InFoundAnimationGraph)
	{
		if (InFoundAnimationGraph.IsStale())
		{
			// Found a stale weak ptr, so regenerate (this may be a procedural graph or a previously loaded graph that has been GCed)
			InFoundAnimationGraph = ProduceGraph();
		}
		AnimationGraph = InFoundAnimationGraph;
	};

	Private::GAnimationGraphMap.FindOrProduceAndApplyForWrite(Key, ProduceGraph, ApplyWeakGraph);
	return AnimationGraph.Get();
}

FAnimNextFactoryParams FAnimGraphFactory::GetDefaultParamsForAsset(FGraphAssetHandleConstView Asset)
{
	check(Asset.IsValid());
	const TPair<FAnimNextFactoryParams, FParamsInitializer>* ParamsTask = FindGraphParamsForAssetData(Asset);
	check(ParamsTask != nullptr);	// Unregistered object type

	FAnimNextFactoryParams ParamsCopy = ParamsTask->Key;
	if (ParamsCopy.Builder.Stacks.Num() > 0)
	{
		// Temp array used for init callback
		TArray<TArrayView<FAnimNextSimpleAnimGraphBuilderTraitDesc>, TInlineAllocator<8>> TraitDescs;
		TraitDescs.Reserve(ParamsCopy.Builder.Stacks.Num());
		for (FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc : ParamsCopy.Builder.Stacks)
		{
			TraitDescs.Add(StackDesc.TraitDescs);
		}

		if (ParamsTask->Value)
		{
			ParamsTask->Value(Asset, ParamsCopy);
		}
	}
	return ParamsCopy;
}

void FAnimGraphFactory::InitializeDefaultParamsForAsset(FGraphAssetHandleConstView Asset, FAnimNextFactoryParams& InOutParams)
{
	check(Asset.IsValid());
	const TPair<FAnimNextFactoryParams, FParamsInitializer>* ParamsTask = FindGraphParamsForAssetData(Asset);
	check(ParamsTask != nullptr);	// Unregistered object type

	if (InOutParams.Builder.Stacks.Num() > 0)
	{
		// Temp array used for init callback
		TArray<TArrayView<FAnimNextSimpleAnimGraphBuilderTraitDesc>, TInlineAllocator<8>> TraitDescs;
		TraitDescs.Reserve(InOutParams.Builder.Stacks.Num());
		for (FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc : InOutParams.Builder.Stacks)
		{
			TraitDescs.Add(StackDesc.TraitDescs);
		}
		
		if (ParamsTask->Value)
		{
			ParamsTask->Value(Asset, InOutParams);
		}
	}
}

void FAnimGraphFactory::RegisterAsset(const TSubScriptStructOf<FUAFGraphFactoryAsset>& ScriptStruct, FAnimNextFactoryParams&& InParams, FParamsInitializer&& InInitializer)
{
	checkf(AllFactoryStructs.Contains(ScriptStruct.Get()) == false, TEXT("Attempting to register ScriptStruct %s twice"), *ScriptStruct->GetStructPathName().ToString());

	DefaultParamsForObject.Add(ScriptStruct->GetStructPathName(), { MoveTemp(InParams), MoveTemp(InInitializer) });
	AllFactoryStructs.Add(TSoftObjectPtr<UScriptStruct>(ScriptStruct.Get()));
}

void FAnimGraphFactory::Init()
{
	Private::GAnimGraphFactoryPreExitHandle = FCoreDelegates::OnEnginePreExit.AddStatic(&FAnimGraphFactory::OnPreExit);
}

void FAnimGraphFactory::Destroy()
{
	FCoreDelegates::OnEnginePreExit.Remove(Private::GAnimGraphFactoryPreExitHandle);
	Private::GAnimGraphFactoryPreExitHandle.Reset();
}

void FAnimGraphFactory::OnPreExit()
{
	Private::GAnimationGraphMap.Empty();
	DefaultParamsForObject.Empty();
	AllFactoryStructs.Empty();
}

}
