// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/AnimNextAnimGraphBuilder.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "TraitCore/TraitWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraphBuilder)

uint64 FAnimNextAnimGraphBuilder::GetKey() const
{
	if (CachedKey == 0)
	{
		CachedKey = RecalculateKey();
	}
	return CachedKey;
}

