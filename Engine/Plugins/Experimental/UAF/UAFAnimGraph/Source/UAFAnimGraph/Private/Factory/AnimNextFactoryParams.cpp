// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/AnimNextFactoryParams.h"
#include "Graph/AnimNextGraphInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextFactoryParams)

void FAnimNextFactoryParams::InitializeInstance(FUAFAssetInstance& InInstance) const
{
	for (const FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc : Builder.Stacks)
	{
		for (const FAnimNextSimpleAnimGraphBuilderTraitDesc& InitializePayload : StackDesc.TraitDescs)
		{
			InInstance.AccessVariablesStruct(InitializePayload.TraitData.GetScriptStruct(), [&InitializePayload](FStructView InStructView)
			{
				check(InitializePayload.TraitData.GetScriptStruct() == InStructView.GetScriptStruct());
				InStructView.GetScriptStruct()->CopyScriptStruct(InStructView.GetMemory(), InitializePayload.TraitData.GetMemory());
			});
		}
	}

	const UE::UAF::FInstanceTaskContext TaskContext(InInstance);
	for (const UE::UAF::FInstanceTask& InitializeTask : InitializeTasks)
	{
		InitializeTask(TaskContext);
	}
}

void FAnimNextFactoryParams::Reset()
{
	Builder.Reset();
	InitializeTasks.Empty();
}
