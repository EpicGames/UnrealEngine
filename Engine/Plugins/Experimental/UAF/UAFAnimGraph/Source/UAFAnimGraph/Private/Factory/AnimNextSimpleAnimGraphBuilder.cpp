// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/AnimNextSimpleAnimGraphBuilder.h"

#include "Factory/AnimGraphBuilderContext.h"
#include "Misc/HashBuilder.h"
#include "TraitCore/NodeHandle.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/TraitWriter.h"
#include "Traits/ReferencePoseTrait.h"
#include "Hash/xxhash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSimpleAnimGraphBuilder)

bool FAnimNextSimpleAnimGraphBuilder::Build(UE::UAF::FAnimGraphBuilderContext& InContext) const
{
	using namespace UE::UAF;

	if (!Validate())
	{
		return false;
	}

	InContext.ComponentStructs = ComponentStructs;
	InContext.VariableStructs = VariableStructs;
	InContext.ReferencedVariableAssets.Append(ReferencedVariableAssets);
	InContext.ReferencedVariableRigVMAssets.Append(ReferencedVariableRigVMAssets);

	// Index variables for mapping lookup
	TMap<FAnimNextVariableReference, int32, TInlineSetAllocator<128>> VariableIndexMap;
	VariableIndexMap.Reserve(VariableStructs.Num() * 4);
	int32 VariableIndex = 0;
	for (const UScriptStruct* VariableStruct : VariableStructs)
	{
		for (TFieldIterator<FProperty> It(VariableStruct); It; ++It)
		{
			VariableIndexMap.Add(FAnimNextVariableReference::FromName(It->GetFName(), VariableStruct), VariableIndex++);
		}
	}

	struct FTraitStackData
	{
		FNodeHandle NodeHandle;
		TArray<TInstancedStruct<FAnimNextTraitSharedData>, TInlineAllocator<4>> TraitStructs;
		TArray<FTraitUID, TInlineAllocator<4>> TraitUIDs;
		TMap<TTuple<uint32, uint32, FName>, int32, TInlineSetAllocator<64>> VariableIndexMap;
	};

	TArray<FTraitStackData, TInlineAllocator<4>> TraitStacks;
	TraitStacks.Reserve(Stacks.Num());
	
	// Setup trait stacks & variable mappings
	const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

	const int32 NumTraitStacks = Stacks.Num();
	for (int32 TraitStackIndex = 0; TraitStackIndex < NumTraitStacks; ++TraitStackIndex)
	{
		int32 TraitIndex = 0;
		const FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc = Stacks[TraitStackIndex];

		FTraitStackData TraitStackData;
		TraitStackData.TraitStructs.Reserve(StackDesc.TraitDescs.Num());
		TraitStackData.TraitUIDs.Reserve(StackDesc.TraitDescs.Num());

		for (const FAnimNextSimpleAnimGraphBuilderTraitDesc& TraitDesc : StackDesc.TraitDescs)
		{
			TraitStackData.TraitStructs.Add(TraitDesc.TraitData);
			const FTrait* Trait = TraitRegistry.Find(TraitDesc.TraitData.GetScriptStruct());
			TraitStackData.TraitUIDs.Add(Trait->GetTraitUID());

			for (TFieldIterator<FProperty> It(TraitDesc.TraitData.GetScriptStruct()); It; ++It)
			{
				FName PropertyName = It->GetFName();

				// See if this property is mapped
				for (const FAnimNextSimpleAnimGraphBuilderVariableMapping& VariableMapping : TraitDesc.VariableMappings)
				{
					if (PropertyName == VariableMapping.PropertyName)
					{
						int32* VariableIndexPtr = VariableIndexMap.Find(VariableMapping.VariableReference);
						if (ensure(VariableIndexPtr != nullptr))
						{
							TraitStackData.VariableIndexMap.Add({TraitIndex, TraitStackIndex, It->GetFName()}, *VariableIndexPtr);
						}
						break;
					}
				}
			}

			TraitIndex++;
		}
		TraitStacks.Add(MoveTemp(TraitStackData));
	}

	// Register nodes
	auto RegisterNode = [&InContext](FTraitStackData& InTraitStackData)
	{
		TArray<uint8> NodeTemplateBuffer;
		const FNodeTemplate* NodeTemplate = FNodeTemplateBuilder::BuildNodeTemplate(InTraitStackData.TraitUIDs, NodeTemplateBuffer);
		InTraitStackData.NodeHandle = InContext.TraitWriter.RegisterNode(*NodeTemplate);
	};
	
	for (int32 TraitStackIndex = 0; TraitStackIndex < NumTraitStacks; ++TraitStackIndex)
	{
		FTraitStackData& TraitStackData = TraitStacks[TraitStackIndex];
		RegisterNode(TraitStackData);

		// First trait is always the root
		if (TraitStackIndex == 0)
		{
			InContext.RootTraitHandle = FAnimNextEntryPointHandle(TraitStackData.NodeHandle);
		}
	}

	// Link up stacks
	for (int32 TraitStackIndex = 0; TraitStackIndex < NumTraitStacks; ++TraitStackIndex)
	{
		const FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc = Stacks[TraitStackIndex];
		FTraitStackData& StackData = TraitStacks[TraitStackIndex];
		if (TraitStacks.IsValidIndex(StackDesc.Link.StackIndex))
		{
			FTraitStackData& TargetStackData = TraitStacks[StackDesc.Link.StackIndex];
			if (TargetStackData.TraitStructs.IsValidIndex(StackDesc.Link.TraitIndex) && !StackDesc.Link.ChildName.IsNone())
			{
				TInstancedStruct<FAnimNextTraitSharedData>& TargetTraitStruct = TargetStackData.TraitStructs[StackDesc.Link.TraitIndex];
				for (TFieldIterator<FProperty> It(TargetTraitStruct.GetScriptStruct()); It; ++It)
				{
					FStructProperty* StructProperty = CastField<FStructProperty>(*It);
					if (StructProperty && StructProperty->GetFName() == StackDesc.Link.ChildName && StructProperty->Struct == FAnimNextTraitHandle::StaticStruct())
					{
						FAnimNextTraitHandle* Handle = It->ContainerPtrToValuePtr<FAnimNextTraitHandle>(TargetTraitStruct.GetMutableMemory());
						*Handle = FAnimNextTraitHandle(StackData.NodeHandle);
					}
				}
			}
		}
	}

	for (int32 TraitStackIndex = 0; TraitStackIndex < NumTraitStacks; ++TraitStackIndex)
	{
		FTraitStackData& TraitStackData = TraitStacks[TraitStackIndex];

		// Add new trait stack with refpose trait to cover 'unlinked' child pins
		const FTrait* Trait = TraitRegistry.Find(FAnimNextReferencePoseTraitSharedData::StaticStruct());
		const FTraitUID RefPoseUID = Trait->GetTraitUID();
		for (TInstancedStruct<FAnimNextTraitSharedData>& TraitStruct : TraitStackData.TraitStructs)
		{
			for (TFieldIterator<FProperty> It(TraitStruct.GetScriptStruct()); It; ++It)
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(*It);
				if (StructProperty && StructProperty->Struct == FAnimNextTraitHandle::StaticStruct())
				{
					FAnimNextTraitHandle* Handle = It->ContainerPtrToValuePtr<FAnimNextTraitHandle>(TraitStruct.GetMutableMemory());
					if (!Handle->GetNodeHandle().IsValid())
					{
						FTraitStackData RefPoseTraitStackData;
						TInstancedStruct<FAnimNextTraitSharedData> InstancedStruct;
						InstancedStruct.InitializeAsScriptStruct(FAnimNextReferencePoseTraitSharedData::StaticStruct());

						RefPoseTraitStackData.TraitStructs.Add(MoveTemp(InstancedStruct));
						RefPoseTraitStackData.TraitUIDs.Add(RefPoseUID);

						RegisterNode(RefPoseTraitStackData);

						// Link to this new stack
						*Handle = FAnimNextTraitHandle(RefPoseTraitStackData.NodeHandle);

						// Mutating TraitStacks while iterating them here, but we dont want to iterate into the refpose stacks we are adding here so its OK
						TraitStacks.Add(RefPoseTraitStackData);
					}
				}
			}
		}
	}

	// Write traits
	InContext.TraitWriter.BeginNodeWriting();
	for (int32 TraitStackIndex = 0; TraitStackIndex < TraitStacks.Num(); ++TraitStackIndex)
	{
		const FTraitStackData& TraitStackData = TraitStacks[TraitStackIndex];
		InContext.TraitWriter.WriteNode(TraitStackData.NodeHandle,
			[&TraitStackData, TraitStackIndex](uint32 InTraitIndex, FName InPropertyName) -> uint16
			{
				if (const int32* IndexPtr = TraitStackData.VariableIndexMap.Find({InTraitIndex, TraitStackIndex, InPropertyName}))
				{
					return (uint16)*IndexPtr;
				}
				return MAX_uint16;
			},
			[&TraitStackData](uint32 InTraitIndex)
			{
				return TConstStructView<FAnimNextTraitSharedData>(TraitStackData.TraitStructs[InTraitIndex]);
			});
	}
	InContext.TraitWriter.EndNodeWriting();

	ensure(InContext.TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None);

	return true;
}

bool FAnimNextSimpleAnimGraphBuilder::ValidateTraitStruct(int32 InStackIndex, int32 InTraitIndex, TConstStructView<FAnimNextTraitSharedData> InStruct) const
{
	using namespace UE::UAF;

	const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
	const FTrait* Trait = TraitRegistry.Find(InStruct.GetScriptStruct());
	if (Trait == nullptr)
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Could not find registered trait"));
		return false;
	}

	if (InTraitIndex == 0 && Trait->GetTraitMode() != ETraitMode::Base)
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: First trait should always be a base trait"));
		return false;
	}

	return true;
}

bool FAnimNextSimpleAnimGraphBuilder::ValidateLink(int32 InOutputStackIndex, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputChildName) const
{
	using namespace UE::UAF;

	if (!Stacks.IsValidIndex(InOutputStackIndex))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Output stack index is invalid"));
		return false;
	}

	if (!Stacks.IsValidIndex(InInputStackIndex))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Input stack index is invalid"));
		return false;
	}

	if (!Stacks[InInputStackIndex].TraitDescs.IsValidIndex(InInputTraitIndex))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Input trait index is invalid"));
		return false;
	}

	const UScriptStruct* TraitStruct = Stacks[InInputStackIndex].TraitDescs[InInputTraitIndex].TraitData.GetScriptStruct();
	const FStructProperty* ChildProperty = CastField<FStructProperty>(TraitStruct->FindPropertyByName(InInputChildName));
	if (ChildProperty == nullptr || ChildProperty->Struct != FAnimNextTraitHandle::StaticStruct())
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Input child name is invalid"));
		return false;
	}

	return true;
}

bool FAnimNextSimpleAnimGraphBuilder::ValidateVariableMapping(FAnimNextVariableReference InVariable, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputPropertyName) const
{
	if (!InVariable.IsValid() || InVariable.GetObject() == nullptr || !InVariable.GetObject()->IsA<UScriptStruct>())
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Variable reference is invalid"));
		return false;
	}

	if (!VariableStructs.Contains(CastChecked<UScriptStruct>(InVariable.GetObject())))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: No variable struct found for variable reference"));
		return false;
	}
	
	if (!Stacks.IsValidIndex(InInputStackIndex))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Input stack index is invalid"));
		return false;
	}

	if (!Stacks[InInputStackIndex].TraitDescs.IsValidIndex(InInputTraitIndex))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Input trait index is invalid"));
		return false;
	}

	const UScriptStruct* TraitStruct = Stacks[InInputStackIndex].TraitDescs[InInputTraitIndex].TraitData.GetScriptStruct();
	const FProperty* ChildProperty = TraitStruct->FindPropertyByName(InInputPropertyName);
	if (ChildProperty == nullptr)
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Input property name is invalid"));
		return false;
	}

	const FProperty* VariableProperty = InVariable.ResolveProperty();
	if (VariableProperty == nullptr)
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Variable property is invalid"));
		return false;
	}

	if (VariableProperty->GetClass() != ChildProperty->GetClass())
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Variable mapping property class does not match"));
		return false;
	}

	return true;
}

bool FAnimNextSimpleAnimGraphBuilder::Validate() const
{
	if (Stacks.Num() == 0)
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: No stacks found"));
		return false;
	}

	for(int32 StackIndex = 0; StackIndex < Stacks.Num(); ++StackIndex)
	{
		const FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc = Stacks[StackIndex];
		const int32 NumTraits = StackDesc.TraitDescs.Num();
		if (NumTraits == 0)
		{
			ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Empty stack found"));
			return false;
		}

		for (int32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FAnimNextSimpleAnimGraphBuilderTraitDesc& TraitDesc = StackDesc.TraitDescs[TraitIndex];
			if (!ValidateTraitStruct(StackIndex, TraitIndex, TraitDesc.TraitData))
			{
				return false;
			}

			if (StackDesc.Link.StackIndex != INDEX_NONE)
			{
				if (!ValidateLink(StackIndex, StackDesc.Link.StackIndex, StackDesc.Link.TraitIndex, StackDesc.Link.ChildName))
				{
					return false;
				}
			}
			
			for(const FAnimNextSimpleAnimGraphBuilderVariableMapping& VariableMapping : TraitDesc.VariableMappings)
			{
				if (!ValidateVariableMapping(VariableMapping.VariableReference, StackIndex, TraitIndex, VariableMapping.PropertyName))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void FAnimNextSimpleAnimGraphBuilder::AutoMapTraitHelper(int32 InStackIndex, const UScriptStruct* TraitStruct)
{
	if (!VariableStructs.Contains(TraitStruct))
	{
		VariableStructs.Add(TraitStruct);
	}

	const int32 TraitIndex = Stacks[InStackIndex].TraitDescs.IndexOfByPredicate([TraitStruct](const FAnimNextSimpleAnimGraphBuilderTraitDesc& InDesc)
	{
		return InDesc.TraitData.GetScriptStruct() == TraitStruct;
	});
	
	if (ensure(TraitIndex != INDEX_NONE))
	{
		for (TFieldIterator<FProperty> It(TraitStruct); It; ++It)
		{
			// Dont auto-map child trait handles
			if (FStructProperty* StructProperty = CastField<FStructProperty>(*It))
			{
				if (StructProperty->Struct == FAnimNextTraitHandle::StaticStruct())
				{
					continue;
				}
			}

			const FName PropertyName = It->GetFName();
			const FAnimNextVariableReference VariableReference = FAnimNextVariableReference::FromName(PropertyName, TraitStruct);
			AddVariableMapping(VariableReference, InStackIndex, TraitIndex, PropertyName);
		}
	}
}

void FAnimNextSimpleAnimGraphBuilder::AddTraitStructView(int32 InStackIndex, UE::UAF::ETraitVariableMapping InMapping, TConstStructView<FAnimNextTraitSharedData> InStructView)
{
	using namespace UE::UAF;

	if (Stacks.Num() < InStackIndex + 1)
	{
		Stacks.SetNum(InStackIndex + 1);
	}

	if (!ValidateTraitStruct(InStackIndex, Stacks[InStackIndex].TraitDescs.Num(), InStructView))
	{
		return;
	}

	const UScriptStruct* TraitStruct = InStructView.GetScriptStruct();

	auto FindExistingTrait = [&InStructView](const FAnimNextSimpleAnimGraphBuilderTraitDesc& InExistingDesc)
	{
		return InStructView.GetScriptStruct() == InExistingDesc.TraitData.GetScriptStruct();
	};

	if (FAnimNextSimpleAnimGraphBuilderTraitDesc* ExistingDesc = Stacks[InStackIndex].TraitDescs.FindByPredicate(FindExistingTrait))
	{
		ExistingDesc->TraitData = InStructView;
	}
	else
	{
		FAnimNextSimpleAnimGraphBuilderTraitDesc Desc;
		Desc.TraitData = InStructView;
		Stacks[InStackIndex].TraitDescs.Emplace(MoveTemp(Desc));
	}

	if (InMapping == ETraitVariableMapping::All)
	{
		AutoMapTraitHelper(InStackIndex, TraitStruct);
	}

	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddTraitInstancedStruct(int32 InStackIndex, UE::UAF::ETraitVariableMapping InMapping, TInstancedStruct<FAnimNextTraitSharedData>&& InStruct)
{
	using namespace UE::UAF;

	if (Stacks.Num() < InStackIndex + 1)
	{
		Stacks.SetNum(InStackIndex + 1);
	}

	if (!ValidateTraitStruct(InStackIndex, Stacks[InStackIndex].TraitDescs.Num(), InStruct))
	{
		return;
	}

	const UScriptStruct* TraitStruct = InStruct.GetScriptStruct();

	auto FindExistingTrait = [&InStruct](const FAnimNextSimpleAnimGraphBuilderTraitDesc& InExistingDesc)
	{
		return InStruct.GetScriptStruct() == InExistingDesc.TraitData.GetScriptStruct();
	};

	if (FAnimNextSimpleAnimGraphBuilderTraitDesc* ExistingDesc = Stacks[InStackIndex].TraitDescs.FindByPredicate(FindExistingTrait))
	{
		ExistingDesc->TraitData = MoveTemp(InStruct);
	}
	else
	{
		FAnimNextSimpleAnimGraphBuilderTraitDesc Desc;
		Desc.TraitData = MoveTemp(InStruct);
		Stacks[InStackIndex].TraitDescs.Emplace(MoveTemp(Desc));
	}

	if (InMapping == ETraitVariableMapping::All)
	{
		AutoMapTraitHelper(InStackIndex, TraitStruct);
	}

	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddLink(int32 InOutputStackIndex, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputChildName)
{
	if (ValidateLink(InOutputStackIndex, InInputStackIndex, InInputTraitIndex, InInputChildName))
	{
		FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc = Stacks[InOutputStackIndex];
		StackDesc.Link.StackIndex = InInputStackIndex;
		StackDesc.Link.TraitIndex = InInputTraitIndex;
		StackDesc.Link.ChildName = InInputChildName;
	}
	
	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddVariableMapping(FAnimNextVariableReference InVariable, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputPropertyName)
{
	if (ValidateVariableMapping(InVariable, InInputStackIndex, InInputTraitIndex, InInputPropertyName))
	{
		FAnimNextSimpleAnimGraphBuilderVariableMapping Mapping;
		Mapping.VariableReference = InVariable;
		Mapping.PropertyName = InInputPropertyName;
		Stacks[InInputStackIndex].TraitDescs[InInputTraitIndex].VariableMappings.Add(Mapping);
	}
	
	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddVariableMappingToAll(FAnimNextVariableReference InVariable, FName InInputPropertyName)
{
	if (!InVariable.IsValid() || InVariable.GetObject() == nullptr || !InVariable.GetObject()->IsA<UScriptStruct>())
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: Variable reference is invalid"));
		return;
	}

	if (!VariableStructs.Contains(CastChecked<UScriptStruct>(InVariable.GetObject())))
	{
		ensureMsgf(false, TEXT("FAnimNextSimpleAnimGraphBuilder: No variable struct found for variable reference"));
		return;
	}

	for (FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc : Stacks)
	{
		for(FAnimNextSimpleAnimGraphBuilderTraitDesc& TraitDesc : StackDesc.TraitDescs)
		{
			for (TFieldIterator<FProperty> It(TraitDesc.TraitData.GetScriptStruct()); It; ++It)
			{
				const FName PropertyName = It->GetFName();
				if (PropertyName == InInputPropertyName)
				{
					FAnimNextSimpleAnimGraphBuilderVariableMapping Mapping;
					Mapping.VariableReference = InVariable;
					Mapping.PropertyName = InInputPropertyName;
					TraitDesc.VariableMappings.Add(Mapping);

					InvalidateKey();
				}
			}
		}
	}
}

void FAnimNextSimpleAnimGraphBuilder::AddVariableStruct(const UScriptStruct* InStruct)
{
	VariableStructs.Add(InStruct);
	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddVariablesRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAssetInterface)
{
	ReferencedVariableRigVMAssets.Add(RigVMAssetInterface);
	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddVariablesAsset(const TObjectPtr<const UUAFRigVMAsset>& UAFAsset)
{
	ReferencedVariableAssets.Add(UAFAsset);
	InvalidateKey();
}

void FAnimNextSimpleAnimGraphBuilder::AddComponentStruct(const UScriptStruct* InStruct)
{
	using namespace UE::UAF;

	auto FindExistingStruct = [&InStruct](const UScriptStruct* InExistingStruct)
	{
		return InStruct == InExistingStruct;
	};

	if (TObjectPtr<const UScriptStruct>* ExistingStruct = ComponentStructs.FindByPredicate(FindExistingStruct))
	{
		*ExistingStruct = InStruct;
	}
	else
	{
		ComponentStructs.Emplace(InStruct);
	}

	InvalidateKey();
}

uint64 FAnimNextSimpleAnimGraphBuilder::RecalculateKey() const
{
	FXxHash64Builder Hash;
	const int32 NumStacks = Stacks.Num();
	Hash.Update(&NumStacks, sizeof(NumStacks));
	for (const FAnimNextSimpleAnimGraphBuilderTraitStackDesc& StackDesc : Stacks)
	{
		const int32 StackSize = StackDesc.TraitDescs.Num();
		Hash.Update(&StackSize, sizeof(StackSize));
		Hash.Update(&StackDesc.Link, sizeof(StackDesc.Link));
		for(const FAnimNextSimpleAnimGraphBuilderTraitDesc& TraitDesc : StackDesc.TraitDescs)
		{
			const UScriptStruct* Struct = TraitDesc.TraitData.GetScriptStruct();
			Hash.Update(&Struct, sizeof(Struct));

			const int32 NumVariableMappings = TraitDesc.VariableMappings.Num();
			Hash.Update(&NumVariableMappings, sizeof(NumVariableMappings));
			for (const FAnimNextSimpleAnimGraphBuilderVariableMapping& VariableMapping : TraitDesc.VariableMappings)
			{
				uint32 VariableHash = GetTypeHash(VariableMapping.VariableReference);
				Hash.Update(&VariableHash, sizeof(VariableHash));
				uint32 PropertyNameHash = GetTypeHash(VariableMapping.PropertyName);
				Hash.Update(&PropertyNameHash, sizeof(PropertyNameHash));
			}
		}
	}

	const int32 NumVariables = VariableStructs.Num();
	Hash.Update(&NumVariables, sizeof(NumVariables));
	for (const UScriptStruct* VariableStruct : VariableStructs)
	{
		Hash.Update(&VariableStruct, sizeof(VariableStruct));
	}

	const int32 NumComponents = ComponentStructs.Num();
	Hash.Update(&NumComponents, sizeof(NumComponents));
	for (const UScriptStruct* ComponentStruct: ComponentStructs)
	{
		Hash.Update(&ComponentStruct, sizeof(ComponentStruct));
	}

	const int32 NumReferencedVariableAssets = ReferencedVariableAssets.Num();
	Hash.Update(&NumReferencedVariableAssets, sizeof(NumReferencedVariableAssets));
	for (auto ReferencedVariableAsset : ReferencedVariableAssets)
	{
		uint64 ObjHash = GetTypeHash(ReferencedVariableAsset);
		Hash.Update(&ObjHash, sizeof(ObjHash));
	}

	const int32 NumReferencedVariableRigVMAssets = ReferencedVariableRigVMAssets.Num();
	Hash.Update(&NumReferencedVariableRigVMAssets, sizeof(NumReferencedVariableRigVMAssets));
	for (auto ReferencedVariableRigVMAsset : ReferencedVariableRigVMAssets)
	{
		uint64 ObjHash = GetTypeHash(ReferencedVariableRigVMAsset);
		Hash.Update(&ObjHash, sizeof(ObjHash));
	}

	return Hash.Finalize().Hash;
}
