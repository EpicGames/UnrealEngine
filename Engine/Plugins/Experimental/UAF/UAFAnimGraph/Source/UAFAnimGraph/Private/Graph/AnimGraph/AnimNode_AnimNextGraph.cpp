// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimGraph/AnimNode_AnimNextGraph.h"
#include "Animation/AnimInstance.h"
#include "DataRegistry.h"
#include "ReferencePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimInstanceProxy.h"
#include "GameFramework/Actor.h"
#include "AnimationRuntime.h"
#include "GenerationTools.h"
#include "Graph/AnimNext_LODPose.h"
#include "Engine/SkeletalMesh.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Graph/AnimNextGraphInstance.h"
#include "UAF/UAFSkeletonUserData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_AnimNextGraph)

#if WITH_EDITOR
#include "Editor.h"
#endif

TAutoConsoleVariable<int32> CVarAnimNextForceAnimBP(TEXT("a.AnimNextForceAnimBP"), 0, TEXT("If != 0, then we use the input pose of the AnimNext AnimBP node instead of the AnimNext graph."));

FAnimNode_AnimNextGraph::FAnimNode_AnimNextGraph()
	: FAnimNode_CustomProperty()
	, AnimationGraph(nullptr)
	, LODThreshold(INDEX_NONE)
{
}

void FAnimNode_AnimNextGraph::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_CustomProperty::OnInitializeAnimInstance(InProxy, InAnimInstance);

	InitializeProperties(InAnimInstance, GetTargetClass());
}

void FAnimNode_AnimNextGraph::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	SourceLink.GatherDebugData(DebugData.BranchFlow(1.f));
}

void FAnimNode_AnimNextGraph::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	using namespace UE::UAF;

	SourceLink.Update(Context);

	if (IsLODEnabled(Context.AnimInstanceProxy) && !CVarAnimNextForceAnimBP.GetValueOnAnyThread() && GraphInstance.IsValid())
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		PropagateInputProperties(Context.AnimInstanceProxy->GetAnimInstanceObject());

		FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), Context.GetDeltaTime());
		UE::UAF::UpdateGraph(UpdateGraphContext);
	}

	FAnimNode_CustomProperty::Update_AnyThread(Context);

	//TRACE_ANIM_NODE_VALUE(Context, TEXT("Class"), *GetNameSafe(ControlRigClass.Get()));
}

void FAnimNode_AnimNextGraph::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	using namespace UE::UAF;

	SourceLink.Initialize(Context);

	// Release the instance if the graph has changed
	if(GraphInstance.IsValid() && !GraphInstance->UsesAnimationGraph(AnimationGraph))
	{
		GraphInstance.Reset();
	}

	// Lazily (re-)allocate graph instance if required
	if(!GraphInstance.IsValid())
	{
		GraphInstance = AnimationGraph->AllocateInstance();
	}

	FAnimNode_CustomProperty::Initialize_AnyThread(Context);
}

void FAnimNode_AnimNextGraph::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_CustomProperty::CacheBones_AnyThread(Context);

	SourceLink.CacheBones(Context);
}

void FAnimNode_AnimNextGraph::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	using namespace UE::UAF;

	if (!CVarAnimNextForceAnimBP.GetValueOnAnyThread() && GraphInstance.IsValid())
	{
		USkeletalMeshComponent* SkeletalMeshComponent = Output.AnimInstanceProxy->GetSkelMeshComponent();
		check(SkeletalMeshComponent != nullptr);

		FDataHandle RefPoseHandle = FDataRegistry::Get()->GetOrGenerateReferencePose(SkeletalMeshComponent);
		FAnimNextGraphReferencePose GraphReferencePose(RefPoseHandle);

		const int32 LODLevel = Output.AnimInstanceProxy->GetLODLevel();

		const UE::UAF::FReferencePose& RefPose = RefPoseHandle.GetRef<UE::UAF::FReferencePose>();
		FAnimNextGraphLODPose ResultPose;
		ResultPose.LODPose = FLODPoseHeap(RefPose, LODLevel, true, Output.ExpectsAdditivePose());

		{
			UE::UAF::FEvaluateGraphContext EvaluateGraphContext(*GraphInstance.Get(), RefPose, LODLevel);

			if (UE::UAF::UseAbstractSkeletonRuntime())
			{
                USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
                check(SkeletalMesh);
                
				// By default, use the everything set
				USkeleton* Skeleton = const_cast<USkeleton*>(RefPose.GetSkeletonAsset());
				check(Skeleton);
				
				UUAFSkeletonUserData* UserData = UUAFSkeletonUserData::FromSkeleton(Skeleton);
				UAbstractSkeletonSetBinding* Binding = UserData ? UserData->GetSetBinding() : nullptr;
				FName SetName;

				EvaluateGraphContext.SetSkeletonSet(Binding, SkeletalMesh, SetName);
			}

			const FEvaluationProgram EvaluationProgram = UE::UAF::EvaluateGraph(EvaluateGraphContext);

			FEvaluationVM EvaluationVM(EEvaluationFlags::All, RefPose, LODLevel);
			bool bHasValidOutput = false;

			if (!EvaluationProgram.IsEmpty())
			{
				EvaluationProgram.Execute(EvaluationVM);

				TUniquePtr<FKeyframeState> EvaluatedKeyframe;
				if (EvaluationVM.PopValue(KEYFRAME_STACK_NAME, EvaluatedKeyframe))
				{
					ResultPose.LODPose.CopyFrom(EvaluatedKeyframe->Pose);
					ResultPose.Curves.CopyFrom(EvaluatedKeyframe->Curves);
					ResultPose.Attributes.CopyFrom(EvaluatedKeyframe->Attributes);
					bHasValidOutput = true;
				}
			}

			if (!bHasValidOutput)
			{
				// We need to output a valid pose, generate one
				FKeyframeState ReferenceKeyframe = EvaluationVM.MakeReferenceKeyframe(Output.ExpectsAdditivePose());
				ResultPose.LODPose.CopyFrom(ReferenceKeyframe.Pose);
				ResultPose.Curves.CopyFrom(ReferenceKeyframe.Curves);
				ResultPose.Attributes.CopyFrom(ReferenceKeyframe.Attributes);
			}
		}

		FGenerationTools::RemapPose(ResultPose.LODPose, Output);
		Output.Curve.CopyFrom(ResultPose.Curves);
		FGenerationTools::RemapAttributes(ResultPose.LODPose, ResultPose.Attributes, Output);
	}
	else
	{
		if (SourceLink.GetLinkNode())
		{
			SourceLink.Evaluate(Output);
		}
	}

	FAnimNode_CustomProperty::Evaluate_AnyThread(Output);
}

void FAnimNode_AnimNextGraph::PostSerialize(const FArchive& Ar)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// after compile, we have to reinitialize
	// because it needs new execution code
	// since memory has changed
	if (Ar.IsObjectReferenceCollector())
	{
		if (AnimationGraph)
		{
			//AnimNextGraph->Initialize();
		}
	}
}

void FAnimNode_AnimNextGraph::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	if(GraphInstance.IsValid())
	{
		FAnimNextGraphInstance* ImplPtr = GraphInstance.Get();
		Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), ImplPtr);
	}
}

void FAnimNode_AnimNextGraph::InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass)
{
	// Build property lists
	SourceProperties.Reset(SourcePropertyNames.Num());
	DestProperties.Reset(SourcePropertyNames.Num());

	check(SourcePropertyNames.Num() == DestPropertyNames.Num());

	for (int32 Idx = 0; Idx < SourcePropertyNames.Num(); ++Idx)
	{
		FName& SourceName = SourcePropertyNames[Idx];
		UClass* SourceClass = InSourceInstance->GetClass();

		FProperty* SourceProperty = FindFProperty<FProperty>(SourceClass, SourceName);
		SourceProperties.Add(SourceProperty);
		DestProperties.Add(nullptr);
	}
}

void FAnimNode_AnimNextGraph::PropagateInputProperties(const UObject* InSourceInstance)
{
	if (InSourceInstance)
	{
		// Assign value to the properties exposed as pins
		for (int32 PropIdx = 0; PropIdx < SourceProperties.Num(); ++PropIdx)
		{
		}
	}
}


#if WITH_EDITOR

void FAnimNode_AnimNextGraph::HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	FAnimNode_CustomProperty::HandleObjectsReinstanced_Impl(InSourceObject, InTargetObject, OldToNewInstanceMap);
}

#endif
