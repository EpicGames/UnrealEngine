// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimGraphReference.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/WeakAnimGraphReference.h"
#include "Module/AnimNextModuleInstance.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"

namespace UE::UAF
{

FAnimGraphReference::FAnimGraphReference(TNonNullPtr<const UUAFAnimGraph> InAnimGraph, const FUAFWeakSystemReference& InSystemReference, FAnimNextFactoryParams&& InFactoryParams)
{
	check(InSystemReference.IsValid());
	Ptr = InAnimGraph->PreAllocateInstance({ .SystemReference = InSystemReference }, MoveTemp(InFactoryParams));
}

FWeakAnimGraphReference FAnimGraphReference::ToWeakRef() const
{
	return FWeakAnimGraphReference(*this);
}

void FAnimGraphReference::Reset()
{
	Ptr.Reset();
}

bool FAnimGraphReference::IsValid() const
{
	return Ptr.IsValid();
}

const UUAFAnimGraph* FAnimGraphReference::GetAnimGraph() const
{
	return Ptr.IsValid() ? Ptr->GetAnimationGraph() : nullptr;
}

TSharedPtr<FAnimNextGraphInstance> FAnimGraphReference::GetOrAllocate(FExecutionContext& InContext, const FTraitBinding& InBinding) const
{
	if (Ptr.IsValid())
	{
		Ptr->AllocateVariablesInternal(nullptr, InBinding.GetTraitPtr().GetNodeInstance()->GetOwner().AsShared());
		Ptr->AllocateGraphInternal();
		return Ptr;
	}
	return TSharedPtr<FAnimNextGraphInstance>();
}

void FAnimGraphReference::QueueTask(FUniqueAnimGraphTask&& InTaskFunction) const
{
	if (Ptr.IsValid() && Ptr->GetModuleInstance())
	{
		Ptr->GetModuleInstance()->QueueTask([TaskFunction = MoveTemp(InTaskFunction), WeakAnimGraph = TWeakPtr<FAnimNextGraphInstance>(Ptr)](const UE::UAF::FModuleTaskContext& InContext) mutable
		{
			if (TSharedPtr<FAnimNextGraphInstance> PinnedAnimGraph = WeakAnimGraph.Pin())
			{
				PinnedAnimGraph->RunOrQueueTask(MoveTemp(TaskFunction));
			}
		});
	}
}

void FAnimGraphReference::SetDebugName(FName InName) const
{
	if (Ptr.IsValid())
	{
		Ptr->SetDebugName(InName);
	}
}

void FAnimGraphReference::AddReferencedObjects(FReferenceCollector& InCollector) const
{
	if (Ptr.IsValid())
	{
		InCollector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), Ptr.Get());
	}
}

}
