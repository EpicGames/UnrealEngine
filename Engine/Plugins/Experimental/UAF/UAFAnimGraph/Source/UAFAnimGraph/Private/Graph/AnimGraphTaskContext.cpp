// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimGraphTaskContext.h"
#include "Graph/AnimNextGraphInstance.h"

namespace UE::UAF
{

FAnimNextGraphInstance& FAnimGraphTaskContext::GetAnimGraphInstance() const
{
	return static_cast<FAnimNextGraphInstance&>(Instance);
}

FAnimGraphTaskContext::FAnimGraphTaskContext(FAnimNextGraphInstance& InGraphInstance)
	: FInstanceTaskContext(InGraphInstance)
{
}

}