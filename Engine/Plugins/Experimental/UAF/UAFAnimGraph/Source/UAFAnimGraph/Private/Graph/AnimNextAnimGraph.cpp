// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNextGraphInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraph)

bool FAnimNextAnimGraph::IsEqualForInjectionSiteChange(const FAnimNextAnimGraph& InOther) const
{
	return Asset == InOther.Asset &&
			HostGraph == InOther.HostGraph;
}
