// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextAnimationGraph.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Graph/RigUnit_AnimNextShimRoot.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "TraitCore/TraitReader.h"
#include "TraitCore/ExecutionContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Serialization/MemoryReader.h"
#include "AnimNextAnimGraphStats.h"
#include "Graph/AnimNextGraphEntryPoint.h"
#include "Graph/RigUnit_AnimNextGraphEvaluator.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectResource.h"
#include "Module/AnimNextModuleInstance.h"
#include "DataRegistryTypes.h"
#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "UAFAnimGraphInitializerComponent.h"
#include "Factory/AnimNextFactoryParams.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/ExternalAssetDependencyGatherer.h"
REGISTER_ASSETDEPENDENCY_GATHERER(FExternalAssetDependencyGatherer, UUAFAnimGraph);
#endif // WITH_EDITOR

DEFINE_STAT(STAT_AnimNext_Graph_AllocateInstance);

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimationGraph)

UUAFAnimGraph::UUAFAnimGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExtendedExecuteContext.SetContextPublicDataStruct(FAnimNextExecuteContext::StaticStruct());

#if WITH_EDITOR
	if (!IsTemplate() && !HasAnyFlags(RF_Transient))
	{
		FEditorDelegates::OnPreForceDeleteObjects.AddUObject(this, &UUAFAnimGraph::OnPreForceDeleteObjects);
	}
#endif
}

TSharedPtr<FAnimNextGraphInstance> UUAFAnimGraph::PreAllocateInstance(const UE::UAF::FGraphAllocationParams& InParams, FAnimNextFactoryParams&& InFactoryParams) const
{
	LLM_SCOPE_BYNAME(TEXT("UAF/AnimGraph"))
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Graph_AllocateInstance);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName EntryPoint = (InParams.EntryPoint == NAME_None) ? DefaultEntryPoint : InParams.EntryPoint;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	const FAnimNextTraitHandle ResolvedRootTraitHandle = ResolvedRootTraitHandles.FindRef(EntryPoint);
	if (!ResolvedRootTraitHandle.IsValid())
	{
		UE_LOGF(LogAnimation, Error, "UUAFAnimGraph::AllocateInstance: Could not allocate graph instance - Root trait handle has not been resolved successfully.");
		return nullptr;
	}

#if WITH_EDITOR
	if (CompilationState == EAnimNextRigVMAssetState::CompiledWithErrors)
	{
		return nullptr;
	}
#endif

	TSharedPtr<FAnimNextGraphInstance> InstanceImpl = MakeShared<FAnimNextGraphInstance>();
	InstanceImpl->Asset = this;
	InstanceImpl->ModuleInstance = InParams.SystemReference.WeakPtr;
	InstanceImpl->EntryPoint = EntryPoint;
	if (InFactoryParams.IsValid())
	{
		InstanceImpl->Components.Add(TInstancedStruct<FUAFAnimGraphInitializerComponent>::Make(MoveTemp(InFactoryParams)));
	}
	InstanceImpl->AllocationState = FAnimNextGraphInstance::EAllocationState::PreAllocated;
	return InstanceImpl;
}

TSharedPtr<FAnimNextGraphInstance> UUAFAnimGraph::AllocateInstance(const UE::UAF::FGraphAllocationParams& InParams, FAnimNextFactoryParams&& InFactoryParams) const
{
	LLM_SCOPE_BYNAME(TEXT("UAF/AnimGraph"))
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Graph_AllocateInstance);

	TSharedPtr<FAnimNextGraphInstance> InstanceImpl = PreAllocateInstance(InParams, MoveTemp(InFactoryParams));
	if (!InstanceImpl.IsValid())
	{
		return InstanceImpl;
	}

	InstanceImpl->AllocateVariablesInternal(InParams.Overrides, InParams.ParentGraphInstance);
	InstanceImpl->AllocateGraphInternal();

	if (!InstanceImpl->IsValid())
	{
		// We failed to allocate our instance, reset the ptr
		InstanceImpl.Reset();
		UE_LOGF(LogAnimation, Error, "UUAFAnimGraph::AllocateInstance: Failed to allocate graph instance.");
	}

#if WITH_EDITORONLY_DATA
	if (InstanceImpl.IsValid() && InstanceImpl->IsValid())
	{
		FScopeLock Lock(&GraphInstancesLock);
		check(!GraphInstances.Contains(InstanceImpl.Get()));
		GraphInstances.Add(InstanceImpl.Get());
	}
#endif

	return InstanceImpl;
}

#if WITH_EDITOR
void UUAFAnimGraph::BeginDestroy()
{
	Super::BeginDestroy();

	if (!IsTemplate() && !HasAnyFlags(RF_Transient))
	{
		FEditorDelegates::OnPreForceDeleteObjects.RemoveAll(this);
	}
}
#endif

void UUAFAnimGraph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if(Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextCombineParameterBlocksAndGraphs)
		{
			// Skip over shared archive buffer if we are loading from an older version
			if (const FLinkerLoad* Linker = GetLinker())
			{
				const int32 LinkerIndex = GetLinkerIndex();
				const FObjectExport& Export = Linker->ExportMap[LinkerIndex];
				Ar.Seek(Export.SerialOffset + Export.SerialSize);
			}
		}
		else
		{
			int32 SharedDataArchiveBufferSize = 0;
			Ar << SharedDataArchiveBufferSize;

#if !WITH_EDITORONLY_DATA
			// When editor data isn't present, we don't persist the archive buffer as it is only needed on load
			// to populate the graph shared data
			TArray<uint8> SharedDataArchiveBuffer;
#endif

			SharedDataArchiveBuffer.SetNumUninitialized(SharedDataArchiveBufferSize);
			Ar.Serialize(SharedDataArchiveBuffer.GetData(), SharedDataArchiveBufferSize);

			if (Ar.IsLoadingFromCookedPackage())
			{
				// If we are cooked, we populate our graph shared data otherwise in the editor we'll compile on load
				// and re-populate everything then to account for changes in code/content
				LoadFromArchiveBuffer(SharedDataArchiveBuffer);
			}
		}
	}
	else if (Ar.IsSaving())
	{
#if WITH_EDITORONLY_DATA
		// We only save the archive buffer, if code changes we'll be able to de-serialize from it when
		// building the runtime buffer
		// This allows us to have editor only/non-shipping only properties that are stripped out on load
		int32 SharedDataArchiveBufferSize = SharedDataArchiveBuffer.Num();
		Ar << SharedDataArchiveBufferSize;
		Ar.Serialize(SharedDataArchiveBuffer.GetData(), SharedDataArchiveBufferSize);
#endif
	}
	else
	{
		// Counting, etc
		Ar << SharedDataBuffer;

#if WITH_EDITORONLY_DATA
		Ar << SharedDataArchiveBuffer;
#endif
	}
}

#if WITH_EDITORONLY_DATA
void UUAFAnimGraph::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	// Temp fix for Control Rig Trait getting called serialization before the class it depends has been loaded
	// Ideally, we need a mechanism to allow traits to provide dependencies
	for (TObjectPtr<UObject>& ReferencedObject : GraphReferencedObjects)
	{
		if (URigVMBlueprintGeneratedClass* ReferencedObjectPtr = Cast<URigVMBlueprintGeneratedClass>(ReferencedObject.Get()))
		{
			OutDeps.Add(ReferencedObjectPtr);
		}
	}
}
#endif

bool UUAFAnimGraph::LoadFromArchiveBuffer(const TArray<uint8>& InSharedDataArchiveBuffer)
{
	using namespace UE::UAF;

	// Reconstruct our graph shared data
	FMemoryReader GraphSharedDataArchive(InSharedDataArchiveBuffer);
	FTraitReader TraitReader(GraphReferencedObjects, GraphReferencedSoftObjects, GraphSharedDataArchive);

	const FTraitReader::EErrorState ErrorState = TraitReader.ReadGraph(SharedDataBuffer);
	if (ErrorState == FTraitReader::EErrorState::None)
	{
		for(int32 EntryPointIndex = 0; EntryPointIndex < EntryPoints.Num(); ++EntryPointIndex)
		{
			const FAnimNextGraphEntryPoint& EntryPoint = EntryPoints[EntryPointIndex];
			ResolvedRootTraitHandles.Add(EntryPoint.EntryPointName, TraitReader.ResolveEntryPointHandle(EntryPoint.RootTraitHandle));
			ResolvedEntryPoints.Add(EntryPoint.EntryPointName, EntryPointIndex);
		}

		// Make sure our execute method is registered
		FRigUnit_AnimNextGraphEvaluator::RegisterExecuteMethod(ExecuteDefinition);
		return true;
	}
	else
	{
		SharedDataBuffer.Empty(0);
		ResolvedRootTraitHandles.Add(DefaultEntryPoint, FAnimNextTraitHandle());
		return false;
	}
}

#if WITH_EDITOR
void UUAFAnimGraph::OnPreForceDeleteObjects(const TArray<UObject*>& ObjectsToDelete)
{
	if (!ObjectsToDelete.Contains(this))
	{
		// Not deleting this graph instance
		return;
	}

	FScopeLock Lock(&GraphInstancesLock);

	TSet<FAnimNextGraphInstance*> GraphInstancesCopy = GraphInstances;
	for (FAnimNextGraphInstance* GraphInstance : GraphInstancesCopy)
	{
		GraphInstance->Release();
	}
}
#endif

#if WITH_EDITORONLY_DATA
void UUAFAnimGraph::FreezeGraphInstances()
{
	FScopeLock Lock(&GraphInstancesLock);

	TSet<FAnimNextGraphInstance*> GraphInstancesCopy = GraphInstances;
	for (FAnimNextGraphInstance* GraphInstance : GraphInstancesCopy)
	{
		GraphInstance->Freeze();
	}
}

void UUAFAnimGraph::ThawGraphInstances()
{
	FScopeLock Lock(&GraphInstancesLock);

	TSet<FAnimNextGraphInstance*> GraphInstancesCopy = GraphInstances;
	for (FAnimNextGraphInstance* GraphInstance : GraphInstancesCopy)
	{
		GraphInstance->Thaw();
	}
}
#endif
