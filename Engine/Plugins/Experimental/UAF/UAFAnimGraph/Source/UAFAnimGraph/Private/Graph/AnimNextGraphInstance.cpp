// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextGraphInstance.h"
#include "Graph/AnimNextGraphInstance.h"

#include "AnimNextAnimGraphStats.h"
#include "ObjectTrace.h"
#include "UAFAnimGraphInitializerComponent.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Graph/AnimGraphTaskContext.h"
#include "Script/UAFRigVMComponent.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphContextData.h"
#include "Graph/AnimNextGraphLatentPropertiesContextData.h"
#include "TraitCore/ExecutionContext.h"
#include "Graph/RigUnit_AnimNextShimRoot.h"
#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/IGarbageCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextGraphInstance)

DEFINE_STAT(STAT_AnimNext_Graph_RigVM);

FAnimNextGraphInstance::FAnimNextGraphInstance()
	: FUAFAssetInstance(StaticStruct())
	, GCHandler(*this)
{
}

FAnimNextGraphInstance::~FAnimNextGraphInstance()
{
	TRACE_INSTANCE_LIFETIME_END(ModuleInstance.IsValid() ? ModuleInstance.Pin()->GetObject() : nullptr, GetUniqueId());
	Release();
}

void FAnimNextGraphInstance::Release()
{
#if WITH_EDITORONLY_DATA
	if(const UUAFAnimGraph* Graph = GetAnimationGraph())
	{
		FScopeLock Lock(&Graph->GraphInstancesLock);
		Graph->GraphInstances.Remove(this);
	}
#endif

	if (!GraphInstancePtr.IsValid())
	{
		return;
	}

	GraphInstancePtr.Reset();
	ModuleInstance.Reset();
	RootGraphInstance = nullptr;
	GCHandler.TraitsWithReferences.Reset();
	ReleaseComponents();
	Asset = nullptr;
}

bool FAnimNextGraphInstance::IsValid() const
{
	return GraphInstancePtr.IsValid();
}

const UUAFAnimGraph* FAnimNextGraphInstance::GetAnimationGraph() const
{
	return CastChecked<UUAFAnimGraph>(Asset, ECastCheckedType::NullAllowed);
}

FName FAnimNextGraphInstance::GetEntryPoint() const
{
	return EntryPoint;
}

UE::UAF::FWeakTraitPtr FAnimNextGraphInstance::GetGraphRootPtr() const
{
	return GraphInstancePtr;
}

FAnimNextModuleInstance* FAnimNextGraphInstance::GetModuleInstance() const
{
	return ModuleInstance.Pin().Get();
}

FUAFWeakSystemReference FAnimNextGraphInstance::GetModuleInstanceReference() const
{
	if(TSharedPtr<FAnimNextModuleInstance> PinnedModuleInstance = ModuleInstance.Pin())
	{
		return FUAFWeakSystemReference(PinnedModuleInstance->GetReference());
	}
	return FUAFWeakSystemReference();
}

FAnimNextGraphInstance* FAnimNextGraphInstance::GetParentGraphInstance() const
{
	if(IsRoot())
	{
		return nullptr;
	}
	else
	{
		return static_cast<FAnimNextGraphInstance*>(HostInstance.Pin().Get());
	}
}

FAnimNextGraphInstance* FAnimNextGraphInstance::GetRootGraphInstance() const
{
	return RootGraphInstance;
}

bool FAnimNextGraphInstance::UsesAnimationGraph(const UUAFAnimGraph* InAnimationGraph) const
{
	return GetAnimationGraph() == InAnimationGraph;
}

bool FAnimNextGraphInstance::UsesEntryPoint(FName InEntryPoint) const
{
	if(const UUAFAnimGraph* AnimationGraph = GetAnimationGraph())
	{
		if(InEntryPoint == NAME_None)
		{
			return EntryPoint == AnimationGraph->DefaultEntryPoint;
		}

		return InEntryPoint == EntryPoint;
	}
	return false;
}

bool FAnimNextGraphInstance::IsRoot() const
{	
	return this == RootGraphInstance;
}

bool FAnimNextGraphInstance::HasUpdated() const
{
	return bHasUpdatedOnce;
}

void FAnimNextGraphInstance::MarkAsUpdated()
{
	bHasUpdatedOnce = true;
}

void FAnimNextGraphInstance::ExecuteLatentPins(const TConstArrayView<UE::UAF::FLatentPropertyHandle>& LatentHandles, void* DestinationBasePtr, float DeltaTime, bool bIsFrozen, bool bJustBecameRelevant)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Graph_RigVM);

	if (!IsValid())
	{
		return;
	}

	FUAFRigVMComponent* RigVMComponent = TryGetComponent<FUAFRigVMComponent>();
	if (ensure(RigVMComponent))
	{
		FAnimNextGraphLatentPropertiesContextData ContextData( *this, LatentHandles, DestinationBasePtr, DeltaTime, bIsFrozen, bJustBecameRelevant);
		RigVMComponent->StaticCallEvent(*RigVMComponent, ContextData);
	}
}

void FAnimNextGraphInstance::CopyVariablesToLatentPins(const TConstArrayView<UE::UAF::FLatentPropertyHandle>& LatentHandles, void* DestinationBasePtr, bool bIsFrozen, bool bJustBecameRelevant)
{
	for (UE::UAF::FLatentPropertyHandle Handle : LatentHandles)
	{
		if (!Handle.IsIndexValid())
		{
			// This handle isn't valid
			continue;
		}

		if (!Handle.IsOffsetValid())
		{
			// This handle isn't valid
			continue;
		}

		if (bIsFrozen && Handle.CanFreeze())
		{
			// This handle can freeze and we are frozen, no need to update it
			continue;
		}

		if (!bJustBecameRelevant && Handle.OnBecomeRelevant())
		{
			// This handle should only update on become relevant
			continue;
		}

		uint8* DestinationPtr = (uint8*)DestinationBasePtr + Handle.GetLatentPropertyOffset();
		AccessVariablePropertyByIndex(Handle.GetLatentPropertyIndex(), [DestinationPtr](const FProperty* InProperty, TArrayView<uint8> InData)
		{
			// Copy from our source into our destination
			// We assume the source and destination properties are identical
			check(InData.Num() == InProperty->GetElementSize());
			InProperty->CopyCompleteValue(DestinationPtr, InData.GetData());
		});
	}
}

void FAnimNextGraphInstance::AllocateVariablesInternal(const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides, TWeakPtr<FAnimNextGraphInstance> InParentGraphInstance)
{
	using namespace UE::UAF;
	
	check(AllocationState >= EAllocationState::PreAllocated);

	if (AllocationState >= EAllocationState::VariablesAllocated)
	{
		// Nothing to do, already allocated variables
		return;
	}

	if (InParentGraphInstance.IsValid())
	{
		HostInstance = InParentGraphInstance;
		RootGraphInstance = InParentGraphInstance.Pin()->GetRootGraphInstance();
	}
	else
	{
		HostInstance = ModuleInstance;
		RootGraphInstance = this;
	}

	auto FindInitializerComponent = [](const TInstancedStruct<FUAFAssetInstanceComponent>& InComponent)
	{
		return InComponent.GetScriptStruct() == FUAFAnimGraphInitializerComponent::StaticStruct();
	};

	TInstancedStruct<FUAFAssetInstanceComponent> InitializerStruct;
	int32 InitializerComponentIndex = Components.IndexOfByPredicate(FindInitializerComponent);
	if (InitializerComponentIndex != INDEX_NONE)
	{
		InitializerStruct = MoveTemp(Components[InitializerComponentIndex]);
		Components.RemoveAt(InitializerComponentIndex, EAllowShrinking::No);
	}

	// Copy any built-in components from the asset
	CopyDefaultComponents();

	// Set up variable's data
	InitializeVariables(InOverrides);

	// Bind components now variables are set up
	BindDefaultComponents();

	// Flush any initialization tasks we have
	if (InitializerStruct.IsValid())
	{
		const FUAFAnimGraphInitializerComponent& InitializerComponent = InitializerStruct.Get<FUAFAnimGraphInitializerComponent>();
		InitializerComponent.FactoryParams.InitializeInstance(*this);
		for (const FUniqueAnimGraphTask& Task : InitializerComponent.TaskQueue)
		{
			Task(FAnimGraphTaskContext(*this));
		}
	}

	AllocationState = EAllocationState::VariablesAllocated;
}

void FAnimNextGraphInstance::AllocateGraphInternal()
{
	check(AllocationState >= EAllocationState::VariablesAllocated);

	if (AllocationState == EAllocationState::Allocated)
	{
		// Nothing to do, already allocated
		return;
	}

	// Verify the resolved entry point
	const FAnimNextTraitHandle ResolvedRootTraitHandle = GetAnimationGraph()->ResolvedRootTraitHandles.FindRef(EntryPoint);
	if (!ensure(ResolvedRootTraitHandle.IsValid()))
	{
		return;
	}

	UE::UAF::FExecutionContext Context(*this);
	GraphInstancePtr = Context.AllocateNodeInstance(*this, ResolvedRootTraitHandle);

	AllocationState = EAllocationState::Allocated;
}

#if WITH_EDITORONLY_DATA
void FAnimNextGraphInstance::Freeze()
{
	if (!IsValid())
	{
		return;
	}

	GraphInstancePtr.Reset();
	ReleaseComponents();
	GCHandler.TraitsWithReferences.Reset();
	bHasUpdatedOnce = false;
}

void FAnimNextGraphInstance::Thaw()
{
	if (const UUAFAnimGraph* AnimationGraph = GetAnimationGraph())
	{
		// Copy any built-in components from the asset
		CopyDefaultComponents();

		if (Variables.bHasBeenInitialized)
		{
			MigrateVariables();
		}
		else
		{
			InitializeVariables();
		}

		// Bind components now variables are set up
		BindDefaultComponents();

		{
			UE::UAF::FExecutionContext Context(*this);
			if(const FAnimNextTraitHandle* FoundHandle = AnimationGraph->ResolvedRootTraitHandles.Find(EntryPoint))
			{
				GraphInstancePtr = Context.AllocateNodeInstance(*this, *FoundHandle);
			}
		}

		if (!IsValid())
		{
			// We failed to allocate our instance, clear everything
			Release();
		}
	}
}

#endif

void FAnimNextGraphInstance::RegisterWithGC(const UE::UAF::FWeakTraitPtr& InTraitPtr)
{
	GCHandler.TraitsWithReferences.Add(InTraitPtr);
}

void FAnimNextGraphInstance::UnregisterWithGC(const UE::UAF::FWeakTraitPtr& InTraitPtr)
{
	using namespace UE::UAF;
	
	const int32 EntryIndex = GCHandler.TraitsWithReferences.IndexOfByPredicate(
		[&InTraitPtr](const FWeakTraitPtr& TraitPtr)
		{
			return TraitPtr == InTraitPtr;
		});

	if (ensure(EntryIndex != INDEX_NONE))
	{
		GCHandler.TraitsWithReferences.RemoveAtSwap(EntryIndex);
	}
}

void FAnimNextGraphInstance::RunOrQueueTask(UE::UAF::FUniqueAnimGraphTask&& InTaskFunction)
{
	using namespace UE::UAF;

	if (AllocationState == EAllocationState::PreAllocated)
	{
		FUAFAnimGraphInitializerComponent& InitializerComponent = GetOrAddComponent<FUAFAnimGraphInitializerComponent>();
		InitializerComponent.TaskQueue.Add(MoveTemp(InTaskFunction));
	}
	else
	{
		InTaskFunction(UE::UAF::FAnimGraphTaskContext(*this));
	}
}

void FAnimNextGraphInstance::FGCHandler::AddReferencedObjects(FReferenceCollector& Collector)
{
	using namespace UE::UAF;

	Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), &Owner);

	FExecutionContext Context;
	FTraitStackBinding TraitStack;
	TTraitBinding<IGarbageCollection> GCTrait;

	// TODO: If we kept the entries sorted by graph instance, we could re-use the execution context
	for (const FWeakTraitPtr& TraitPtr : TraitsWithReferences)
	{
		Context.BindTo(TraitPtr);

		if (Context.GetStack(TraitPtr, TraitStack))
		{
			if(ensure(TraitStack.GetInterface(GCTrait)))
			{
				GCTrait.AddReferencedObjects(Context, Collector);
			}
		}
	}
}

FString FAnimNextGraphInstance::FGCHandler::GetReferencerName() const
{
	return TEXT("UAF Graph Instance");
}
