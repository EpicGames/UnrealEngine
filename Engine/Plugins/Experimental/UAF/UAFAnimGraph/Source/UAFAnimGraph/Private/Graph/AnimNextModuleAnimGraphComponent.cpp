// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextModuleAnimGraphComponent.h"

#include "Graph/AnimNextGraphInstance.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleAnimGraphComponent)

TWeakPtr<FAnimNextGraphInstance> FAnimNextModuleAnimGraphComponent::AllocateInstance(const UUAFAnimGraph* InAnimationGraph, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides)
{
	check(InAnimationGraph);

	TSharedPtr<FAnimNextGraphInstance> NewInstance = InAnimationGraph->AllocateInstance(
		{
			.SystemReference = GetModuleInstance().GetReference(),
			.Overrides = InOverrides
		});
	if(!NewInstance.IsValid())
	{
		return TWeakPtr<FAnimNextGraphInstance>();
	}

	// Persist a strong reference to overrides here
	GraphInstances.Add(NewInstance.ToSharedRef());
	return NewInstance;
}

void FAnimNextModuleAnimGraphComponent::ReleaseInstance(TWeakPtr<FAnimNextGraphInstance> InWeakInstance)
{
	TSharedPtr<FAnimNextGraphInstance> PinnedInstance = InWeakInstance.Pin();
	if(PinnedInstance.IsValid())
	{
		check(GraphInstances.Contains(PinnedInstance));			// Should not be releasing this instance if it is not owned by this module
		GraphInstances.Remove(PinnedInstance);
		check(PinnedInstance.GetSharedReferenceCount() == 1);	// This should be the final reference, all others should be weak
	}
}

void FAnimNextModuleAnimGraphComponent::AddStructReferencedObjects(class FReferenceCollector& Collector)
{
	for (const TSharedPtr<FAnimNextGraphInstance>& GraphInstance : GraphInstances)
	{
		Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), GraphInstance.Get());
	}
}

