// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextSkeletalMeshComponentLibrary.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSkeletalMeshComponentLibrary)

FAnimNextGraphReferencePose UAnimNextSkeletalMeshComponentLibrary::GetReferencePose(USkeletalMeshComponent* InComponent)
{
	UE::UAF::FDataHandle RefPoseHandle = UE::UAF::FDataRegistry::Get()->GetOrGenerateReferencePose(InComponent);
	return FAnimNextGraphReferencePose(RefPoseHandle);
}
