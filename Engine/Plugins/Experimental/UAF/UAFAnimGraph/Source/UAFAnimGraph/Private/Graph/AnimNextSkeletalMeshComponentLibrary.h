// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Graph/AnimNext_LODPose.h"
#include "AnimNextSkeletalMeshComponentLibrary.generated.h"

class USkeletalMeshComponent;

// Access to non-UProperty/UFunction data on USkeletalMeshComponent
UCLASS()
class UAnimNextSkeletalMeshComponentLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Returns the reference pose for the specified component
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	static FAnimNextGraphReferencePose GetReferencePose(USkeletalMeshComponent* InComponent);
};
