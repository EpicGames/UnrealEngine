// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigDecorator_AnimNextCppTrait.h"

#if WITH_EDITOR
#include "RigVMModel/RigVMController.h"
#include "TraitCore/TraitRegistry.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDecorator_AnimNextCppTrait)

#if WITH_EDITOR
void FRigDecorator_AnimNextCppDecorator::GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray) const
{
	if (DecoratorSharedDataStruct == nullptr)
	{
		return;
	}

	FStructOnScope OriginalValueMemoryScope(DecoratorSharedDataStruct);
	FStructOnScope DefaultValueMemoryScope(DecoratorSharedDataStruct);

	if (!InDefaultValue.IsEmpty())
	{
		FRigVMPinDefaultValueImportErrorContext ErrorPipe;
		DecoratorSharedDataStruct->ImportText(*InDefaultValue, DefaultValueMemoryScope.GetStructMemory(), nullptr, PPF_SerializedAsImportText, &ErrorPipe, DecoratorSharedDataStruct->GetName());
	}

	const TFunction<ERigVMPinDefaultValueType(const FName&)> DefaultValueTypeGetter = [&OriginalValueMemoryScope, &DefaultValueMemoryScope](const FName& InPropertyName)
	{
		if(const FProperty* Property = OriginalValueMemoryScope.GetStruct()->FindPropertyByName(InPropertyName))
		{
			if(Property->Identical_InContainer(OriginalValueMemoryScope.GetStructMemory(), DefaultValueMemoryScope.GetStructMemory()))
			{
				return ERigVMPinDefaultValueType::Unset;
			}
			return ERigVMPinDefaultValueType::Override;
		}
		return ERigVMPinDefaultValueType::AutoDetect;
	};


	// Give the Trait an opportunity to generate its own pins
	const UE::UAF::FTrait* Trait = GetTrait();
	check(Trait != nullptr);
	FAnimNextTraitSharedData* TraitSharedData = (FAnimNextTraitSharedData*)DefaultValueMemoryScope.GetStructMemory();

	// Obtain programmatic pins
	const int32 ProgrammaticPinIndexStart = OutPinArray.Num();
	Trait->GetProgrammaticPins(TraitSharedData, InController, InParentPinIndex, InTraitPin, InDefaultValue, OutPinArray);

	const int32 ProgrammaticPinIndexEnd = OutPinArray.Num();
	for (int32 PinIndex = ProgrammaticPinIndexStart; PinIndex < ProgrammaticPinIndexEnd; ++PinIndex)
	{
		FRigVMPinInfo& PinInfo = OutPinArray[PinIndex];
		PinInfo.bIsLazy = true;			// we flag all programmatic pins as lazy, as currently we have to remap them inside the trait
		PinInfo.Property = nullptr;		// we setup the pin explicitly, avoid RigVM do additional setup
	}

	// Generate the shared data struct pins
	OutPinArray.AddPins(DecoratorSharedDataStruct, InController, ERigVMPinDirection::Invalid, InParentPinIndex, DefaultValueTypeGetter, DefaultValueMemoryScope.GetStructMemory(), true);

	const int32 NumPins = OutPinArray.Num();
	for (int32 PinIndex = ProgrammaticPinIndexEnd; PinIndex < NumPins; ++PinIndex)  // iterate pins created after the programmatic ones
	{
		FRigVMPinInfo& PinInfo = OutPinArray[PinIndex];

		if (PinInfo.Property == nullptr)
		{
			// This pin doesn't have a property, we'll have to assume that it has been fully specified by the trait
			continue;
		}

		const bool bIsInline = PinInfo.Property->HasMetaData("Inline");
		const bool bIsTraitHandle = PinInfo.Property->GetCPPType() == TEXT("FAnimNextTraitHandle");

		// Trait handle pins are never hidden because we need to still be able to link things to it
		// UI display will use the hidden property if specified
		const bool bIsHidden = bIsTraitHandle ? false : PinInfo.Property->HasMetaData(FRigVMStruct::HiddenMetaName);

		// Check if the metadata stipulates that we should explicitly hide this property, if not we mark it as an input
		PinInfo.Direction = bIsHidden ? ERigVMPinDirection::Hidden : ERigVMPinDirection::Input;

		// For top level properties of traits, if we don't explicitly tag the property as inline or hidden, it is lazy
		// Except for trait handles which are never lazy since they just encode graph connectivity
		if (InParentPinIndex == PinInfo.ParentIndex && !bIsHidden && !bIsInline && !bIsTraitHandle)
		{
			if (Trait->IsPropertyLatent(*TraitSharedData, PinInfo.Property->GetFName()))
			{
				PinInfo.bIsLazy = true;
			}
			else
			{
				// This can occur when a property is latent (default) and wasn't included in the GENERATE_TRAIT_LATENT_PROPERTIES enumerator
				InController->ReportErrorf(TEXT("Shared data property '%s' is latent but is missing from the latent property enumerator"), *PinInfo.Name.ToString());
				PinInfo.bIsLazy = false;
			}
		}
		else
		{
			PinInfo.bIsLazy = false;
		}

		// Remove our property because we configure the pin explicitly
		PinInfo.Property = nullptr;
	}
}

FRigVMStructUpgradeInfo FRigDecorator_AnimNextCppDecorator::GetUpgradeInfo() const
{
	const UE::UAF::FTrait* Trait = GetTrait();
	check(Trait != nullptr);
	return Trait->GetUpgradeInfo();
}

const UE::UAF::FTrait* FRigDecorator_AnimNextCppDecorator::GetTrait() const
{
	return UE::UAF::FTraitRegistry::Get().Find(DecoratorSharedDataStruct);
}

#endif
