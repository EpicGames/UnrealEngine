// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextGraphEvaluator.h"
#include "Graph/AnimNextGraphContextData.h"
#include "Graph/AnimNextGraphEvaluatorExecuteDefinition.h"
#include "Graph/AnimNextGraphLatentPropertiesContextData.h"
#include "TraitCore/LatentPropertyHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextGraphEvaluator)

namespace UE::UAF::Private
{
	static FRWLock GRegisteredGraphEvaluatorMethodsLock;
	static TMap<uint32, FAnimNextGraphEvaluatorExecuteDefinition> GRegisteredGraphEvaluatorMethods;

	TArray<FRigVMFunctionArgument> GetGraphEvaluatorFunctionArguments(const FAnimNextGraphEvaluatorExecuteDefinition& ExecuteDefinition)
	{
		TArray<FRigVMFunctionArgument> Arguments;
		Arguments.Reserve(ExecuteDefinition.Arguments.Num());

		for (const FAnimNextGraphEvaluatorExecuteArgument& Argument : ExecuteDefinition.Arguments)
		{
			Arguments.Add(FRigVMFunctionArgument(Argument.Name, Argument.CPPType, ERigVMFunctionArgumentDirection::Input));
		}

		return Arguments;
	}
}

void FRigUnit_AnimNextGraphEvaluator::StaticExecute(FRigVMExtendedExecuteContext& RigVMExecuteContext, FRigVMMemoryHandleArray RigVMMemoryHandles, FRigVMPredicateBranchArray RigVMBranches)
{
	const FAnimNextExecuteContext& VMExecuteContext = RigVMExecuteContext.GetPublicData<FAnimNextExecuteContext>();
	const FAnimNextGraphLatentPropertiesContextData& LatentPropertiesContextData = VMExecuteContext.GetContextData<FAnimNextGraphLatentPropertiesContextData>();

	const TConstArrayView<UE::UAF::FLatentPropertyHandle>& LatentHandles = LatentPropertiesContextData.GetLatentHandles();
	uint8* DestinationBasePtr = (uint8*)LatentPropertiesContextData.GetDestinationBasePtr();
	const bool bIsFrozen = LatentPropertiesContextData.IsFrozen();
	const bool bJustBecameRelevant = LatentPropertiesContextData.JustBecameRelevant();

	for (UE::UAF::FLatentPropertyHandle Handle : LatentHandles)
	{
		if (!Handle.IsIndexValid())
		{
			// This handle isn't valid
			continue;
		}

		if (!Handle.IsOffsetValid())
		{
			// This handle isn't valid
			continue;
		}

		if ((bIsFrozen && Handle.CanFreeze()) || (!bJustBecameRelevant && Handle.OnBecomeRelevant()))
		{
			// This handle can freeze and we are frozen, no need to update it
			continue;
		}

		FRigVMMemoryHandle& MemoryHandle = RigVMMemoryHandles[Handle.GetLatentPropertyIndex()];

		// We cannot currently determine from our memory handle whether this value is a direct wire-up to a variable, in which
		// case it will not be lazy, thus we cannot verify lazy-correctness before calling into this so we just guard instead.
		if (MemoryHandle.IsLazy())
		{
			MemoryHandle.ComputeLazyValueIfNecessary(RigVMExecuteContext, RigVMExecuteContext.GetSliceHash());
		}

		const uint8* SourcePtr = MemoryHandle.GetInputData();
		uint8* DestinationPtr = DestinationBasePtr + Handle.GetLatentPropertyOffset();

		// Copy from our source into our destination
		// We assume the source and destination properties are identical
		URigVMMemoryStorage::CopyProperty(
			MemoryHandle.GetProperty(), DestinationPtr,
			MemoryHandle.GetProperty(), SourcePtr);
	}
}

void FRigUnit_AnimNextGraphEvaluator::RegisterExecuteMethod(const FAnimNextGraphEvaluatorExecuteDefinition& ExecuteDefinition)
{
	using namespace UE::UAF::Private;

	{
		UE::TWriteScopeLock WriteLock(GRegisteredGraphEvaluatorMethodsLock);
		if (GRegisteredGraphEvaluatorMethods.Contains(ExecuteDefinition.Hash))
		{
			return;	// Already registered
		}

		GRegisteredGraphEvaluatorMethods.Add(ExecuteDefinition.Hash, ExecuteDefinition);
	}

	const FString FullExecuteMethodName = FString::Printf(TEXT("FRigUnit_AnimNextGraphEvaluator::%s"), *ExecuteDefinition.MethodName);

	const TArray<FRigVMFunctionArgument> GraphEvaluatorArguments = GetGraphEvaluatorFunctionArguments(ExecuteDefinition);
	FRigVMRegistry::Get().Register(*FullExecuteMethodName, &FRigUnit_AnimNextGraphEvaluator::StaticExecute, FRigUnit_AnimNextGraphEvaluator::StaticStruct(), GraphEvaluatorArguments);
}

const FAnimNextGraphEvaluatorExecuteDefinition* FRigUnit_AnimNextGraphEvaluator::FindExecuteMethod(uint32 ExecuteMethodHash)
{
	using namespace UE::UAF::Private;

	UE::TWriteScopeLock ReadLock(GRegisteredGraphEvaluatorMethodsLock);
	return GRegisteredGraphEvaluatorMethods.Find(ExecuteMethodHash);
}
