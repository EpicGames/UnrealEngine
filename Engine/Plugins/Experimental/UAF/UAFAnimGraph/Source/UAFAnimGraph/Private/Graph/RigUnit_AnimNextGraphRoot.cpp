// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextGraphRoot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextGraphRoot)

FName FRigUnit_AnimNextGraphRoot::EventName = TEXT("DummyExecute");
FName FRigUnit_AnimNextGraphRoot::DefaultEntryPoint = TEXT("Root");

FRigUnit_AnimNextGraphRoot_DummyExecute()
{
	// This RigUnit should never execute, it is only used in the editor
	ensure(false);
}
