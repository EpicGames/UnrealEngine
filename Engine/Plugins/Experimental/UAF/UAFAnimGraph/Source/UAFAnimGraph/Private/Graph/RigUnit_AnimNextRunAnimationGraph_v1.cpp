// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextRunAnimationGraph_v1.h"

#include "AnimNextAnimGraphStats.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextModuleAnimGraphComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "TraitCore/TraitEventList.h"
#include "UAF/UAFSkeletonUserData.h"
#include "Graph/UAFSystemOutputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextRunAnimationGraph_v1)


FRigUnit_AnimNextRunAnimationGraph_v1_Execute()
{
	SCOPED_NAMED_EVENT(UAF_Run_Graph_V1, FColor::Orange);

	using namespace UE::UAF;

	const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
	FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();

	if (!ReferencePose.ReferencePose.IsValid())
	{
		return;
	}

	FAnimNextModuleAnimGraphComponent& AnimationGraphComponent = ModuleInstance.GetOrAddComponent<FAnimNextModuleAnimGraphComponent>();

	if(Graph == nullptr)
	{
		AnimationGraphComponent.ReleaseInstance(WorkData.WeakInstance);
		return;
	}

	// Release the instance if the graph has changed
	if(WorkData.WeakInstance.IsValid() && !WorkData.WeakInstance.Pin()->UsesAnimationGraph(Graph))
	{
		AnimationGraphComponent.ReleaseInstance(WorkData.WeakInstance);
	}

	// Lazily (re-)allocate graph instance if required
	if(!WorkData.WeakInstance.IsValid())
	{
		WorkData.WeakInstance = AnimationGraphComponent.AllocateInstance(Graph, Overrides.GetOverrides().Pin());
	}

	if(!WorkData.WeakInstance.IsValid())
	{
		return;
	}

	// Take a strong reference to the host instance, we are going to run it
	TSharedRef<FAnimNextGraphInstance> InstanceRef = WorkData.WeakInstance.Pin().ToSharedRef();

	const UE::UAF::FReferencePose& RefPose = ReferencePose.ReferencePose.GetRef<UE::UAF::FReferencePose>();

	int32 DesiredLOD = LOD;
	if (DesiredLOD == -1)
	{
		DesiredLOD = RefPose.GetSourceLODLevel();
	}

	// TODO: Currently forcing additive flag to false here
	if (Result.LODPose.ShouldPrepareForLOD(RefPose, DesiredLOD, false))
	{
		Result.LODPose.PrepareForLOD(RefPose, DesiredLOD, true, false);
	}

	ensure(Result.LODPose.LODLevel == DesiredLOD);

	// Every graph in a schedule will see the same input events (if they were queued before the schedule started)
	FUpdateGraphContext UpdateGraphContext(InstanceRef.Get(), ExecuteContext.GetDeltaTime());
	FTraitEventList& InputEventList = UpdateGraphContext.GetInputEventList();

	// A schedule can contain multiple graphs, we copy the input event list since it might be appended to during our update
	{
		UE::TReadScopeLock ReadLock(ModuleInstance.EventListLock);
		InputEventList = ModuleInstance.InputEventList;
	}

	// Track how many input events we started with, we'll append the new ones
	const int32 NumOriginalInputEvents = InputEventList.Num();

	// Internally we use memstack allocation, so we need a mark here
	FMemStack& MemStack = FMemStack::Get();
	FMemMark MemMark(MemStack);

	// We allocate a dummy buffer to trigger the allocation of a large chunk if this is the first mark
	// This reduces churn internally by avoiding a chunk to be repeatedly allocated and freed as we push/pop marks
	MemStack.Alloc(size_t(FPageAllocator::SmallPageSize) + 1, 16);

	UpdateGraph(UpdateGraphContext);
	
	FEvaluateGraphContext EvaluateGraphContext(InstanceRef.Get(), RefPose, DesiredLOD);

	if (UseAbstractSkeletonRuntime())
	{
		FUAFSystemOutputComponent& OutputPoseComponent = ModuleInstance.GetOrAddComponent<FUAFSystemOutputComponent>();
    		
    	USkeletalMesh* SkeletalMesh = OutputPoseComponent.GetSkeletalMesh();
    	check(SkeletalMesh);
    	
		// By default, use the everything set
		USkeleton* Skeleton = const_cast<USkeleton*>(RefPose.GetSkeletonAsset());
		check(Skeleton)
		
		UUAFSkeletonUserData* UserData = UUAFSkeletonUserData::FromSkeleton(Skeleton);
		UAbstractSkeletonSetBinding* Binding = UserData != nullptr ? UserData->GetSetBinding() : nullptr;
		FName SetName;

		EvaluateGraphContext.SetSkeletonSet(Binding, SkeletalMesh, SetName);
	}

	EvaluateGraph(EvaluateGraphContext, Result);

	// We might have appended new input/output events, append them
	{
		const int32 NumInputEvents = InputEventList.Num();

		UE::TWriteScopeLock WriteLock(ModuleInstance.EventListLock);

		// Append the new input events
		for (int32 EventIndex = NumOriginalInputEvents; EventIndex < NumInputEvents; ++EventIndex)
		{
			FAnimNextTraitEventPtr& Event = InputEventList[EventIndex];
			if (Event->IsValid())
			{
				ModuleInstance.InputEventList.Push(MoveTemp(Event));
			}
		}

		// Append our output events
		ModuleInstance.OutputEventList.Append(UpdateGraphContext.GetOutputEventList());
	}
}
