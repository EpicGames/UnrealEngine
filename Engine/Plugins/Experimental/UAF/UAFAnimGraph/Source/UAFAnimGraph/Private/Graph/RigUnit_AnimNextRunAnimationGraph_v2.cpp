// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextRunAnimationGraph_v2.h"

#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "Module/AnimNextModuleInstance.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/AnimNextModuleAnimGraphComponent.h"
#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "Injection/GraphInstanceInjectionComponent.h"
#include "TraitCore/TraitEventList.h"
#include "Injection/AnimNextModuleInjectionComponent.h"
#include "Graph/UAFAnimGraphTrace.h"
#include "Factory/AnimGraphFactory.h"
#include "UAF/UAFSkeletonUserData.h"
#include "UAFLogging.h"
#include "Engine/SkeletalMesh.h"
#include "Graph/RigUnit_UAFRunAsset.h"
#include "Graph/UAFSystemOutputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextRunAnimationGraph_v2)

FRigUnit_AnimNextRunAnimationGraph_v2_Execute()
{
	SCOPED_NAMED_EVENT(UAF_Run_Graph_V2, FColor::Orange);

	using namespace UE::UAF;

	const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
	FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();

	FDataHandle RefPoseDataHandle = ReferencePose.ReferencePose;

	// Fall back to output pose ref pose
	if (!RefPoseDataHandle.IsValid())
	{
		FUAFSystemOutputComponent& OutputPoseComponent = ModuleInstance.GetOrAddComponent<FUAFSystemOutputComponent>();
		RefPoseDataHandle = OutputPoseComponent.GetRefPose().ReferencePose;
	}

	if (!RefPoseDataHandle.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not run graph - Ref Pose is invalid. [Asset: %s] [HostGraph: %s]"), 
			*GetNameSafe(Graph.Asset), *GetNameSafe(Graph.HostGraph));
		return;
	}

	const UE::UAF::FReferencePose& RefPose = RefPoseDataHandle.GetRef<UE::UAF::FReferencePose>();

	int32 DesiredLOD = LOD;
	if (DesiredLOD == -1)
	{
		DesiredLOD = RefPose.GetSourceLODLevel();
	}

	// TODO: Currently forcing additive flag to false here
	if (Result.LODPose.ShouldPrepareForLOD(RefPose, DesiredLOD, false))
	{
		Result.LODPose.PrepareForLOD(RefPose, DesiredLOD, true, false);
	}

	ensure(Result.LODPose.LODLevel == DesiredLOD);

	// Get a host to run this graph
	const UUAFAnimGraph* HostGraph = Graph.HostGraph ? Graph.HostGraph.Get() : FAnimGraphFactory::GetDefaultGraphHost();
	FAnimNextModuleAnimGraphComponent& AnimationGraphComponent = ModuleInstance.GetOrAddComponent<FAnimNextModuleAnimGraphComponent>();
	if(HostGraph == nullptr)
	{
		AnimationGraphComponent.ReleaseInstance(WorkData.WeakHost);
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not run graph - No valid host graph could be found. [Asset: %s] [HostGraph: %s]"),
			*GetNameSafe(Graph.Asset), *GetNameSafe(Graph.HostGraph));
		return;
	}


	// Release the instance if the host graph has changed
	if(WorkData.WeakHost.IsValid() && !WorkData.WeakHost.Pin()->UsesAnimationGraph(HostGraph))
	{
		AnimationGraphComponent.ReleaseInstance(WorkData.WeakHost);
	}

	// Lazily (re-)allocate graph instance if required
	if(!WorkData.WeakHost.IsValid())
	{
		// Reset our graph reference, we may be allocating a new graph
		WorkData.InjectedGraphReference.Reset();

		TSharedPtr<FAnimNextGraphInstance> NewGraphInstance = AnimationGraphComponent.AllocateInstance(HostGraph, Overrides.GetOverrides().Pin()).Pin();
		if (NewGraphInstance.IsValid())
		{
			WorkData.WeakHost = NewGraphInstance;

			FGraphInstanceInjectionComponent& HostInjectionComponent = NewGraphInstance->GetOrAddComponent<FGraphInstanceInjectionComponent>();
			WorkData.InjectedGraphReference = HostInjectionComponent.GetInjectionInfo().GetDefaultInjectionSite();
		}
	}

	if(!WorkData.WeakHost.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not run graph - Host Graph could not be allocated. [Asset: %s] [HostGraph: %s]"),
			*GetNameSafe(Graph.Asset), *GetNameSafe(Graph.HostGraph));
		return;
	}

	// Take a strong reference to the host instance, we are going to run it
	TSharedRef<FAnimNextGraphInstance> HostInstanceRef = WorkData.WeakHost.Pin().ToSharedRef();
	FAnimNextGraphInstance& HostInstance = HostInstanceRef.Get();

	// Re-set the injected graph each time, as it may change per-update
	if (!WorkData.InjectedGraphReference.IsNone())
	{
		HostInstance.SetVariable(WorkData.InjectedGraphReference, Graph);
	}

	// Every graph in a schedule will see the same input events (if they were queued before the schedule started)
	FUpdateGraphContext UpdateGraphContext(HostInstance, ExecuteContext.GetDeltaTime());

	FTraitEventList& InputEventList = UpdateGraphContext.GetInputEventList();

	// A module can contain multiple graphs, we copy the input event list since it might be appended to during our update
	{
		UE::TReadScopeLock ReadLock(ModuleInstance.EventListLock);
		InputEventList = ModuleInstance.InputEventList;
	}

	// Track how many input events we started with, we'll append the new ones
	const int32 NumOriginalInputEvents = InputEventList.Num();

	// Internally we use memstack allocation, so we need a mark here
	FMemStack& MemStack = FMemStack::Get();
	FMemMark MemMark(MemStack);

	// We allocate a dummy buffer to trigger the allocation of a large chunk if this is the first mark
	// This reduces churn internally by avoiding a chunk to be repeatedly allocated and freed as we push/pop marks
	MemStack.Alloc(size_t(FPageAllocator::SmallPageSize) + 1, 16);

	UpdateGraph(UpdateGraphContext);

	FEvaluateGraphContext EvaluateGraphContext(HostInstance, RefPose, DesiredLOD);

	if (UseAbstractSkeletonRuntime())
	{
		FUAFSystemOutputComponent& OutputPoseComponent = ModuleInstance.GetOrAddComponent<FUAFSystemOutputComponent>();
		
		USkeletalMesh* SkeletalMesh = OutputPoseComponent.GetSkeletalMesh();
		
		USkeleton* Skeleton = const_cast<USkeleton*>(RefPose.GetSkeletonAsset());
		check(Skeleton);
		
		UUAFSkeletonUserData* UserData = UUAFSkeletonUserData::FromSkeleton(Skeleton);
		UAbstractSkeletonSetBinding* Binding = UserData != nullptr ? UserData->GetSetBinding() : nullptr;

		EvaluateGraphContext.SetSkeletonSet(Binding, SkeletalMesh, AttributeSet);
	}

	EvaluateGraph(EvaluateGraphContext, Result);

	TRACE_UAF_MODULE(ModuleInstance);
	TRACE_UAF_GRAPHINSTANCES(HostInstance);

	// We might have appended new input/output events, append them
	{
		const int32 NumInputEvents = InputEventList.Num();

		UE::TWriteScopeLock WriteLock(ModuleInstance.EventListLock);

		// Append the new input events
		for (int32 EventIndex = NumOriginalInputEvents; EventIndex < NumInputEvents; ++EventIndex)
		{
			FAnimNextTraitEventPtr& Event = InputEventList[EventIndex];
			if (Event->IsValid())
			{
				ModuleInstance.InputEventList.Push(MoveTemp(Event));
			}
		}

		// Append our output events
		ModuleInstance.OutputEventList.Append(UpdateGraphContext.GetOutputEventList());
	}
}

FRigVMStructUpgradeInfo FRigUnit_AnimNextRunAnimationGraph_v2::GetUpgradeInfo() const
{
	FRigUnit_UAFRunAsset NewNode;
	NewNode.Graph = Graph;
	NewNode.Overrides = Overrides;
	NewNode.AttributeSet = AttributeSet;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}