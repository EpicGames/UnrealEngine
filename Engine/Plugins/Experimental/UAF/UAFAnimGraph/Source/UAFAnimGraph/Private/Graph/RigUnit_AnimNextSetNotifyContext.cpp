// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_AnimNextSetNotifyContext.h"

#include "Components/SkeletalMeshComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFNotifyDispatcherComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextSetNotifyContext)

FRigUnit_AnimNextSetNotifyContext_Execute()
{
	FAnimNextModuleInstance& ModuleInstance = ExecuteContext.GetContextData<FAnimNextModuleContextData>().GetModuleInstance();
	FUAFNotifyDispatcherComponent& NotifyDispatcher = ModuleInstance.GetOrAddComponent<FUAFNotifyDispatcherComponent>();
	NotifyDispatcher.SkeletalMeshComponent = SkeletalMeshComponent;
	NotifyDispatcher.NotifyQueue.PredictedLODLevel = SkeletalMeshComponent ? SkeletalMeshComponent->GetPredictedLODLevel() : 0;
}
