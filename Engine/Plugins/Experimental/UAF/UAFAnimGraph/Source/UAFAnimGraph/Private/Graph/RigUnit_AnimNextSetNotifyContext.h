// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "AnimNextExecuteContext.h"
#include "RigUnit_AnimNextSetNotifyContext.generated.h"

#define UE_API UAFANIMGRAPH_API

/** Sets up the context in which notifies are called */
USTRUCT(meta=(DisplayName="Set Notify Context", Category="Notifies", NodeColor="0, 1, 1", Keywords="Event,Output"))
struct FRigUnit_AnimNextSetNotifyContext : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// The mesh component to supply to any notify dispatches
	UPROPERTY(meta = (Input))
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	// The execute context to use for the given skeletal mesh component
	UPROPERTY(meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API
