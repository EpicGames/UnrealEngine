// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextShimRoot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextShimRoot)

FName FRigUnit_AnimNextShimRoot::EventName = TEXT("Execute");

FRigUnit_AnimNextShimRoot_Execute()
{
	// This RigUnit is a no-op entry point, it simply forwards execution to the real entry point
	// @see FRigUnit_AnimNextRuntimeRoot
}
