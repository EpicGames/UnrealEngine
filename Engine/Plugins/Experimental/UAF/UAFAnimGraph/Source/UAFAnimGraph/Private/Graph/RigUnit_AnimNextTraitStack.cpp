// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextTraitStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextTraitStack)

FRigUnit_AnimNextTraitStack_Execute()
{
	// This RigUnit should never execute, it is only used in the editor as a container
	ensure(false);
}
