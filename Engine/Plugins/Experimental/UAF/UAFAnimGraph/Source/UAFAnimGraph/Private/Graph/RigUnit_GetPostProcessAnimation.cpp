// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_GetPostProcessAnimation.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Graph/PostProcessAnimationAssetUserData.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetPostProcessAnimation)

FRigUnit_GetPostProcessAnimation_Execute()
{
	Graph = {};
	bShouldEvaluate = true;
	LODThreshold = -1;

	USkeletalMeshComponent* UsedSkeletalMeshComponent = SkeletalMeshComponent;
	if (!UsedSkeletalMeshComponent)
	{
		const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
		FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();
		const FAnimNextSkeletalMeshComponentReferenceComponent& ComponentReference = ModuleInstance.GetOrAddComponent<FAnimNextSkeletalMeshComponentReferenceComponent>();
		UsedSkeletalMeshComponent = ComponentReference.GetComponent();
	}

	if (UsedSkeletalMeshComponent)
	{
		USkeletalMesh* SkeletalMesh = UsedSkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh)
		{
			UPostProcessAnimationUserAssetData* UserAssetData = Cast<UPostProcessAnimationUserAssetData>(SkeletalMesh->GetAssetUserDataOfClass(UPostProcessAnimationUserAssetData::StaticClass()));
			if (UserAssetData)
			{
				Graph.Asset = UserAssetData->AnimationAsset;
			}

			LODThreshold = SkeletalMesh->GetPostProcessAnimGraphLODThreshold();
			bShouldEvaluate = SkeletalMesh->ShouldEvaluatePostProcessAnimGraph(LODLevel);
		}
	}
	else
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not get post process animation - Skeletal mesh component is not valid."));
	}
}
