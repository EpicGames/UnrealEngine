// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Graph/AnimNextAnimGraph.h"

#include "RigUnit_GetPostProcessAnimation.generated.h"

/** Get post process animation data for a given skeletal mesh. */
USTRUCT(meta=(DisplayName="Get Post-Process Animation", Category="Animation Graph", NodeColor="0.5, 1, 1", Keywords="Graph,Port"))
struct FRigUnit_GetPostProcessAnimation : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	/** Skeletal mesh component to be used to read the post-process animation from the assigned skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Post-Process Animation", meta = (Input))
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	/** Current LOD we run animation with on the given skeletal mesh component. */
	UPROPERTY(EditAnywhere, Category = "Post-Process Animation", meta = (Input))
	int32 LODLevel = INDEX_NONE;

	/** Post-process animation graph to run, read from the given skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Post-Process Animation", meta = (Output))
	FAnimNextAnimGraph Graph;

	/** Can we skip or should we run the output animation graph based on the given input LOD and the skeletal mesh's post-process LOD threshold. */
	UPROPERTY(EditAnywhere, Category = "Post-Process Animation", meta = (Output))
	bool bShouldEvaluate = true;

	/** Raw post-process animation LOD threshold read from the given skeletal mesh. This can be used for custom thresholding when the "Should Evaluate" is not sufficient. */
	UPROPERTY(EditAnywhere, Category = "Post-Process Animation", meta = (Output))
	int32 LODThreshold = -1;

	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};
