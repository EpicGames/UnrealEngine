// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "RigUnit_HasValidEvaluationModifier.generated.h"

// Used to determine whether an evaluation modifier is valid (indicates active injection)
USTRUCT(meta=(DisplayName="Has Valid Evaluation Modifier", Category="Animation Next Injection", NodeColor="0, 1, 1"))
struct FRigUnit_HasValidEvaluationModifier : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The Injection Data instance to check
	UPROPERTY(EditAnywhere, Category = "Struct", meta = (Input))
	FAnimNextGraphInjectionData InjectionData;

	// Whether Evaluation Modifier is valid (indicates active injection)
	UPROPERTY(EditAnywhere, Category = "Struct", meta = (Output))
	bool bValue = false;
};
