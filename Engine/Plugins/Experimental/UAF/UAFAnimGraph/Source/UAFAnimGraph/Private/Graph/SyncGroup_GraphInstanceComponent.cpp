// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/SyncGroup_GraphInstanceComponent.h"

#include "Graph/AnimNextAnimationGraph.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/IGroupSynchronization.h"
#include "TraitInterfaces/ITimeline.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "AnimationUtils.h"
#include "Module/AnimNextModuleInstance.h"
#include "VisualLogger/VisualLogger.h"

// Enabled by default in development builds
// We log debug information using the visual logger
// Use Rewind Debugger to record and replay logs in editor

#include UE_INLINE_GENERATED_CPP_BY_NAME(SyncGroup_GraphInstanceComponent)
#define UE_DEBUG_SYNC_GROUPS !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

// [Sync Group Details]
// Sequence synchronization is a complex topic, hopefully explained here in sufficient
// details to clarify what is going on.
// 
// UE synchronization is group based: players are assigned a group they belong to
// and they synchronize within that group. In order to do so, one member of the group
// is elected leader (@see FindLeaderIndex, @see EAnimGroupSynchronizationRole) while
// the rest are followers.
// 
// [Sync Group Name]
// Typically, when a graph is hand authored, a user specified group name will be used.
// This would be common for something like locomotion. However, sometimes we wish
// for things to synchronize together but not with re-entrant versions.
// Consider a blend space. We wish for its sequences to synchronize together. However
// if our parent node spawns a new instance of the blend space with different parameters
// we would like for each instance to NOT synchronize together. This is called
// self-synchronization: it is local only to the current instance. In a case like this,
// if we select a user supplied group name, then each instance will share the same
// group name and self-synchronization is not possible. To that end, unique group names
// can be generated and used for this. The parent node would be responsible for generating
// the unique group name (e.g. motion matching node) and to forward it to its children.
// 
// [Sync Group Mode]
// There are three sync group modes (@see EAnimGroupSynchronizationMode):
//    - No synchronization: disables synchronization
//    - Using group name: uses the user supplied group name to synchronize with other members
//    - Using unique group name: ignores any user supplied group name and generates a new
//      unique name for members to synchronize with
// 
// [Sync Group Method]
// There are two synchronization methods: time based and marker based. This is mostly
// dictated by the leader. If the leader has markers, everyone in the group will
// attempt to use them. Members without markers will revert to time based syncing.
// If the leader does not have markers, all members will use time based syncing.
// 
// [Sync Point Matching]
// When a new member joins a group, we need to determine where it starts playing.
// If we opt to match the sync point, we will make a best effort attempt to find a
// suitable start point as follows.
// 
// When a new leader joins an existing group, we first look at the old leader.
// If the old leader was using marker based syncing, then we use its old marker pair.
// If the old leader was using time based syncing (maybe it was alone in the group),
// we attempt to find the marker pair where it currently resides. We can only do so
// if the old leader is still a member of the current group. Once we have a valid
// marker pair for the old leader, we have two choices: match the phase or match
// the marker pair.
// 
// Matching the phase is most desirable but it has the strictest requirements. Phase
// matching allows the synchronization behavior to be deterministic: it doesn't matter
// when a member joins, it always ends up at the same phase even if it must loop to
// match. It works as follows:
//     - Find the marker pair of the leader (or old leader)
//       This pair comprises of: prev/next marker indices
//     - Collect all markers from the start of the sequence, up to our current pair
//     - The joining member can then skip each collected marker, in order, starting
//       from the start of the sequence.
// 
// For this to work, phase matching requires:
//     - Both sequences must begin with the same marker (e.g. both start with LeftFoot)
//     - Both sequences must have their markers in the same order (e.g. LRLR and LRLRLR)
// 
// When phase matching is not possible, we attempt to match the marker pair. We do so
// as follows:
//     - Find the marker pair of the leader (or old leader)
//       This pair comprises of: prev/next marker indices
//     - The joining member can then start searching for the closest marker pair
//       that matches. This could find a pair that is before/after the current position.
//       Non-matching markers will be skipped/ignored. As such, it is beneficial if
//       the new member has an approximate start position already set. If the pair
//       already matches then we can begin playing from nearby.
// 
// If the new leader joins an existing group and it does not use markers, then it
// will attempt to use the old leader's normalized play position like a time based
// follower would. Play then resumes at the same normalized position within the group.
// 
// If a new leader joins but it does NOT wish to match the sync point, it will snap
// the group to its current position (e.g. teleport/force update). When this occurs,
// all members of the group will be treated as if they have just joined and thus defer
// to their own sync point matching behavior.
// 
// Similarly, when a new follower joins a group, it can also attempt to match the sync
// point of the leader. To do so, it operates in the same way as a new leader would:
// we look for the leader's marker pair and we attempt to match the phase or the marker
// pair as described above. If we aren't using markers, then we match the normalized
// play position of the leader.
// 
// [Time Advancement]
// Group synchronization hijacks the normal graph time update in that players will not
// advance on their own. Instead, we must wait for all group members to be collected to
// begin synchronization.
// 
// Leaders will advance normally using their own delta time.
// 
// Followers will compute their own delta time based on how much the leader advanced.
// When markers are used, followers will attempt to pass the same markers passed by
// the leader to attempt to keep in phase (when possible). Unknown or mismatched markers
// will be skipped. The delta time computed is then the difference between where the
// follower was and where it should be within the new marker pair.
// When time based synchronization is used, we similarly compute our delta time based
// on where the follower was and where it should be based on the normalized play time
// of the leader.
// 
// Groups with a single member or un-grouped members advance normally without synchronization.
// 
// At times, a player will advance in time without playing events (e.g. notifies). This
// typically occurs when we need to snap to a position (e.g. when we join). When this occurs,
// we do not wish to fire events between our current and desired position. We always
// perform one advance that does trigger events for leaders. Followers never trigger events.
//

namespace UE::UAF
{
	namespace Private
	{
#if UE_DEBUG_SYNC_GROUPS
		// Enables/disable debug logging
		static TAutoConsoleVariable<bool> CDebugVarEnableSyncLog(TEXT("a.AnimNext.EnableSyncLog"), true, TEXT("Toggles debug sync group logging"));

		// Enables sync mode debugging
		// 0: Auto Sync (uses markers when present, time otherwise, default behavior)
		// 1: Time Sync (ignores markers)
		// 2: No Sync (un-grouped sync behavior)
		static TAutoConsoleVariable<int32> CDebugVarSyncMode(TEXT("a.AnimNext.SyncGroupMode"), 0, TEXT("Debug sync group follower mode. 0: Auto Sync, 1: Time Sync, 2: No Sync"));
#endif

		// Encapsulates the group phase position
		struct FSyncGroupPhasePosition
		{
			// The name of the previous marker in our phase
			FName PrevMarkerName;

			// The name of the next marker in our phase
			FName NextMarkerName;

			// The index of the previous marker in our phase
			int32 PrevMarkerIndex = MarkerIndexSpecialValues::Uninitialized;

			// The index of the next marker in our phase
			int32 NextMarkerIndex = MarkerIndexSpecialValues::Uninitialized;

			// The normalized relative position between our two markers
			float PositionBetweenMarkers = 0.0f;
		};

		// Sync group member state as collected during graph traversal
		struct FSyncGroupMember
		{
			FTraitUpdateState					TraitState;

			// Strong pointer to trait to keep it alive
			FTraitPtr							TraitPtr;

			FSyncGroupParameters				GroupParameters;

			// Whether or not this member was part of the group on a given update
			bool bIsActive = false;

			// Whether or not we joined the group in the current update
			bool bJustJoined = false;
		};

		// Sync group state as collected during graph traversal
		struct FSyncGroupState
		{
			// The sync group name
			FName GroupName;

			// The list of sync group members during the current update
			TArray<FSyncGroupMember> Members;

			// If this sync group was active during the previous update, this is the timeline progress of its leader
			FTimelineState PreviousLeaderTimelineState;

			// If this sync group was active during the previous update, this is the list of sync markers from the previous leader
			TArray<FTimelineSyncMarker, TInlineAllocator<8>> PreviousLeaderSyncMarkers;

			// If this sync group was active during the previous update, this is the phase position where the leader ended
			FSyncGroupPhasePosition PreviousLeaderPhasePosition;

#if UE_DEBUG_SYNC_GROUPS
			int32 PreviousLeaderIndex = INDEX_NONE;
#endif

			// Whether or not this sync group was active during the previous update
			bool bIsActive = false;

			// Whether or not this sync group just formed during the current update
			bool bJustFormed = true;
		};

		// Sync group member context when performing group synchronization
		struct FSyncGroupMemberContext
		{
			const FSyncGroupMember* State = nullptr;

			FTraitStackBinding TraitStack;
			TTraitBinding<ITimeline> TimelineTrait;

			FTimelineSyncMarkerArray SyncMarkers;
			bool bUseMarkerSyncing = false;

			void Init(const FSyncGroupMember& InState)
			{
				State = &InState;
				TraitStack.Reset();
				TimelineTrait.Reset();
				SyncMarkers.Reset();
				bUseMarkerSyncing = false;
			}
		};

		// Sync group context when performing group synchronization
		struct FSyncGroupContext
		{
			const FSyncGroupState* State = nullptr;

			TArrayView<FSyncGroupMemberContext> Members;

			// If this sync group was active during the previous update, this is the timeline progress of its leader
			FTimelineState PreviousLeaderTimelineState;

			// Current leader progress, set after leader has advanced
			FTimelineState LeaderTimelineState;
			int32 LeaderIndex = INDEX_NONE;
			bool bIsLeaderPlayingForward = false;
			float LeaderStartRatio = 0.0f;
			float LeaderEndRatio = 0.0f;
			TArray<FName, TMemStackAllocator<>> MarkersPassed;

			bool bCanGroupUseMarkerSyncing = false;
			FSyncGroupPhasePosition LeaderPhaseStart;
			FSyncGroupPhasePosition LeaderPhaseEnd;

			TSet<FName> ValidMarkers;
			TSet<FName> CandidateMarkers;
			TBitArray<TMemStackAllocator<>> GroupSeenMarkers;
			TBitArray<TMemStackAllocator<>> MemberSeenMarkers;
		};

		// A unique group name we've generated
		struct FSyncGroupUniqueName
		{
			FName GroupName;

			FSyncGroupUniqueName* NextFreeEntry = nullptr;
		};

		static FName NAME_UniqueGroupNamePrefix = TEXT("UE_UNIQUE_GROUP_NAME");

		static void InitGroup(const FSyncGroupState& GroupState, TArray<FSyncGroupMemberContext, TMemStackAllocator<>>& ContextPool, FSyncGroupContext& GroupContext)
		{
			const int32 NumMembers = GroupState.Members.Num();
			check(NumMembers != 0);	// Groups should never be empty

			TArrayView<FSyncGroupMemberContext> Contextes(ContextPool.GetData(), NumMembers);
			for (int32 MemberIndex = 0; MemberIndex < NumMembers; ++MemberIndex)
			{
				Contextes[MemberIndex].Init(GroupState.Members[MemberIndex]);
			}

			GroupContext.State = &GroupState;
			GroupContext.Members = Contextes;
			GroupContext.PreviousLeaderTimelineState = GroupState.PreviousLeaderTimelineState;
			GroupContext.LeaderTimelineState.Reset();
			GroupContext.LeaderIndex = INDEX_NONE;
			GroupContext.bIsLeaderPlayingForward = false;
			GroupContext.LeaderStartRatio = 0.0f;
			GroupContext.LeaderEndRatio = 0.0f;
			GroupContext.MarkersPassed.Reset();
			GroupContext.bCanGroupUseMarkerSyncing = false;
			GroupContext.ValidMarkers.Reset();
			GroupContext.CandidateMarkers.Reset();
			GroupContext.GroupSeenMarkers.Reset();
			GroupContext.MemberSeenMarkers.Reset();
		}

		static void FindLeaderIndex(FExecutionContext& Context, FSyncGroupContext& GroupContext)
		{
			// Find our leader by looking at the total weight and the group role
			const int32 NumMembers = GroupContext.Members.Num();
			check(NumMembers != 0);	// Groups should never be empty

			int32 LeaderIndex = INDEX_NONE;
			float LeaderTotalWeight = -1.0f;

			for (int32 MemberIndex = 0; MemberIndex < NumMembers; ++MemberIndex)
			{
				const FSyncGroupMemberContext& MemberContext = GroupContext.Members[MemberIndex];
				const FSyncGroupMember& GroupMember = *MemberContext.State;

				switch (GroupMember.GroupParameters.GroupRole)
				{
				case EAnimGroupSynchronizationRole::CanBeLeader:
				case EAnimGroupSynchronizationRole::TransitionLeader:
					// Highest weight is the leader
					if (GroupMember.TraitState.GetTotalWeight() > LeaderTotalWeight)
					{
						LeaderIndex = MemberIndex;
						LeaderTotalWeight = GroupMember.TraitState.GetTotalWeight();
					}
					break;
				case EAnimGroupSynchronizationRole::AlwaysLeader:
				case EAnimGroupSynchronizationRole::ExclusiveAlwaysLeader:
					// Always set the leader index
					LeaderIndex = MemberIndex;
					LeaderTotalWeight = 2.0f;		// Some high value
					break;
				default:
				case EAnimGroupSynchronizationRole::AlwaysFollower:
				case EAnimGroupSynchronizationRole::TransitionFollower:
					// Never set the leader index
					// If we find no leader, we'll use the first index as set below
					break;
				}
			}

			if (LeaderIndex == INDEX_NONE)
			{
				// If none of the entries wish to be a leader, grab the first and force it
				LeaderIndex = 0;
			}

#if UE_DEBUG_SYNC_GROUPS
			if (CDebugVarEnableSyncLog.GetValueOnAnyThread() && GroupContext.State->PreviousLeaderIndex != LeaderIndex)
			{
				UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Is New Leader"),
					*GroupContext.State->GroupName.ToString(),
					GroupContext.Members[LeaderIndex].State->TraitPtr.GetNodeInstance());
			}
#endif

			GroupContext.LeaderIndex = LeaderIndex;
		}

		// Returns true if sync markers are valid and sorted, false otherwise
		// Only in non-test/shipping builds
		static bool CheckSyncMarkersSorted(const FTimelineSyncMarkerArray& SyncMarkers)
		{
			bool bIsValid = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			float PreviousMarkerPosition = 0.0f;
			for (const FTimelineSyncMarker& SyncMarker : SyncMarkers)
			{
				const float MarkerPosition = SyncMarker.GetPosition();

				if (!ensureMsgf(MarkerPosition >= 0.0 && MarkerPosition >= PreviousMarkerPosition, TEXT("Sync Markers should have a positive finite position and should be sorted in ascending order")))
				{
					bIsValid = false;
					break;
				}

				PreviousMarkerPosition = MarkerPosition;
			}
#endif

			return bIsValid;
		}

		static void BuildGroupState(FExecutionContext& Context, FSyncGroupContext& GroupContext)
		{
			const int32 NumMembers = GroupContext.Members.Num();
			const int32 LeaderIndex = GroupContext.LeaderIndex;
			FSyncGroupMemberContext& GroupLeaderContext = GroupContext.Members[LeaderIndex];

			bool bCanGroupUseMarkerSyncing = false;

			// Get the group sync marker names from the leader
			{
				const FSyncGroupMember& GroupLeader = *GroupLeaderContext.State;

				FTraitStackBinding& LeaderTraitStack = GroupLeaderContext.TraitStack;

				Context.BindTo(GroupLeader.TraitPtr);
				ensure(Context.GetStack(GroupLeader.TraitPtr, LeaderTraitStack));
				ensure(LeaderTraitStack.GetInterface(GroupLeaderContext.TimelineTrait));

				if (NumMembers == 1)
				{
					return;	// If the leader is alone, we have no need for syncing, we'll advance the leader normally
				}

				FTimelineSyncMarkerArray& SyncMarkers = GroupLeaderContext.SyncMarkers;

				SyncMarkers.Reset();
				GroupLeaderContext.TimelineTrait.GetSyncMarkers(Context, SyncMarkers);

				bCanGroupUseMarkerSyncing = !SyncMarkers.IsEmpty() && CheckSyncMarkersSorted(SyncMarkers);
				GroupLeaderContext.bUseMarkerSyncing = bCanGroupUseMarkerSyncing;
			}

			// If our leader has sync markers, iterate over every member of the group and remove markers
			// that they do not share
			if (bCanGroupUseMarkerSyncing)
			{
				TSet<FName>& ValidMarkers = GroupContext.ValidMarkers;
				TSet<FName>& CandidateMarkers = GroupContext.CandidateMarkers;
				TBitArray<TMemStackAllocator<>>& GroupSeenMarkers = GroupContext.GroupSeenMarkers;
				TBitArray<TMemStackAllocator<>>& MemberSeenMarkers = GroupContext.MemberSeenMarkers;

				CandidateMarkers.Reset();
				CandidateMarkers.Reserve(GroupLeaderContext.SyncMarkers.Num());
				for (const FTimelineSyncMarker& SyncMarker : GroupLeaderContext.SyncMarkers)
				{
					CandidateMarkers.Add(SyncMarker.GetName());
				}

				// The group of seen markers starts with the list from the leader
				// We'll combine this bitset for every follower and only the markers in common to each of them will be used
				const int32 NumGroupSyncMarkerBits = CandidateMarkers.GetMaxIndex();
				GroupSeenMarkers.Reset();
				GroupSeenMarkers.Add(false, NumGroupSyncMarkerBits);

				// Initialize seen markers using the leader
				for (auto It = CandidateMarkers.CreateConstIterator(); It; ++It)
				{
					const FSetElementId SyncMarkerId = It.GetId();
					check(SyncMarkerId.IsValidId());

					GroupSeenMarkers[SyncMarkerId.AsInteger()] = true;
				}

				for (int32 MemberIndex = 0; MemberIndex < NumMembers; ++MemberIndex)
				{
					if (MemberIndex == LeaderIndex)
					{
						continue;	// Ignore the leader
					}

					FSyncGroupMemberContext& GroupMemberContext = GroupContext.Members[MemberIndex];
					const FSyncGroupMember& GroupMember = *GroupMemberContext.State;

					FTraitStackBinding& MemberTraitStack = GroupMemberContext.TraitStack;

					Context.BindTo(GroupMember.TraitPtr);
					ensure(Context.GetStack(GroupMember.TraitPtr, MemberTraitStack));
					ensure(MemberTraitStack.GetInterface(GroupMemberContext.TimelineTrait));

					FTimelineSyncMarkerArray& SyncMarkers = GroupMemberContext.SyncMarkers;
					SyncMarkers.Reset();
					GroupMemberContext.TimelineTrait.GetSyncMarkers(Context, SyncMarkers);

					// Members without markers can still use time based syncing
					const bool bCanMemberUseMarkerSyncing = !SyncMarkers.IsEmpty() && CheckSyncMarkersSorted(SyncMarkers);
					GroupMemberContext.bUseMarkerSyncing = bCanMemberUseMarkerSyncing;

					if (bCanMemberUseMarkerSyncing)
					{
						MemberSeenMarkers.Reset();
						MemberSeenMarkers.Add(false, NumGroupSyncMarkerBits);

						// Increment the markers this member contains
						for (const FTimelineSyncMarker& SyncMarker : SyncMarkers)
						{
							const FSetElementId SyncMarkerId = CandidateMarkers.FindId(SyncMarker.GetName());
							if (SyncMarkerId.IsValidId())
							{
								MemberSeenMarkers[SyncMarkerId.AsInteger()] = true;
							}
						}

						GroupSeenMarkers.CombineWithBitwiseAND(MemberSeenMarkers, EBitwiseOperatorFlags::MaintainSize);
					}
				}

				// Build the list of valid markers
				// A marker is valid if it is present in every member of the group
				// We build a new set as it is faster than repeatedly removing entries
				{
					ValidMarkers.Reset();
					ValidMarkers.Reserve(GroupSeenMarkers.Num());

					FSetElementId SyncMarkerId = FSetElementId::FromInteger(GroupSeenMarkers.FindFrom(true, 0));
					while (CandidateMarkers.IsValidId(SyncMarkerId))
					{
						ValidMarkers.Add(CandidateMarkers.Get(SyncMarkerId));

						SyncMarkerId = FSetElementId::FromInteger(GroupSeenMarkers.FindFrom(true, SyncMarkerId.AsInteger() + 1));
					}
				}

				bCanGroupUseMarkerSyncing = !ValidMarkers.IsEmpty();
			}
			else
			{
				for (int32 MemberIndex = 0; MemberIndex < NumMembers; ++MemberIndex)
				{
					if (MemberIndex == LeaderIndex)
					{
						continue;	// Ignore the leader
					}

					FSyncGroupMemberContext& GroupMemberContext = GroupContext.Members[MemberIndex];
					const FSyncGroupMember& GroupMember = *GroupMemberContext.State;

					FTraitStackBinding& MemberTraitStack = GroupMemberContext.TraitStack;

					Context.BindTo(GroupMember.TraitPtr);
					ensure(Context.GetStack(GroupMember.TraitPtr, MemberTraitStack));
					ensure(MemberTraitStack.GetInterface(GroupMemberContext.TimelineTrait));
				}
			}

			GroupContext.bCanGroupUseMarkerSyncing = bCanGroupUseMarkerSyncing;
		}

		// Returns the sync marker index immediately after the specified position
		static int32 GetNextSyncMarkerAt(const FTimelineSyncMarkerArray& SyncMarkers, float Position)
		{
			// Returns the index of the first sync marker greater than the specified position
			return Algo::UpperBoundBy(SyncMarkers, Position, [](const FTimelineSyncMarker& Marker) { return Marker.GetPosition(); });
		}

		// Returns the valid pair of sync markers around the specified timeline position
		// This can return the animation boundary if the timeline isn't looping
		static void GetNearestSyncMarkersAt(const TSet<FName>& ValidMarkers, const FTimelineSyncMarkerArray& SyncMarkers, const FTimelineState& TimelineState, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker)
		{
			const float CurrentTime = TimelineState.GetPosition();
			const float Duration = TimelineState.GetDuration();

			// Pick a guess to start our search
			int32 NextMarkerIndex = GetNextSyncMarkerAt(SyncMarkers, CurrentTime);
			int32 PrevMarkerIndex = NextMarkerIndex >= 0 ? (NextMarkerIndex - 1) : INDEX_NONE;

			float PrevTimeToMarker = 0.0f;
			float NextTimeToMarker = 0.0f;

			const bool bIsLooping = TimelineState.IsLooping();

			// Handle previous marker
			{
				// Refine our search using the valid markers
				while (PrevMarkerIndex >= 0)
				{
					if (ValidMarkers.Contains(SyncMarkers[PrevMarkerIndex].GetName()))
					{
						// This marker is valid, use it
						break;
					}

					PrevMarkerIndex--;
				}

				// If we reached the start of the marker list
				if (PrevMarkerIndex == INDEX_NONE)
				{
					if (bIsLooping)
					{
						PrevMarkerIndex = SyncMarkers.Num() - 1;

						while (PrevMarkerIndex >= 0)
						{
							if (ValidMarkers.Contains(SyncMarkers[PrevMarkerIndex].GetName()))
							{
								// This marker is valid, use it
								break;
							}

							PrevMarkerIndex--;
						}

						check(PrevMarkerIndex != INDEX_NONE);

						// Marker lives in previous loop iteration, in the 'past'
						const float MarkerTime = SyncMarkers[PrevMarkerIndex].GetPosition() - Duration;
						PrevTimeToMarker = MarkerTime - CurrentTime;
					}
					else
					{
						PrevMarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
						PrevTimeToMarker = CurrentTime;
					}
				}
				else
				{
					PrevTimeToMarker = SyncMarkers[PrevMarkerIndex].GetPosition() - CurrentTime;
				}
			}

			// Handle next marker
			{
				// Refine our search using the valid markers
				while (NextMarkerIndex < SyncMarkers.Num())
				{
					if (ValidMarkers.Contains(SyncMarkers[NextMarkerIndex].GetName()))
					{
						// This marker is valid, use it
						break;
					}

					NextMarkerIndex++;
				}

				// If we reached the end of the marker list
				if (NextMarkerIndex == SyncMarkers.Num())
				{
					if (bIsLooping)
					{
						NextMarkerIndex = 0;

						while (NextMarkerIndex < SyncMarkers.Num())
						{
							if (ValidMarkers.Contains(SyncMarkers[NextMarkerIndex].GetName()))
							{
								// This marker is valid, use it
								break;
							}

							NextMarkerIndex++;
						}

						check(NextMarkerIndex != SyncMarkers.Num());

						// Marker lives in next loop iteration, in the 'future'
						const float MarkerTime = SyncMarkers[NextMarkerIndex].GetPosition() + Duration;
						NextTimeToMarker = MarkerTime - CurrentTime;
					}
					else
					{
						NextMarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
						NextTimeToMarker = CurrentTime;
					}
				}
				else
				{
					NextTimeToMarker = SyncMarkers[NextMarkerIndex].GetPosition() - CurrentTime;
				}
			}

			OutPrevMarker.MarkerIndex = PrevMarkerIndex;
			OutNextMarker.MarkerIndex = NextMarkerIndex;
		}

		static FName GetMarkerName(int32 MarkerIndex, const TArrayView<const FTimelineSyncMarker>& SyncMarkers)
		{
			return SyncMarkers.IsValidIndex(MarkerIndex) ? SyncMarkers[MarkerIndex].GetName() : NAME_None;
		}

		static FName GetMarkerName(int32 MarkerIndex, const FTimelineSyncMarkerArray& SyncMarkers)
		{
			return SyncMarkers.IsValidIndex(MarkerIndex) ? SyncMarkers[MarkerIndex].GetName() : NAME_None;
		}

		static FName GetMarkerName(const FMarkerPair& Marker, const FTimelineSyncMarkerArray& SyncMarkers)
		{
			return GetMarkerName(Marker.MarkerIndex, SyncMarkers);
		}

		static float GetMarkerPosition(int32 MarkerIndex, const FTimelineSyncMarkerArray& SyncMarkers, float FallbackValue)
		{
			return SyncMarkers.IsValidIndex(MarkerIndex) ? SyncMarkers[MarkerIndex].GetPosition() : FallbackValue;
		}

		static float GetMarkerPosition(const FMarkerPair& Marker, const FTimelineSyncMarkerArray& SyncMarkers, float FallbackValue)
		{
			return GetMarkerPosition(Marker.MarkerIndex, SyncMarkers, FallbackValue);
		}

		// Returns the closest pair of sync markers to the specified timeline position that matches exactly the supplied sync position
		static void GetExactBoundarySyncMarkersAt(
			const FSyncGroupPhasePosition& LeaderPhasePosition,
			const FTimelineSyncMarkerArray& FollowerSyncMarkers, const FTimelineState& FollowerTimelineState,
			FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker)
		{
			const float CurrentTime = FollowerTimelineState.GetPosition();
			const float Duration = FollowerTimelineState.GetDuration();
			const bool bIsLooping = FollowerTimelineState.IsLooping();
			const int32 NumSyncMarkers = FollowerSyncMarkers.Num();

			float BestTimeDelta = FLT_MAX;

			// Handle case for looping and sync position not being on either boundary.
			for (int32 PrevMarkerIdx = 0; PrevMarkerIdx < NumSyncMarkers; ++PrevMarkerIdx)
			{
				const FTimelineSyncMarker& PrevMarker = FollowerSyncMarkers[PrevMarkerIdx];

				if (PrevMarker.GetName() != LeaderPhasePosition.PrevMarkerName)
				{
					continue;	// Not matching, look for the next pair candidate
				}

				const float PrevMarkerTime = PrevMarker.GetPosition();
				const int32 EndMarkerSearchStart = PrevMarkerIdx + 1;
				const int32 EndCount = bIsLooping ? NumSyncMarkers + EndMarkerSearchStart : NumSyncMarkers;

				for (int32 NextMarkerCount = EndMarkerSearchStart; NextMarkerCount < EndCount; ++NextMarkerCount)
				{
					const int32 NextMarkerIdx = NextMarkerCount % NumSyncMarkers;
					const FTimelineSyncMarker& NextMarker = FollowerSyncMarkers[NextMarkerIdx];

					// We have matched the position's next marker name.
					if (NextMarker.GetName() != LeaderPhasePosition.NextMarkerName)
					{
						continue;	// Not matching, look for the next pair candidate
					}

					float NextMarkerTime = NextMarker.GetPosition();

					// Handle case where we need to loop to get to be able to get to the next marker.
					bool bLooped = false;
					if (NextMarkerTime < PrevMarkerTime)
					{
						NextMarkerTime += Duration;
						bLooped = true;
					}

					// Get current time based of sync position.
					float DesiredTime = FMath::LerpStable(PrevMarkerTime, NextMarkerTime, LeaderPhasePosition.PositionBetweenMarkers);

					// Find marker indices closest to input time position.
					float TimeDelta = FMath::Abs(DesiredTime - CurrentTime);
					if (TimeDelta < BestTimeDelta)
					{
						BestTimeDelta = TimeDelta;
						OutPrevMarker.MarkerIndex = PrevMarkerIdx;
						OutNextMarker.MarkerIndex = NextMarkerIdx;
					}
					else if (bLooped)
					{
						// If we looped, we extended our next marker past the end of the sequence
						// This means that there are two points we need to test:
						//   - The one that lands near the end of the sequence (possibly overshooting/looping around)
						//   - The one that lands near the start of the sequence (possibly undershooting/looping around)
						// 
						// We tested the first one above, now test the second
						DesiredTime -= Duration;

						TimeDelta = FMath::Abs(DesiredTime - CurrentTime);
						if (TimeDelta < BestTimeDelta)
						{
							BestTimeDelta = TimeDelta;
							OutPrevMarker.MarkerIndex = PrevMarkerIdx;
							OutNextMarker.MarkerIndex = NextMarkerIdx;
						}
					}

					// This marker test is done, move onto next one.
					break;
				}

				// If we get here and we haven't found a match and we are not looping then there
				// is no point running the rest of the loop set up something as relevant as we can and carry on
				if (OutPrevMarker.MarkerIndex == MarkerIndexSpecialValues::Uninitialized)
				{
					// Find nearest previous marker that is earlier than our current time
					BestTimeDelta = CurrentTime - PrevMarkerTime;
					int32 PrevMarkerToUse = PrevMarkerIdx + 1;
					while (BestTimeDelta > 0.f && PrevMarkerToUse < NumSyncMarkers)
					{
						BestTimeDelta = CurrentTime - FollowerSyncMarkers[PrevMarkerToUse].GetPosition();
						++PrevMarkerToUse;
					}

					OutPrevMarker.MarkerIndex = PrevMarkerToUse - 1;	// We always go one past the marker we actually want to use

					// This goes to minus one as the very fact we are here means
					// that there is no next marker to use
					OutNextMarker.MarkerIndex = -1;

					break; // no need to keep searching, we are done
				}
			}
		}

		// Returns the boundary sync markers matching the sync position pair nearest the specified timeline position and the adjusted current time based on the sync position
		static void GetNearestMatchingBoundarySyncMarkers(
			const FSyncGroupPhasePosition& LeaderPhasePosition,
			const FTimelineSyncMarkerArray& FollowerSyncMarkers, const FTimelineState& FollowerTimelineState,
			FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker)
		{
			check(LeaderPhasePosition.PrevMarkerName != NAME_None || LeaderPhasePosition.NextMarkerName != NAME_None);

			// If the sync position's previous marker is the start boundary
			//		- We keep the previous marker at the boundary
			//		- We look for the next marker, the first instance of the next marker's name
			//		- We return the current time based on the sync position (see GetCurrentTimeFromMarkers)
			//		- Done
			if (LeaderPhasePosition.PrevMarkerName == NAME_None)
			{
				OutPrevMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;

				for (int32 MarkerIndex = 0; MarkerIndex < FollowerSyncMarkers.Num(); ++MarkerIndex)
				{
					if (FollowerSyncMarkers[MarkerIndex].GetName() == LeaderPhasePosition.NextMarkerName)
					{
						OutNextMarker.MarkerIndex = MarkerIndex;
						break;
					}
				}

				return;
			}

			// If the sync position's next marker is the end boundary
			//		- We keep the next marker at the boundary
			//		- We look for the previous marker, the last instance of the previous marker's name
			//		- We return the current time based on the sync position (see GetCurrentTimeFromMarkers)
			//		- Done
			if (LeaderPhasePosition.NextMarkerName == NAME_None)
			{
				OutNextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;

				for (int32 MarkerIndex = FollowerSyncMarkers.Num() - 1; MarkerIndex >= 0; --MarkerIndex)
				{
					if (FollowerSyncMarkers[MarkerIndex].GetName() == LeaderPhasePosition.PrevMarkerName)
					{
						OutPrevMarker.MarkerIndex = MarkerIndex;
						break;
					}
				}

				return;
			}

			// Otherwise
			//		- Scan the marker pairs that match the previous/next markers from the leader
			//		- We look for the pair that yields an adjusted current time closest to the follower's current time
			//		- Once we find that pair, we return the current time based on the sync position (see GetCurrentTimeFromMarkers)
			//		- If no such pair is found, the follower must not contain the pair of markers present on the leader (not possible)
			//		- Done
			GetExactBoundarySyncMarkersAt(LeaderPhasePosition, FollowerSyncMarkers, FollowerTimelineState, OutPrevMarker, OutNextMarker);
		}

		// Returns the relative sync position between the two specified markers
		static FSyncGroupPhasePosition CalculateSyncPosition(const FTimelineSyncMarkerArray& SyncMarkers, const FTimelineState& TimelineState, const FMarkerPair& PrevMarker, const FMarkerPair& NextMarker)
		{
			const float Duration = TimelineState.GetDuration();
			float CurrentTime = TimelineState.GetPosition();

			FSyncGroupPhasePosition PhasePosition;
			float PrevTime;
			float NextTime;

			// Get previous marker's time and name.
			if (PrevMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary && ensureAlwaysMsgf(SyncMarkers.IsValidIndex(PrevMarker.MarkerIndex),
				TEXT("MarkerCount: %d, PrevMarker : %d, NextMarker: %d, CurrentTime : %0.2f"), SyncMarkers.Num(), PrevMarker.MarkerIndex, NextMarker.MarkerIndex, CurrentTime))
			{
				PrevTime = SyncMarkers[PrevMarker.MarkerIndex].GetPosition();
				PhasePosition.PrevMarkerIndex = PrevMarker.MarkerIndex;
				PhasePosition.PrevMarkerName = SyncMarkers[PrevMarker.MarkerIndex].GetName();
			}
			else
			{
				PrevTime = 0.0f;
				PhasePosition.PrevMarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
				PhasePosition.PrevMarkerName = NAME_None;
			}

			// Get next marker's time and name.
			if (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary && ensureAlwaysMsgf(SyncMarkers.IsValidIndex(NextMarker.MarkerIndex),
				TEXT("MarkerCount: %d, PrevMarker : %d, NextMarker: %d, CurrentTime : %0.2f"), SyncMarkers.Num(), PrevMarker.MarkerIndex, NextMarker.MarkerIndex, CurrentTime))
			{
				NextTime = SyncMarkers[NextMarker.MarkerIndex].GetPosition();
				PhasePosition.NextMarkerIndex = NextMarker.MarkerIndex;
				PhasePosition.NextMarkerName = SyncMarkers[NextMarker.MarkerIndex].GetName();
			}
			else
			{
				NextTime = Duration;
				PhasePosition.NextMarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
				PhasePosition.NextMarkerName = NAME_None;
			}

			if (PrevTime > NextTime)
			{
				// We looped, account for it
				PrevTime = PrevTime > CurrentTime ? (PrevTime - Duration) : PrevTime;
				NextTime = NextTime < CurrentTime ? (NextTime + Duration) : NextTime;
			}
			else if (PrevTime == NextTime)
			{
				// If the two markers overlap, space them apart by the duration to ensure the time between them isn't zero
				PrevTime -= Duration;
			}

			if (PrevTime > CurrentTime)
			{
				// Our current time isn't between the markers, we looped
				CurrentTime += Duration;
			}

			ensure(NextTime > PrevTime);
			const float TimeBetweenMarkers = FMath::Max(NextTime - PrevTime, SMALL_NUMBER);

			// Store the encoded current time position as a ratio between markers
			const float PositionBetweenMarkers = (CurrentTime - PrevTime) / TimeBetweenMarkers;
			ensure(PositionBetweenMarkers >= (0.0f - SMALL_NUMBER) && PositionBetweenMarkers <= (1.0f + SMALL_NUMBER));

			PhasePosition.PositionBetweenMarkers = FMath::Clamp(PositionBetweenMarkers, 0.0f, 1.0f);
			return PhasePosition;
		}

		// Returns the desired time based on the provided markers and normalized position between
		static float CalculateTimeFromSyncPosition(const FTimelineSyncMarkerArray& SyncMarkers, const FTimelineState& TimelineState, const FMarkerPair& PrevMarker, const FMarkerPair& NextMarker, float NormalizedPosition)
		{
			const float CurrentTime = TimelineState.GetPosition();
			const float Duration = TimelineState.GetDuration();

			float PrevMarkerTime = GetMarkerPosition(PrevMarker, SyncMarkers, 0.0f);
			float NextMarkerTime = GetMarkerPosition(NextMarker, SyncMarkers, Duration);

			if (PrevMarkerTime >= NextMarkerTime)
			{
				// We are looping around, fixup the previous marker to come before the next marker
				// extending it past the start of the timeline (negative position)
				PrevMarkerTime -= Duration;
			}

			float DesiredTime = FMath::LerpStable(PrevMarkerTime, NextMarkerTime, NormalizedPosition);

			if (DesiredTime < 0.0f)
			{
				// The markers are looping around but the desired time hasn't looped yet, wrap back
				DesiredTime += Duration;
			}

			return FMath::Clamp(DesiredTime, 0.0f, Duration);
		}

		static void AdvanceAndCollectSyncMarkersPassed(
			const FTimelineState& TimelineState, float DeltaTime, bool bIsPlayingForward,
			const FTimelineSyncMarkerArray& SyncMarkers, FMarkerPair& PrevMarker, FMarkerPair& NextMarker,
			TArray<FName, TMemStackAllocator<>>& OutMarkersPassed)
		{
			OutMarkersPassed.Reset();

			const float Duration = TimelineState.GetDuration();
			const bool bIsLooping = TimelineState.IsLooping();

			// Treat delta time as being positive since we don't know if we move backwards because of the delta time (e.g. game rewind)
			// or if it's because of the play rate
			float RemainingDeltaTime = FMath::Abs(DeltaTime);
			float CurrentTime = TimelineState.GetPosition();

			if (bIsPlayingForward)
			{
				// Progressively consume our delta time
				while (RemainingDeltaTime > 0.0f)
				{
					// Our next marker is the end boundary. (Only possible if sequence is not looping)
					if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
					{
						break;
					}

					const FTimelineSyncMarker& NextSyncMarker = SyncMarkers[NextMarker.MarkerIndex];

					float NextMarkerTime;
					if (CurrentTime <= NextSyncMarker.GetPosition())
					{
						// Next marker is ahead of us, use it as is
						NextMarkerTime = NextSyncMarker.GetPosition();
					}
					else
					{
						// We are looping
						check(bIsLooping);
						NextMarkerTime = NextSyncMarker.GetPosition() + Duration;
					}

					const float TargetTime = CurrentTime + RemainingDeltaTime;
					if (TargetTime > NextMarkerTime)
					{
						// We passed this marker
						OutMarkersPassed.Add(NextSyncMarker.GetName());

						// Update our marker tracking
						PrevMarker = NextMarker;
						NextMarker.MarkerIndex++;
						if (NextMarker.MarkerIndex >= SyncMarkers.Num())
						{
							NextMarker.MarkerIndex = bIsLooping ? 0 : MarkerIndexSpecialValues::AnimationBoundary;
						}

						// Update our time tracking
						const float ConsumedTime = NextMarkerTime - CurrentTime;
						CurrentTime = NextSyncMarker.GetPosition();
						RemainingDeltaTime -= ConsumedTime;
					}
					else
					{
						break;
					}
				}
			}
			else
			{
				// Progressively consume our delta time
				while (RemainingDeltaTime > 0.0f)
				{
					// Our next marker is the end boundary. (Only possible if sequence is not looping)
					if (PrevMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
					{
						break;
					}

					const FTimelineSyncMarker& PrevSyncMarker = SyncMarkers[PrevMarker.MarkerIndex];

					float PrevMarkerTime;
					if (CurrentTime >= PrevSyncMarker.GetPosition())
					{
						// Previous marker is ahead of us, use it as is
						PrevMarkerTime = PrevSyncMarker.GetPosition();
					}
					else
					{
						// We are looping
						check(bIsLooping);
						PrevMarkerTime = PrevSyncMarker.GetPosition() - Duration;
					}

					const float TargetTime = CurrentTime - RemainingDeltaTime;
					if (TargetTime < PrevMarkerTime)
					{
						// We passed this marker
						OutMarkersPassed.Add(PrevSyncMarker.GetName());

						// Update our marker tracking
						NextMarker = PrevMarker;
						PrevMarker.MarkerIndex--;
						if (PrevMarker.MarkerIndex == INDEX_NONE)
						{
							PrevMarker.MarkerIndex = bIsLooping ? (SyncMarkers.Num() - 1) : MarkerIndexSpecialValues::AnimationBoundary;
						}

						// Update our time tracking
						const float ConsumedTime = PrevMarkerTime - CurrentTime;
						CurrentTime = PrevSyncMarker.GetPosition();
						RemainingDeltaTime += ConsumedTime;
					}
					else
					{
						break;
					}
				}
			}
		}

		static void SeekMarkerForward(int32& MarkerIndex, bool bIsLooping, FName TargetMarkerName, const FTimelineSyncMarkerArray& SyncMarkers)
		{
			const int32 NumSyncMarkers = SyncMarkers.Num();

			int32 MaxNumIterations = NumSyncMarkers;
			while (MarkerIndex < NumSyncMarkers && MaxNumIterations > 0)
			{
				if (SyncMarkers[MarkerIndex].GetName() == TargetMarkerName)
				{
					// This is the marker we passed
					break;
				}

				MarkerIndex++;
				MaxNumIterations--;

				if (MarkerIndex == NumSyncMarkers)
				{
					if (bIsLooping)
					{
						MarkerIndex = 0;
					}
					else
					{
						MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
						break;
					}
				}
			}
		}

		static void SeekMarkerBackward(int32& MarkerIndex, bool bIsLooping, FName TargetMarkerName, const FTimelineSyncMarkerArray& SyncMarkers)
		{
			const int32 NumSyncMarkers = SyncMarkers.Num();

			int32 MaxNumIterations = NumSyncMarkers;
			while (MarkerIndex >= 0 && MaxNumIterations > 0)
			{
				if (SyncMarkers[MarkerIndex].GetName() == TargetMarkerName)
				{
					// This is the marker we passed
					break;
				}

				MarkerIndex--;
				MaxNumIterations--;

				if (MarkerIndex < 0)
				{
					if (bIsLooping)
					{
						MarkerIndex = NumSyncMarkers - 1;
					}
					else
					{
						MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
						break;
					}
				}
			}
		}

		static float AdvanceSyncMarkersPassed(
			const FTimelineState& TimelineState, bool bIsPlayingForward,
			const FSyncGroupPhasePosition& LeaderEndPosition,
			const FTimelineSyncMarkerArray& SyncMarkers, const TArray<FName, TMemStackAllocator<>>& MarkersPassed,
			FMarkerPair& PrevMarker, FMarkerPair& NextMarker)
		{
			const bool bIsLooping = TimelineState.IsLooping();
			const int32 NumMarkersPassed = MarkersPassed.Num();

			if (bIsPlayingForward)
			{
				if (NumMarkersPassed > 0)
				{
					// Skip over the markers that passed
					for (int32 PassedMarkerIndex = 0; PassedMarkerIndex < NumMarkersPassed; ++PassedMarkerIndex)
					{
						if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
						{
							// We still have markers left to pass but the follower ran out of markers
							break;
						}

						PrevMarker = NextMarker;

						const FName PassedMarker = MarkersPassed[PassedMarkerIndex];

						// Look for the instance of our passed marker
						SeekMarkerForward(NextMarker.MarkerIndex, bIsLooping, PassedMarker, SyncMarkers);
					}

					if (LeaderEndPosition.NextMarkerName == NAME_None)
					{
						// If our leader has reached the end boundary, make sure we reach it as well
						NextMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
					}
					else if (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary && NumMarkersPassed > 0)
					{
						PrevMarker = NextMarker;

						// Find the next marker to match our leader
						SeekMarkerForward(NextMarker.MarkerIndex, bIsLooping, LeaderEndPosition.NextMarkerName, SyncMarkers);
					}
				}

				if (NextMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
				{
					check(SyncMarkers[NextMarker.MarkerIndex].GetName() == LeaderEndPosition.NextMarkerName);
				}

				return CalculateTimeFromSyncPosition(SyncMarkers, TimelineState, PrevMarker, NextMarker, LeaderEndPosition.PositionBetweenMarkers);
			}
			else
			{
				if (NumMarkersPassed > 0)
				{
					// Skip over the markers that passed
					for (int32 PassedMarkerIndex = 0; PassedMarkerIndex < NumMarkersPassed; ++PassedMarkerIndex)
					{
						if (PrevMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
						{
							// We still have markers left to pass but the follower ran out of markers
							break;
						}

						NextMarker = PrevMarker;

						const FName PassedMarker = MarkersPassed[PassedMarkerIndex];

						// Look for the instance of our passed marker
						SeekMarkerBackward(PrevMarker.MarkerIndex, bIsLooping, PassedMarker, SyncMarkers);
					}

					if (LeaderEndPosition.PrevMarkerName == NAME_None)
					{
						// If our leader has reached the end boundary, make sure we reach it as well
						PrevMarker.MarkerIndex = MarkerIndexSpecialValues::AnimationBoundary;
					}
					else if (PrevMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary && NumMarkersPassed > 0)
					{
						NextMarker = PrevMarker;

						// Find the previous marker to match our leader
						SeekMarkerBackward(PrevMarker.MarkerIndex, bIsLooping, LeaderEndPosition.PrevMarkerName, SyncMarkers);
					}
				}

				if (PrevMarker.MarkerIndex != MarkerIndexSpecialValues::AnimationBoundary)
				{
					check(SyncMarkers[PrevMarker.MarkerIndex].GetName() == LeaderEndPosition.PrevMarkerName);
				}

				return CalculateTimeFromSyncPosition(SyncMarkers, TimelineState, PrevMarker, NextMarker, LeaderEndPosition.PositionBetweenMarkers);
			}
		}

		static int32 GetNumLoopingMarkers(const FTimelineState& TimelineState, const TArrayView<const FTimelineSyncMarker>& SyncMarkers)
		{
			int32 NumSyncMarkers = SyncMarkers.Num();

			if (NumSyncMarkers != 0)
			{
				// Determine if our first and last markers match and repeat on the first/last keyframes in which case we don't have a phase in between

				const FTimelineSyncMarker& FirstMarker = SyncMarkers[0];
				const FTimelineSyncMarker& LastMarker = SyncMarkers.Last();
				if (FirstMarker.GetName() == LastMarker.GetName())
				{
					if (FirstMarker.GetPosition() == 0.0f && LastMarker.GetPosition() == TimelineState.GetDuration())
					{
						// We perfectly overlap, ignore our last marker
						NumSyncMarkers--;
					}
				}
			}

			return NumSyncMarkers;
		}

		// Returns whether or not two members have phases that can be matched
		static bool CanMatchPhase(
			const FTimelineState& TimelineStateA, const TArrayView<const FTimelineSyncMarker>& SyncMarkersA,
			const FTimelineState& TimelineStateB, const TArrayView<const FTimelineSyncMarker>& SyncMarkersB)
		{
			// TODO: We can cache this value once we've computed this between a follower and leader
			// We only need to recompute this if the leader changes or when joining

			// Identify the number of markers contained by a phase cycle
			// We only support phase matching if our cycles match
			// Imagine we have a marker sequence with: RLRL
			// Our phase cycle is: RL
			// We only consider simple cycles where the first marker matches the last and isn't present within the cycle
			// e.g. a cycle cannot contain the first marker, RLR would not be valid for example as part of RLRRLR
			// 
			// Once we've identified our cycle, we can determine whether or not two clips can phase match
			// There are 3 cases to consider:
			//   - Neither clip loops
			//   - One clip loops and the other doesn't
			//   - Both clips loop
			// 
			// We also have to consider looping clips where the last keyframe matches the first (we don't interpolate between)
			// This occurs and is necessary when root motion is used which means the first marker can be repeated: RLRLR

			if (!TimelineStateA.IsLooping() && !TimelineStateB.IsLooping())
			{
				// Neither are looping
				// Both must have a matching number of cycles(we don't support partial matching, e.g. RLRL and RLRLRL matching
				// in the first portion since the first would 'freeze' once it reaches the last marker if it doesn't loop)
				// Here we can also support partial cycles as long as both match (e.g. RLR and RLR)

				const int32 NumSyncMarkersA = SyncMarkersA.Num();
				const int32 NumSyncMarkersB = SyncMarkersB.Num();

				if (NumSyncMarkersA != NumSyncMarkersB)
				{
					return false;	// Cannot match if our marker count differs
				}

				if (NumSyncMarkersA == 0)
				{
					return false;	// Cannot match if we have no markers
				}

				for (int32 MarkerIdx = 0; MarkerIdx < NumSyncMarkersA; ++MarkerIdx)
				{
					if (SyncMarkersA[MarkerIdx].GetName() != SyncMarkersB[MarkerIdx].GetName())
					{
						// Markers do not match, we cannot phase match
						return false;
					}
				}

				// All our markers match
				return true;
			}
			else if (TimelineStateA.IsLooping() && TimelineStateB.IsLooping())
			{
				// Both are looping
				// They can phase match as long as the marker sequence loops properly over their combined length
				// If we have RL and RLRL, our combined length is our greatest common divisor: 4 markers
				// We'd get: RL RL and RLRL
				// If we have RLRL and RLRLRL, our combined length is our greatest common divisor: 12 markers
				// We'd get: RLRL RLRL RLRL and RLRLRL RLRLRL

				const int32 NumSyncMarkersA = GetNumLoopingMarkers(TimelineStateA, SyncMarkersA);
				const int32 NumSyncMarkersB = GetNumLoopingMarkers(TimelineStateB, SyncMarkersB);

				if (NumSyncMarkersA == 0 || NumSyncMarkersB == 0)
				{
					return false;	// Cannot match if we have no markers
				}

				const int32 NumTargetMarkers = FMath::GreatestCommonDivisor(NumSyncMarkersA, NumSyncMarkersB);
				for (int32 TargetMarkerIndex = 0; TargetMarkerIndex < NumTargetMarkers; ++TargetMarkerIndex)
				{
					const int32 MarkerIndexA = TargetMarkerIndex % NumSyncMarkersA;
					const int32 MarkerIndexB = TargetMarkerIndex % NumSyncMarkersB;

					if (SyncMarkersA[MarkerIndexA].GetName() != SyncMarkersB[MarkerIndexB].GetName())
					{
						// Markers do not match, we cannot phase match
						return false;
					}
				}

				// Markers are perfectly in sync
				return true;
			}
			else
			{
				// One of them is looping
				// The non-looping clip must have a matching multiple number of cycles meaning the looping
				// clip must fully loop a number of times (not partially) (e.g. RLRL (looping) and RLRLRL cannot phase match but RLRL (looping) and RLRLRLRL can)
				// This is because we do not track how many times we've looped and so if the shorter sequence is leading, we do not know how far into the second
				// sequence we should be which is problematic if the short leading sequence is the one looping

				const FTimelineState& LoopingTimelineState = TimelineStateA.IsLooping() ? TimelineStateA : TimelineStateB;
				const TArrayView<const FTimelineSyncMarker>& LoopingSyncMarkersList = TimelineStateA.IsLooping() ? SyncMarkersA : SyncMarkersB;
				const TArrayView<const FTimelineSyncMarker>& NonLoopingSyncMarkersList = TimelineStateA.IsLooping() ? SyncMarkersB : SyncMarkersA;

				const int32 NumLoopingMarkers = GetNumLoopingMarkers(LoopingTimelineState, LoopingSyncMarkersList);
				const int32 NumNonLoopingMarkers = NonLoopingSyncMarkersList.Num();

				if (NumNonLoopingMarkers == 0 || NumLoopingMarkers == 0)
				{
					return false;	// Cannot match if we have no markers
				}

				if ((NumNonLoopingMarkers % NumLoopingMarkers) != 0)
				{
					// Our looping clip isn't fully looping, we don't handle partial looping
					return false;
				}

				for (int32 NonLoopingMarkerIdx = 0; NonLoopingMarkerIdx < NumNonLoopingMarkers; ++NonLoopingMarkerIdx)
				{
					const int32 LoopingMarkerIndex = NonLoopingMarkerIdx % NumLoopingMarkers;

					if (NonLoopingSyncMarkersList[NonLoopingMarkerIdx].GetName() != LoopingSyncMarkersList[LoopingMarkerIndex].GetName())
					{
						// Markers do not match, we cannot phase match
						return false;
					}
				}

				// Markers are perfectly in sync
				return true;
			}
		}

		// Returns the phase position for B that matches the current phase on A
		static FSyncGroupPhasePosition FindMatchingPhaseTime(
			const FTimelineState& TimelineStateA, const TArrayView<const FTimelineSyncMarker>& SyncMarkersA, const FSyncGroupPhasePosition& SyncPositionA,
			const FTimelineState& TimelineStateB, const FTimelineSyncMarkerArray& SyncMarkersB)
		{
			// If we get here, we know we can phase match, @see CanMatchPhase above for details

			int32 PrevMarkerIndex;
			int32 NextMarkerIndex;

			if (!TimelineStateA.IsLooping() && !TimelineStateB.IsLooping())
			{
				// Neither are looping
				check(SyncMarkersA.Num() == SyncMarkersB.Num());	// Invariant enforced in CanMatchPhase

				PrevMarkerIndex = SyncPositionA.PrevMarkerIndex;
				NextMarkerIndex = SyncPositionA.NextMarkerIndex;
			}
			else if (TimelineStateA.IsLooping() && TimelineStateB.IsLooping())
			{
				// Both are looping
				const int32 NumSyncMarkersA = GetNumLoopingMarkers(TimelineStateA, SyncMarkersA);
				const int32 NumSyncMarkersB = GetNumLoopingMarkers(TimelineStateB, SyncMarkersB);

				check(NumSyncMarkersA != 0 && NumSyncMarkersB != 0);	// Invariant enforced in CanMatchPhase

				const int32 NumTargetMarkers = FMath::GreatestCommonDivisor(NumSyncMarkersA, NumSyncMarkersB);

				// TODO: Because we don't keep track of how many loop iterations we've performed, this doesn't quite work
				//       If the short loop is leading, when it wraps around we'll cause the longer loop to wrap around as well

				PrevMarkerIndex = SyncPositionA.PrevMarkerIndex % NumTargetMarkers;
				NextMarkerIndex = SyncPositionA.NextMarkerIndex % NumTargetMarkers;
			}
			else
			{
				// One of them is looping
				const FTimelineState& LoopingTimelineState = TimelineStateA.IsLooping() ? TimelineStateA : TimelineStateB;
				const TArrayView<const FTimelineSyncMarker>& LoopingSyncMarkersList = TimelineStateA.IsLooping() ? SyncMarkersA : SyncMarkersB;
				const TArrayView<const FTimelineSyncMarker>& NonLoopingSyncMarkersList = TimelineStateA.IsLooping() ? SyncMarkersB : SyncMarkersA;

				const int32 NumLoopingMarkers = GetNumLoopingMarkers(LoopingTimelineState, LoopingSyncMarkersList);
				const int32 NumNonLoopingMarkers = NonLoopingSyncMarkersList.Num();

				check(NumNonLoopingMarkers != 0 && NumLoopingMarkers != 0);	// Invariant enforced in CanMatchPhase
				check((NumNonLoopingMarkers % NumLoopingMarkers) == 0);		// Invariant enforced in CanMatchPhase

				// TODO: Because we don't keep track of how many loop iterations we've performed, this doesn't quite work
				//       If the short loop is leading, when it wraps around we'll cause the longer loop to wrap around as well

				const int32 NumTargetMarkers = FMath::GreatestCommonDivisor(NumLoopingMarkers, NumNonLoopingMarkers);

				PrevMarkerIndex = SyncPositionA.PrevMarkerIndex % NumTargetMarkers;
				NextMarkerIndex = SyncPositionA.NextMarkerIndex % NumTargetMarkers;
			}

			// Markers should match
			check(GetMarkerName(SyncPositionA.PrevMarkerIndex, SyncMarkersA) == GetMarkerName(PrevMarkerIndex, SyncMarkersB));
			check(GetMarkerName(SyncPositionA.NextMarkerIndex, SyncMarkersA) == GetMarkerName(NextMarkerIndex, SyncMarkersB));

			FSyncGroupPhasePosition PhasePosition = SyncPositionA;
			PhasePosition.PrevMarkerIndex = PrevMarkerIndex;
			PhasePosition.NextMarkerIndex = NextMarkerIndex;
			return PhasePosition;
		}

		static void AdvanceLeaderTimeBased(FExecutionContext& Context, FSyncGroupContext& GroupContext)
		{
			FSyncGroupMemberContext& GroupLeaderContext = GroupContext.Members[GroupContext.LeaderIndex];
			const FSyncGroupMember& GroupLeader = *GroupLeaderContext.State;

			FTraitStackBinding& LeaderTraitStack = GroupLeaderContext.TraitStack;

			TTraitBinding<IGroupSynchronization> GroupSyncTrait;

			Context.BindTo(GroupLeader.TraitPtr);
			ensure(Context.GetStack(GroupLeader.TraitPtr, LeaderTraitStack));
			ensure(LeaderTraitStack.GetInterface(GroupSyncTrait));

			// Cache our starting timeline state
			FTimelineState LeaderStartTimelineState;
			ensure(GroupLeaderContext.TimelineTrait.GetState(Context, LeaderStartTimelineState));

			const float PlayRate = LeaderStartTimelineState.GetPlayRate() != 0.0f ? LeaderStartTimelineState.GetPlayRate() : 1.0f;

			if (GroupLeaderContext.State->bJustJoined &&
				GroupLeaderContext.State->GroupParameters.bMatchSyncPoint &&
				!GroupContext.State->bJustFormed)
			{
				// This is a soft join where we wish to match the current sync group position
				const float CurrentTime = LeaderStartTimelineState.GetPosition();
				const float DesiredTime = GroupContext.PreviousLeaderTimelineState.GetPositionRatio() * LeaderStartTimelineState.GetDuration();

				// Seek where we should be without dispatching events
				const float DeltaTime = (DesiredTime - CurrentTime) / PlayRate;
				GroupSyncTrait.AdvanceBy(Context, DeltaTime, false);

				// Update our cached state
				ensure(GroupLeaderContext.TimelineTrait.GetState(Context, LeaderStartTimelineState));
			}

			// Record where the leader started from, followers that join the group will start there
			GroupContext.LeaderStartRatio = LeaderStartTimelineState.GetPositionRatio();

			// Compute our desired delta time, accounting for the play rate
			const float DeltaTime = GroupLeader.TraitState.GetDeltaTime();
			GroupSyncTrait.AdvanceBy(Context, DeltaTime, true);

			// Get our new state
			FTimelineState LeaderEndTimelineState;
			ensure(GroupLeaderContext.TimelineTrait.GetState(Context, LeaderEndTimelineState));

			GroupContext.LeaderTimelineState = LeaderEndTimelineState;
			GroupContext.bIsLeaderPlayingForward = (DeltaTime * PlayRate) >= 0.0f;
			GroupContext.LeaderEndRatio = LeaderEndTimelineState.GetPositionRatio();

#if UE_DEBUG_SYNC_GROUPS
			if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
			{
				FString DeltaTimeStr = FString::Printf(TEXT("%s %0.2f"),
					GroupContext.bIsLeaderPlayingForward ? TEXT("+") : TEXT("-"),
					DeltaTime * FMath::Abs(PlayRate));

				const bool bLooped = GroupContext.bIsLeaderPlayingForward ?
					(LeaderEndTimelineState.GetPosition() < LeaderStartTimelineState.GetPosition()) :
					(LeaderStartTimelineState.GetPosition() < LeaderEndTimelineState.GetPosition());

				UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Leader [Time] [%0.2f %s -> %0.2f / %0.2f (%3.2f%%)] Playing [%s]%s"),
					*GroupContext.State->GroupName.ToString(),
					GroupLeader.TraitPtr.GetNodeInstance(),
					LeaderStartTimelineState.GetPosition(), *DeltaTimeStr,
					LeaderEndTimelineState.GetPosition(), LeaderEndTimelineState.GetDuration(),
					LeaderEndTimelineState.GetPositionRatio() * 100.0f,
					*LeaderEndTimelineState.GetDebugName().ToString(),
					bLooped ? TEXT(" (looped)") : TEXT(""));
			}
#endif
		}

		static void ValidateLeaderMarkers(const FTimelineState& TimelineState, const FTimelineSyncMarkerArray& SyncMarkers, const FMarkerPair& PrevMarker, const FMarkerPair& NextMarker)
		{
			// Must have found some markers
			check(PrevMarker.MarkerIndex != MarkerIndexSpecialValues::Uninitialized);
			check(NextMarker.MarkerIndex != MarkerIndexSpecialValues::Uninitialized);

			if (PrevMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
			{
				// If previous is the boundary, then next must be the first marker
				check(NextMarker.MarkerIndex == 0);
			}
			else if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary)
			{
				// If next is the boundary, then previous must be the last marker
				check(PrevMarker.MarkerIndex == SyncMarkers.Num() - 1);
			}
			else
			{
				if (TimelineState.IsLooping())
				{
					// If previous is a valid marker, then next must be the following marker, optionally wrapping around
					check((PrevMarker.MarkerIndex + 1) % SyncMarkers.Num() == NextMarker.MarkerIndex);
				}
				else
				{
					// If previous is a valid marker, then next must be the following marker or the boundary if we run out
					check(PrevMarker.MarkerIndex + 1 == NextMarker.MarkerIndex);
				}
			}
		}

		// Ensures that the follower markers we found match the ones from the leader
		// They can mismatch if the leader/follower have mismatched looping state
		// Returns whether or not the markers were modified
		static bool SanitizeFollowerMarkers(
			const FSyncGroupPhasePosition& LeaderPhasePosition,
			const FTimelineSyncMarkerArray& FollowerSyncMarkers, bool bIsPlayingForward, bool bIsLooping,
			FMarkerPair& PrevMarker, FMarkerPair& NextMarker)
		{
			// Must have found some markers
			check(PrevMarker.MarkerIndex != MarkerIndexSpecialValues::Uninitialized);
			check(NextMarker.MarkerIndex != MarkerIndexSpecialValues::Uninitialized);

			bool bAppliedFixup = false;

			if (bIsPlayingForward)
			{
				if (GetMarkerName(PrevMarker, FollowerSyncMarkers) != LeaderPhasePosition.PrevMarkerName)
				{
					// Our previous marker doesn't match, look for the next one that matches
					SeekMarkerForward(PrevMarker.MarkerIndex, bIsLooping, LeaderPhasePosition.PrevMarkerName, FollowerSyncMarkers);

					// Fixup our next marker index
					NextMarker.MarkerIndex = PrevMarker.MarkerIndex + 1;
					if (NextMarker.MarkerIndex == FollowerSyncMarkers.Num())
					{
						NextMarker.MarkerIndex = bIsLooping ? 0 : MarkerIndexSpecialValues::AnimationBoundary;
					}

					bAppliedFixup = true;
				}

				if (GetMarkerName(NextMarker, FollowerSyncMarkers) != LeaderPhasePosition.NextMarkerName)
				{
					// Our next marker doesn't match, look for the next one that matches
					SeekMarkerForward(NextMarker.MarkerIndex, bIsLooping, LeaderPhasePosition.NextMarkerName, FollowerSyncMarkers);

					bAppliedFixup = true;
				}
			}
			else
			{
				if (GetMarkerName(NextMarker, FollowerSyncMarkers) != LeaderPhasePosition.NextMarkerName)
				{
					// Our next marker doesn't match, look for the next one that matches
					SeekMarkerBackward(NextMarker.MarkerIndex, bIsLooping, LeaderPhasePosition.NextMarkerName, FollowerSyncMarkers);

					// Fixup our previous marker index
					if (NextMarker.MarkerIndex == MarkerIndexSpecialValues::AnimationBoundary ||
						(bIsLooping && NextMarker.MarkerIndex == 0))
					{
						PrevMarker.MarkerIndex = FollowerSyncMarkers.Num() - 1;
					}
					else
					{
						PrevMarker.MarkerIndex = NextMarker.MarkerIndex - 1;
					}

					bAppliedFixup = true;
				}

				if (GetMarkerName(PrevMarker, FollowerSyncMarkers) != LeaderPhasePosition.PrevMarkerName)
				{
					// Our previous marker doesn't match, look for the next one that matches
					SeekMarkerBackward(PrevMarker.MarkerIndex, bIsLooping, LeaderPhasePosition.PrevMarkerName, FollowerSyncMarkers);

					bAppliedFixup = true;
				}
			}

			// Follower markers must only match in name, they might not be siblings if the pattern on the leader doesn't match the one on the follower
			// e.g. leader with RLRL and follower with RLLRL
			check(GetMarkerName(PrevMarker, FollowerSyncMarkers) == LeaderPhasePosition.PrevMarkerName);
			check(GetMarkerName(NextMarker, FollowerSyncMarkers) == LeaderPhasePosition.NextMarkerName);

			return bAppliedFixup;
		}

		static float CalculateElapsedTime(const FTimelineState& StartTimelineState, const FTimelineState& EndTimelineState, bool bIsPlayingForward)
		{
			if (bIsPlayingForward)
			{
				if (EndTimelineState.GetPosition() >= StartTimelineState.GetPosition())
				{
					return EndTimelineState.GetPosition() - StartTimelineState.GetPosition();
				}
				else
				{
					// Looped
					return (StartTimelineState.GetDuration() - StartTimelineState.GetPosition()) + EndTimelineState.GetPosition();
				}
			}
			else
			{
				if (EndTimelineState.GetPosition() <= StartTimelineState.GetPosition())
				{
					return EndTimelineState.GetPosition() - StartTimelineState.GetPosition();
				}
				else
				{
					// Looped
					return (EndTimelineState.GetPosition() - EndTimelineState.GetDuration()) - StartTimelineState.GetPosition();
				}
			}
		}

		static void AdvanceLeaderMarkerBased(FExecutionContext& Context, FSyncGroupContext& GroupContext)
		{
			FSyncGroupMemberContext& GroupLeaderContext = GroupContext.Members[GroupContext.LeaderIndex];
			const FSyncGroupMember& GroupLeader = *GroupLeaderContext.State;

			FTraitStackBinding& LeaderTraitStack = GroupLeaderContext.TraitStack;

			TTraitBinding<IGroupSynchronization> GroupSyncTrait;

			Context.BindTo(GroupLeader.TraitPtr);
			ensure(Context.GetStack(GroupLeader.TraitPtr, LeaderTraitStack));
			ensure(LeaderTraitStack.GetInterface(GroupSyncTrait));

			// Cache our starting timeline state
			FTimelineState LeaderStartTimelineState;
			ensure(GroupLeaderContext.TimelineTrait.GetState(Context, LeaderStartTimelineState));

			const float PlayRate = LeaderStartTimelineState.GetPlayRate() != 0.0f ? LeaderStartTimelineState.GetPlayRate() : 1.0f;

#if UE_DEBUG_SYNC_GROUPS
			FString JoiningMode;
#endif

			if (GroupLeaderContext.State->bJustJoined && !GroupContext.State->bJustFormed)
			{
				if (GroupLeaderContext.State->GroupParameters.bMatchSyncPoint)
				{
					const float CurrentTime = LeaderStartTimelineState.GetPosition();
					float DesiredTime;

					// Can we match the phase of the previous leader?
					const bool bCanMatchPhase = CanMatchPhase(
						GroupContext.State->PreviousLeaderTimelineState, GroupContext.State->PreviousLeaderSyncMarkers,
						LeaderStartTimelineState, GroupLeaderContext.SyncMarkers);

					if (bCanMatchPhase)
					{
						// When our phases can match, we simulate playback from the start of the timeline
						// This ensures a deterministic outcome no matter when a leader joins the group
						// We collect the sync markers of our previous leader between [StartPosition, CurrentPosition]
						// and we pass the same markers on our follower

						const FSyncGroupPhasePosition LeaderPhasePosition = FindMatchingPhaseTime(
							GroupContext.State->PreviousLeaderTimelineState, GroupContext.State->PreviousLeaderSyncMarkers, GroupContext.State->PreviousLeaderPhasePosition,
							LeaderStartTimelineState, GroupLeaderContext.SyncMarkers);

						const FMarkerPair PrevMarker(LeaderPhasePosition.PrevMarkerIndex, 0.0f);
						const FMarkerPair NextMarker(LeaderPhasePosition.NextMarkerIndex, 0.0f);

						DesiredTime = CalculateTimeFromSyncPosition(GroupLeaderContext.SyncMarkers, LeaderStartTimelineState, PrevMarker, NextMarker, LeaderPhasePosition.PositionBetweenMarkers);

#if UE_DEBUG_SYNC_GROUPS
						JoiningMode = TEXT(" (phase matched join)");
#endif
					}
					else
					{
						// If we can't match our phases then we attempt to find the closest matching marker
						// pair from our normalized position as dictated by the previous leader (e.g. if the
						// previous leader is at 80%, we look for the closest pair around the 80% mark on our follower)
						// Our current time is thus not relevant and we ignore it

						// We assume that we start at the same position in normalized time as our previous leader
						DesiredTime = GroupContext.PreviousLeaderTimelineState.GetPositionRatio() * LeaderStartTimelineState.GetDuration();

#if UE_DEBUG_SYNC_GROUPS
						JoiningMode = TEXT(" (relative matched join)");
#endif
					}

					// Seek where we should be without dispatching events
					const float DeltaTime = (DesiredTime - CurrentTime) / PlayRate;
					GroupSyncTrait.AdvanceBy(Context, DeltaTime, false);

					// Update our cached state
					ensure(GroupLeaderContext.TimelineTrait.GetState(Context, LeaderStartTimelineState));
				}
				else
				{
					// If we aren't requesting to match the sync point, then we'll use whatever marker pair
					// we currently lie between and the group will snap to us

#if UE_DEBUG_SYNC_GROUPS
					JoiningMode = TEXT(" (unmatched join)");
#endif
				}
			}

			// Find the sync markers around our current position
			FMarkerPair PrevMarkerStart;
			FMarkerPair NextMarkerStart;
			GetNearestSyncMarkersAt(GroupContext.ValidMarkers, GroupLeaderContext.SyncMarkers, LeaderStartTimelineState, PrevMarkerStart, NextMarkerStart);

			ValidateLeaderMarkers(LeaderStartTimelineState, GroupLeaderContext.SyncMarkers, PrevMarkerStart, NextMarkerStart);

			// Cache the start sync position
			GroupContext.LeaderPhaseStart = CalculateSyncPosition(GroupLeaderContext.SyncMarkers, LeaderStartTimelineState, PrevMarkerStart, NextMarkerStart);

			// Record where the leader started from, followers that join the group will start there
			GroupContext.LeaderStartRatio = LeaderStartTimelineState.GetPositionRatio();

			// We advance the leader by its desired delta time
			// Compute our desired delta time, accounting for the play rate
			const float DeltaTime = GroupLeader.TraitState.GetDeltaTime();
			GroupSyncTrait.AdvanceBy(Context, DeltaTime, true);

			// Compute our new timeline state
			FTimelineState LeaderEndTimelineState;
			ensure(GroupLeaderContext.TimelineTrait.GetState(Context, LeaderEndTimelineState));

			GroupContext.LeaderTimelineState = LeaderEndTimelineState;
			GroupContext.bIsLeaderPlayingForward = (DeltaTime * PlayRate) >= 0.0f;

			// Compute actual elapsed time
			// Can't use DeltaTime because it doesn't account for play rate and we might have floating point noise
			const float ElapsedTime = CalculateElapsedTime(LeaderStartTimelineState, LeaderEndTimelineState, GroupContext.bIsLeaderPlayingForward);

			// Advance and collect the markers we passed
			FMarkerPair PrevMarkerEnd = PrevMarkerStart;
			FMarkerPair NextMarkerEnd = NextMarkerStart;
			AdvanceAndCollectSyncMarkersPassed(
				LeaderStartTimelineState, ElapsedTime, GroupContext.bIsLeaderPlayingForward,
				GroupLeaderContext.SyncMarkers, PrevMarkerEnd, NextMarkerEnd, GroupContext.MarkersPassed);

			ValidateLeaderMarkers(LeaderEndTimelineState, GroupLeaderContext.SyncMarkers, PrevMarkerEnd, NextMarkerEnd);

			// Cache the end sync position
			GroupContext.LeaderPhaseEnd = CalculateSyncPosition(GroupLeaderContext.SyncMarkers, LeaderEndTimelineState, PrevMarkerEnd, NextMarkerEnd);
			GroupContext.LeaderEndRatio = LeaderEndTimelineState.GetPositionRatio();

#if DO_CHECK
			if (GroupContext.bIsLeaderPlayingForward)
			{
				if (PrevMarkerStart.MarkerIndex == PrevMarkerEnd.MarkerIndex)
				{
					check(NextMarkerStart.MarkerIndex == NextMarkerEnd.MarkerIndex);
					check(GroupContext.LeaderPhaseStart.PositionBetweenMarkers <= GroupContext.LeaderPhaseEnd.PositionBetweenMarkers);
				}
			}
			else
			{
				if (PrevMarkerStart.MarkerIndex == PrevMarkerEnd.MarkerIndex)
				{
					check(NextMarkerStart.MarkerIndex == NextMarkerEnd.MarkerIndex);
					check(GroupContext.LeaderPhaseStart.PositionBetweenMarkers >= GroupContext.LeaderPhaseEnd.PositionBetweenMarkers);
				}
			}
#endif

#if UE_DEBUG_SYNC_GROUPS
			if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
			{
				FString DeltaTimeStr = FString::Printf(TEXT("%s %0.2f"),
					GroupContext.bIsLeaderPlayingForward ? TEXT("+") : TEXT("-"),
					DeltaTime * FMath::Abs(PlayRate));

				const bool bLooped = GroupContext.bIsLeaderPlayingForward ?
					(LeaderEndTimelineState.GetPosition() < LeaderStartTimelineState.GetPosition()) :
					(LeaderStartTimelineState.GetPosition() < LeaderEndTimelineState.GetPosition());

				FString PassedMarkers;
				if (!GroupContext.MarkersPassed.IsEmpty())
				{
					TArray<FString> MarkersPassed;
					for (FName MarkerName : GroupContext.MarkersPassed)
					{
						MarkersPassed.Add(MarkerName.ToString());
					}

					PassedMarkers = FString::Printf(TEXT(" (passed [%s])"), *FString::Join(MarkersPassed, TEXT(", ")));
				}

				FString JoinedStatus;
				if (GroupLeaderContext.State->bJustJoined)
				{
					JoinedStatus = FString::Printf(TEXT(" (joined from %0.2f)"), LeaderStartTimelineState.GetPosition());
				}

				UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Leader [Mark] [%0.2f %s -> %0.2f / %0.2f (%3.2f%%)] [%s@%0.2f | %s@%0.2f (%3.2f%%) -> %s@%0.2f | %s@%0.2f (%3.2f%%)] Playing [%s]%s%s%s%s"),
					*GroupContext.State->GroupName.ToString(),
					GroupLeader.TraitPtr.GetNodeInstance(),
					LeaderStartTimelineState.GetPosition(), *DeltaTimeStr,
					LeaderEndTimelineState.GetPosition(), LeaderEndTimelineState.GetDuration(),
					LeaderEndTimelineState.GetPositionRatio() * 100.0f,
					*GroupContext.LeaderPhaseStart.PrevMarkerName.ToString(),
					GetMarkerPosition(PrevMarkerStart, GroupLeaderContext.SyncMarkers, 0.0f),
					*GroupContext.LeaderPhaseStart.NextMarkerName.ToString(),
					GetMarkerPosition(NextMarkerStart, GroupLeaderContext.SyncMarkers, LeaderEndTimelineState.GetDuration()),
					GroupContext.LeaderPhaseStart.PositionBetweenMarkers * 100.0f,
					*GroupContext.LeaderPhaseEnd.PrevMarkerName.ToString(),
					GetMarkerPosition(PrevMarkerEnd, GroupLeaderContext.SyncMarkers, 0.0f),
					*GroupContext.LeaderPhaseEnd.NextMarkerName.ToString(),
					GetMarkerPosition(NextMarkerEnd, GroupLeaderContext.SyncMarkers, LeaderEndTimelineState.GetDuration()),
					GroupContext.LeaderPhaseEnd.PositionBetweenMarkers * 100.0f,
					*LeaderEndTimelineState.GetDebugName().ToString(),
					bLooped ? TEXT(" (looped)") : TEXT(""), *PassedMarkers,
					*JoiningMode,
					*JoinedStatus);
			}
#endif
		}

		static void AdvanceLeader(FExecutionContext& Context, FSyncGroupContext& GroupContext)
		{
			bool bUseMarkerSyncing = GroupContext.bCanGroupUseMarkerSyncing;

#if UE_DEBUG_SYNC_GROUPS
			const int32 DebugSyncMode = CDebugVarSyncMode.GetValueOnAnyThread();
			if (DebugSyncMode == 1)
			{
				bUseMarkerSyncing = false;
			}
#endif

			if (bUseMarkerSyncing)
			{
				AdvanceLeaderMarkerBased(Context, GroupContext);
			}
			else
			{
				AdvanceLeaderTimeBased(Context, GroupContext);
			}
		}

		static void AdvanceFollowerTimeBased(FExecutionContext& Context, FSyncGroupContext& GroupContext, FSyncGroupMemberContext& GroupMemberContext)
		{
			TTraitBinding<IGroupSynchronization> GroupSyncTrait;

			const FSyncGroupMember& GroupMember = *GroupMemberContext.State;

			FTraitStackBinding& MemberTraitStack = GroupMemberContext.TraitStack;

			Context.BindTo(GroupMember.TraitPtr);
			ensure(Context.GetStack(GroupMember.TraitPtr, MemberTraitStack));
			ensure(MemberTraitStack.GetInterface(GroupSyncTrait));

			// Cache our starting timeline state
			FTimelineState FollowerStartTimelineState;
			ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerStartTimelineState));

			const bool bIsPlayingForward = GroupContext.bIsLeaderPlayingForward;
			const float PlayRate = FollowerStartTimelineState.GetPlayRate() != 0.0f ? FollowerStartTimelineState.GetPlayRate() : 1.0f;

			if (GroupMemberContext.State->bJustJoined)
			{
				// If we just joined the group as a follower then our current time is not relevant
				// We assume that we start at the same position in normalized time as our leader
				const float CurrentTime = FollowerStartTimelineState.GetPosition();
				const float DesiredTime = GroupContext.LeaderStartRatio * FollowerStartTimelineState.GetDuration();

				// Seek where we should be without dispatching events
				const float DeltaTime = (DesiredTime - CurrentTime) / PlayRate;
				GroupSyncTrait.AdvanceBy(Context, DeltaTime, false);

				// Update our cached state
				ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerStartTimelineState));
			}

#if UE_DEBUG_SYNC_GROUPS
			bool bLooped = false;
#endif

			const float PreviousPosition = FollowerStartTimelineState.GetPosition();
			float CurrentPosition = GroupContext.LeaderEndRatio * FollowerStartTimelineState.GetDuration();

			if (GroupContext.bIsLeaderPlayingForward)
			{
				if (CurrentPosition < PreviousPosition)
				{
					// We must have looped around but we still want a positive delta time to match our leader
					CurrentPosition += FollowerStartTimelineState.GetDuration();

#if UE_DEBUG_SYNC_GROUPS
					bLooped = true;
#endif
				}
			}
			else
			{
				if (CurrentPosition > PreviousPosition)
				{
					// We must have looped around but we still want a negative delta time to match our leader
					CurrentPosition -= FollowerStartTimelineState.GetDuration();

#if UE_DEBUG_SYNC_GROUPS
					bLooped = true;
#endif
				}
			}

			// Compute our desired delta time, accounting for the follower play rate
			const float DeltaTime = (CurrentPosition - PreviousPosition) / PlayRate;

			GroupSyncTrait.AdvanceBy(Context, DeltaTime, true);

#if UE_DEBUG_SYNC_GROUPS
			if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
			{
				FString DeltaTimeStr = FString::Printf(TEXT("%s %0.2f"),
					GroupContext.bIsLeaderPlayingForward ? TEXT("+") : TEXT("-"),
					DeltaTime * FMath::Abs(PlayRate));

				FTimelineState FollowerEndTimelineState;
				ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerEndTimelineState));

				UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Follow [Time] [%0.2f %s -> %0.2f / %0.2f (%3.2f%%)] Playing [%s]%s"),
					*GroupContext.State->GroupName.ToString(),
					GroupMember.TraitPtr.GetNodeInstance(),
					FollowerStartTimelineState.GetPosition(), *DeltaTimeStr,
					FollowerEndTimelineState.GetPosition(), FollowerEndTimelineState.GetDuration(),
					FollowerEndTimelineState.GetPositionRatio() * 100.0f,
					*FollowerEndTimelineState.GetDebugName().ToString(),
					bLooped ? TEXT(" (looped)") : TEXT(""));
			}
#endif
		}

		static void AdvanceFollowerMarkerBased(FExecutionContext& Context, FSyncGroupContext& GroupContext, FSyncGroupMemberContext& GroupMemberContext)
		{
			const FSyncGroupMemberContext& GroupLeaderContext = GroupContext.Members[GroupContext.LeaderIndex];

			const FSyncGroupMember& GroupMember = *GroupMemberContext.State;
			FTraitStackBinding& MemberTraitStack = GroupMemberContext.TraitStack;

			Context.BindTo(GroupMember.TraitPtr);
			ensure(Context.GetStack(GroupMember.TraitPtr, MemberTraitStack));

			TTraitBinding<IGroupSynchronization> GroupSyncTrait;
			ensure(MemberTraitStack.GetInterface(GroupSyncTrait));

			// Cache our starting timeline state
			FTimelineState FollowerStartTimelineState;
			ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerStartTimelineState));

			const bool bIsPlayingForward = GroupContext.bIsLeaderPlayingForward;
			const float PlayRate = FollowerStartTimelineState.GetPlayRate() != 0.0f ? FollowerStartTimelineState.GetPlayRate() : 1.0f;
			const float StartTime = FollowerStartTimelineState.GetPosition();

#if UE_DEBUG_SYNC_GROUPS
			FString JoiningMode;
#endif

			const bool bCanMatchPhase = CanMatchPhase(
				GroupContext.LeaderTimelineState, GroupLeaderContext.SyncMarkers,
				FollowerStartTimelineState, GroupMemberContext.SyncMarkers);

			if (GroupMemberContext.State->bJustJoined)
			{
				if (GroupMemberContext.State->GroupParameters.bMatchSyncPoint)
				{
					float DesiredTime;

					if (bCanMatchPhase)
					{
						// When our phases can match, we simulate playback from the start of the timeline
						// This ensures a deterministic outcome no matter when a member joins the group
						// We collect the sync markers of our leader between [StartPosition, CurrentPosition]
						// and we pass the same markers on our follower

						const FSyncGroupPhasePosition FollowerPhasePosition = FindMatchingPhaseTime(
							GroupContext.LeaderTimelineState, GroupLeaderContext.SyncMarkers, GroupContext.LeaderPhaseStart,
							FollowerStartTimelineState, GroupMemberContext.SyncMarkers);

						const FMarkerPair PrevMarker(FollowerPhasePosition.PrevMarkerIndex, 0.0f);
						const FMarkerPair NextMarker(FollowerPhasePosition.NextMarkerIndex, 0.0f);

						DesiredTime = CalculateTimeFromSyncPosition(GroupMemberContext.SyncMarkers, FollowerStartTimelineState, PrevMarker, NextMarker, FollowerPhasePosition.PositionBetweenMarkers);

#if UE_DEBUG_SYNC_GROUPS
						JoiningMode = TEXT(" (phase matched join)");
#endif
					}
					else
					{
						// If we can't match our phases then we attempt to find the closest matching marker
						// pair from our normalized position as dictated by the leader (e.g. if leader is
						// at 80%, we look for the closest pair around the 80% mark on our follower)
						// Our current time is thus not relevant and we ignore it

						// We assume that we start at the same position in normalized time as our leader
						const float CurrentTime = FollowerStartTimelineState.GetPosition();
						DesiredTime = GroupContext.LeaderStartRatio * FollowerStartTimelineState.GetDuration();

#if UE_DEBUG_SYNC_GROUPS
						JoiningMode = TEXT(" (relative matched join)");
#endif
					}

					// Seek where we should be without dispatching events
					FollowerStartTimelineState = FollowerStartTimelineState.WithPosition(DesiredTime);
				}
				else
				{
					// If we aren't requesting to match the sync point, then we attempt to find the closest
					// matching marker pair from our current position

#if UE_DEBUG_SYNC_GROUPS
					JoiningMode = TEXT(" (unmatched join)");
#endif
				}
			}

#if UE_DEBUG_SYNC_GROUPS
			const float MarkerSearchTime = FollowerStartTimelineState.GetPosition();
#endif

			// Find the nearest markers matching our current time
			FMarkerPair PrevMarkerStart;
			FMarkerPair NextMarkerStart;
			GetNearestMatchingBoundarySyncMarkers(
				GroupContext.LeaderPhaseStart,
				GroupMemberContext.SyncMarkers, FollowerStartTimelineState,
				PrevMarkerStart, NextMarkerStart);

			FMarkerPair PrevMarkerSanitized = PrevMarkerStart;
			FMarkerPair NextMarkerSanitized = NextMarkerStart;
			bool bHadInvalidMarkers = SanitizeFollowerMarkers(
				GroupContext.LeaderPhaseStart,
				GroupMemberContext.SyncMarkers, bIsPlayingForward, FollowerStartTimelineState.IsLooping(),
				PrevMarkerSanitized, NextMarkerSanitized);

			if (GroupMemberContext.State->bJustJoined)
			{
				// If we just joined, we started searching for our markers approximately where the leader was
				// However, we might end up finding markers before/after where we should be and as a result can
				// end up with a slightly positive/negative delta time moving opposite the desired direction
				// To avoid this, now that we've found good markers, we seek again to the position the leader
				// started at between them
				const float CurrentTime = StartTime;
				const float DesiredTime = CalculateTimeFromSyncPosition(
					GroupMemberContext.SyncMarkers, FollowerStartTimelineState,
					PrevMarkerSanitized, NextMarkerSanitized, GroupContext.LeaderPhaseStart.PositionBetweenMarkers);

				// Seek where we should be without dispatching events
				const float DeltaTime = (DesiredTime - CurrentTime) / PlayRate;
				GroupSyncTrait.AdvanceBy(Context, DeltaTime, false);

				// Update our cached state
				ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerStartTimelineState));
			}

			const float CurrentTime = FollowerStartTimelineState.GetPosition();

			FMarkerPair PrevMarkerEnd = PrevMarkerSanitized;
			FMarkerPair NextMarkerEnd = NextMarkerSanitized;
			float DesiredTime = AdvanceSyncMarkersPassed(
				FollowerStartTimelineState, bIsPlayingForward, GroupContext.LeaderPhaseEnd,
				GroupMemberContext.SyncMarkers, GroupContext.MarkersPassed, PrevMarkerEnd, NextMarkerEnd);

			if (bIsPlayingForward)
			{
				if (DesiredTime < CurrentTime)
				{
					// We are looping around, wrap around to ensure our delta time takes us past the end of the timeline
					// The player will handle looping internally
					DesiredTime += FollowerStartTimelineState.GetDuration();
				}
			}
			else
			{
				if (DesiredTime > CurrentTime)
				{
					// We are looping around, wrap around to ensure our delta time takes us past the start of the timeline
					// The played will handle looping internally
					DesiredTime -= FollowerStartTimelineState.GetDuration();
				}
			}

			// Compute our desired delta time, accounting for the follower play rate
			const float DeltaTime = (DesiredTime - CurrentTime) / PlayRate;
			GroupSyncTrait.AdvanceBy(Context, DeltaTime, true);

#if UE_DEBUG_SYNC_GROUPS
			if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
			{
				FString DeltaTimeStr = FString::Printf(TEXT("%s %0.2f"),
					GroupContext.bIsLeaderPlayingForward ? TEXT("+") : TEXT("-"),
					DeltaTime * FMath::Abs(PlayRate));

				FTimelineState FollowerEndTimelineState;
				ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerEndTimelineState));

				const bool bLooped = GroupContext.bIsLeaderPlayingForward ?
					(FollowerEndTimelineState.GetPosition() < FollowerStartTimelineState.GetPosition()) :
					(FollowerStartTimelineState.GetPosition() < FollowerEndTimelineState.GetPosition());

				const bool bUnexpectedDeltaTime = DeltaTime < 0.0f || DeltaTime > 0.3f;

				FString SanitizedResult;
				if (bHadInvalidMarkers)
				{
					SanitizedResult = FString::Printf(TEXT(" (fixed invalid markers [%s@%d | %s@%d] -> [%s@%d | %s@%d])"),
						*GetMarkerName(PrevMarkerStart, GroupMemberContext.SyncMarkers).ToString(),
						PrevMarkerStart.MarkerIndex,
						*GetMarkerName(NextMarkerStart, GroupMemberContext.SyncMarkers).ToString(),
						NextMarkerStart.MarkerIndex,
						*GetMarkerName(PrevMarkerSanitized, GroupMemberContext.SyncMarkers).ToString(),
						PrevMarkerSanitized.MarkerIndex,
						*GetMarkerName(NextMarkerSanitized, GroupMemberContext.SyncMarkers).ToString(),
						NextMarkerSanitized.MarkerIndex);
				}

				FString JoinedStatus;
				if (GroupMemberContext.State->bJustJoined)
				{
					JoinedStatus = FString::Printf(TEXT(" (joined from %0.2f)"), MarkerSearchTime);
				}

				UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Follow [Mark] [%0.2f %s -> %0.2f / %0.2f (%3.2f%%)] [%s@%0.2f | %s@%0.2f (%3.2f%%) -> %s@%0.2f | %s@%0.2f (%3.2f%%)] Playing [%s]%s%s%s%s%s"),
					*GroupContext.State->GroupName.ToString(),
					GroupMember.TraitPtr.GetNodeInstance(),
					CurrentTime, *DeltaTimeStr,
					FollowerEndTimelineState.GetPosition(), FollowerEndTimelineState.GetDuration(),
					FollowerEndTimelineState.GetPositionRatio() * 100.0f,
					*GetMarkerName(PrevMarkerStart, GroupMemberContext.SyncMarkers).ToString(),
					GetMarkerPosition(PrevMarkerStart, GroupMemberContext.SyncMarkers, 0.0f),
					*GetMarkerName(NextMarkerStart, GroupMemberContext.SyncMarkers).ToString(),
					GetMarkerPosition(NextMarkerStart, GroupMemberContext.SyncMarkers, FollowerEndTimelineState.GetDuration()),
					GroupContext.LeaderPhaseStart.PositionBetweenMarkers * 100.0f,
					*GetMarkerName(PrevMarkerEnd, GroupMemberContext.SyncMarkers).ToString(),
					GetMarkerPosition(PrevMarkerEnd, GroupMemberContext.SyncMarkers, 0.0f),
					*GetMarkerName(NextMarkerEnd, GroupMemberContext.SyncMarkers).ToString(),
					GetMarkerPosition(NextMarkerEnd, GroupMemberContext.SyncMarkers, FollowerEndTimelineState.GetDuration()),
					GroupContext.LeaderPhaseEnd.PositionBetweenMarkers * 100.0f,
					*FollowerEndTimelineState.GetDebugName().ToString(),
					bLooped ? TEXT(" (looped)") : TEXT(""),
					bUnexpectedDeltaTime ? TEXT(" (unusual delta time)") : TEXT(""),
					bHadInvalidMarkers ? *SanitizedResult : TEXT(""),
					*JoiningMode,
					*JoinedStatus);
			}
#endif
		}

		static void AdvanceFollowerUngrouped(FExecutionContext& Context, FSyncGroupContext& GroupContext, FSyncGroupMemberContext& GroupMemberContext)
		{
			const FSyncGroupMember& GroupMember = *GroupMemberContext.State;

			FTraitStackBinding& MemberTraitStack = GroupMemberContext.TraitStack;

			Context.BindTo(GroupMember.TraitPtr);
			ensure(Context.GetStack(GroupMember.TraitPtr, MemberTraitStack));

			TTraitBinding<IGroupSynchronization> GroupSyncTrait;
			ensure(MemberTraitStack.GetInterface(GroupSyncTrait));

#if UE_DEBUG_SYNC_GROUPS
			FTimelineState FollowerStartTimelineState;
			ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerStartTimelineState));
#endif

			const float DeltaTime = GroupMember.TraitState.GetDeltaTime();
			GroupSyncTrait.AdvanceBy(Context, DeltaTime, true);

#if UE_DEBUG_SYNC_GROUPS
			if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
			{
				FString DeltaTimeStr = FString::Printf(TEXT("%s %0.2f"),
					GroupContext.bIsLeaderPlayingForward ? TEXT("+") : TEXT("-"),
					DeltaTime * FMath::Abs(FollowerStartTimelineState.GetPlayRate()));

				FTimelineState FollowerEndTimelineState;
				ensure(GroupMemberContext.TimelineTrait.GetState(Context, FollowerEndTimelineState));

				UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Follow [Solo] [%0.2f %s -> %0.2f / %0.2f (%3.2f%%)] Playing [%s]"),
					*GroupContext.State->GroupName.ToString(),
					GroupMember.TraitPtr.GetNodeInstance(),
					FollowerStartTimelineState.GetPosition(), *DeltaTimeStr,
					FollowerEndTimelineState.GetPosition(), FollowerEndTimelineState.GetDuration(),
					FollowerEndTimelineState.GetPositionRatio() * 100.0f,
					*FollowerEndTimelineState.GetDebugName().ToString());
			}
#endif
		}

		static void AdvanceFollowers(FExecutionContext& Context, FSyncGroupContext& GroupContext)
		{
			const int32 NumMembers = GroupContext.Members.Num();
			if (NumMembers == 1)
			{
				return;	// No followers if we have a single leader in the group
			}

			const int32 LeaderIndex = GroupContext.LeaderIndex;

			// Advance every follower to the same progress ratio as the leader
			for (int32 MemberIndex = 0; MemberIndex < NumMembers; ++MemberIndex)
			{
				if (MemberIndex == LeaderIndex)
				{
					continue;	// Ignore the leader, it already advanced
				}

				FSyncGroupMemberContext& GroupMemberContext = GroupContext.Members[MemberIndex];
				const FSyncGroupMember& GroupMember = *GroupMemberContext.State;

				bool bUseMarkerSyncing = GroupMemberContext.bUseMarkerSyncing;
				EAnimGroupSynchronizationRole GroupRole = GroupMember.GroupParameters.GroupRole;

#if UE_DEBUG_SYNC_GROUPS
				const int32 DebugSyncMode = CDebugVarSyncMode.GetValueOnAnyThread();
				if (DebugSyncMode == 1)
				{
					bUseMarkerSyncing = false;
				}
				else if (DebugSyncMode == 2)
				{
					GroupRole = EAnimGroupSynchronizationRole::ExclusiveAlwaysLeader;
				}
#endif

				if (GroupRole == EAnimGroupSynchronizationRole::ExclusiveAlwaysLeader)
				{
					// We asked to be a leader and we weren't picked as leader
					// We don't want to follow anyone so we tick on our own
					AdvanceFollowerUngrouped(Context, GroupContext, GroupMemberContext);
				}
				else if (bUseMarkerSyncing)
				{
					AdvanceFollowerMarkerBased(Context, GroupContext, GroupMemberContext);
				}
				else
				{
					AdvanceFollowerTimeBased(Context, GroupContext, GroupMemberContext);
				}
			}
		}

		static UObject* GetHostObject(const FWeakTraitPtr& TraitPtr)
		{
			const FAnimNextModuleInstance* ModuleInstance = TraitPtr.GetNodeInstance()->GetOwner().GetRootGraphInstance()->GetModuleInstance();
			return ModuleInstance != nullptr ? ModuleInstance->GetObject() : nullptr;
		}
	}
}

// Defaulted constructors/copy-operators in cpp so fwd decls of members can work
FSyncGroupGraphInstanceComponent::FSyncGroupGraphInstanceComponent() = default;
FSyncGroupGraphInstanceComponent::FSyncGroupGraphInstanceComponent(FSyncGroupGraphInstanceComponent&&) = default;
FSyncGroupGraphInstanceComponent::FSyncGroupGraphInstanceComponent(const FSyncGroupGraphInstanceComponent&) = default;
FSyncGroupGraphInstanceComponent& FSyncGroupGraphInstanceComponent::operator=(const FSyncGroupGraphInstanceComponent&) = default;
FSyncGroupGraphInstanceComponent& FSyncGroupGraphInstanceComponent::operator=(FSyncGroupGraphInstanceComponent&&) = default;

FSyncGroupGraphInstanceComponent::~FSyncGroupGraphInstanceComponent()
{
	using namespace UE::UAF;

	// We should have released all used group names before we are destroyed
	check(UsedUniqueGroupNames.IsEmpty());

	Private::FSyncGroupUniqueName* Entry = FirstFreeUniqueGroupName;
	while (Entry != nullptr)
	{
		Private::FSyncGroupUniqueName* NextEntry = Entry->NextFreeEntry;

		delete Entry;

		Entry = NextEntry;
	}

	FirstFreeUniqueGroupName = nullptr;
}

void FSyncGroupGraphInstanceComponent::RegisterWithGroup(const UE::UAF::FSyncGroupParameters& GroupParameters, const UE::UAF::FWeakTraitPtr& TraitPtr, const UE::UAF::FTraitUpdateState& TraitState)
{
	using namespace UE::UAF::Private;

	if (!ensure(TraitPtr.IsValid()))
	{
		return;
	}

	const int32 GroupIndex = SyncGroupMap.FindOrAdd(GroupParameters.GroupName, SyncGroups.Num());
	FSyncGroupState& GroupState = GroupIndex == SyncGroups.Num() ? SyncGroups.AddDefaulted_GetRef() : SyncGroups[GroupIndex];
	GroupState.GroupName = GroupParameters.GroupName;
	GroupState.bIsActive = true;

	bool bIsMemberNew = true;
	for (FSyncGroupMember& GroupMember : GroupState.Members)
	{
		if (GroupMember.TraitPtr == TraitPtr)
		{
			bIsMemberNew = false;
			check(!GroupMember.bIsActive);

			GroupMember.TraitState = TraitState;
			GroupMember.GroupParameters = GroupParameters;
			GroupMember.bIsActive = true;
			break;
		}
	}

	if (bIsMemberNew)
	{
		GroupState.Members.Add(FSyncGroupMember{ TraitState, UE::UAF::FTraitPtr(TraitPtr), GroupParameters, true, true });

#if UE_DEBUG_SYNC_GROUPS
		if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
		{
			UE_VLOG_UELOG(GetHostObject(TraitPtr), LogAnimMarkerSync, Verbose,
				TEXT("[%s] [%p] %s"),
				*GroupParameters.GroupName.ToString(),
				TraitPtr.GetNodeInstance(),
				GroupState.Members.Num() == 1 ? TEXT("Created") : TEXT("Joined"));
		}
#endif
	}
}

FName FSyncGroupGraphInstanceComponent::CreateUniqueGroupName()
{
	using namespace UE::UAF;
	
	// We create a unique group name by incrementing a counter and using it to generate a new FName
	// We also recycle them to keep the counter as low as possible because in some configurations
	// FNames are 32-bit as the number is stored in the string table. Recycling the group names
	// ensures we don't pollute the FName string table.
	// The counter could be the same across multiple graph instances which is fine since we only
	// synchronize using groups within a single graph instance.

	Private::FSyncGroupUniqueName* Entry;

	if (FirstFreeUniqueGroupName != nullptr)
	{
		// Recycle an old entry
		Entry = FirstFreeUniqueGroupName;
		FirstFreeUniqueGroupName = Entry->NextFreeEntry;
	}
	else
	{
		// Generate a new unique group name
		FName UniqueGroupName = FName(Private::NAME_UniqueGroupNamePrefix, UniqueGroupNameCounter);
		UniqueGroupNameCounter++;

		Entry = new Private::FSyncGroupUniqueName();
		Entry->GroupName = UniqueGroupName;
	}

	Entry->NextFreeEntry = nullptr;

	UsedUniqueGroupNames.Add(Entry->GroupName, Entry);

	return Entry->GroupName;
}

void FSyncGroupGraphInstanceComponent::ReleaseUniqueGroupName(FName GroupName)
{
	using namespace UE::UAF::Private;

	FSyncGroupUniqueName* Entry = nullptr;
	if (ensure(UsedUniqueGroupNames.RemoveAndCopyValue(GroupName, Entry)))
	{
		Entry->NextFreeEntry = FirstFreeUniqueGroupName;
		FirstFreeUniqueGroupName = Entry;
	}

	if (const int32* GroupIndex = SyncGroupMap.Find(GroupName))
	{
		FSyncGroupState& GroupState = SyncGroups[*GroupIndex];

#if UE_DEBUG_SYNC_GROUPS
		if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
		{
			for (int32 GroupMemberIndex = GroupState.Members.Num() - 1; GroupMemberIndex >= 0; GroupMemberIndex--)
			{
				const FSyncGroupMember& GroupMember = GroupState.Members[GroupMemberIndex];

				UE_VLOG_UELOG(GetHostObject(GroupMember.TraitPtr), LogAnimMarkerSync, Verbose,
					TEXT("[%s] [%p] Left"),
					*GroupState.GroupName.ToString(),
					GroupMember.TraitPtr.GetNodeInstance());
			}
		}
#endif

		// We are releasing this group, clear any lingering member entries
		// as we might re-use the group name this update
		GroupState.Members.Reset();
	}
}

void FSyncGroupGraphInstanceComponent::PreUpdate(UE::UAF::FExecutionContext& Context)
{
	using namespace UE::UAF::Private;

	for (FSyncGroupState& GroupState : SyncGroups)
	{
		for (FSyncGroupMember& GroupMember : GroupState.Members)
		{
			GroupMember.bIsActive = false;
			GroupMember.bJustJoined = false;
		}

		GroupState.bIsActive = false;
		GroupState.bJustFormed = GroupState.Members.IsEmpty();
	}
}

void FSyncGroupGraphInstanceComponent::PostUpdate(UE::UAF::FExecutionContext& Context)
{
	using namespace UE::UAF::Private;

	// First purge stale entries and update our bookkeeping
	int32 MaxNumMembers = 0;
	{
		bool bRebuildGroupIndexMap = false;
		for (int32 SyncGroupIndex = SyncGroups.Num() - 1; SyncGroupIndex >= 0; --SyncGroupIndex)
		{
			FSyncGroupState& GroupState = SyncGroups[SyncGroupIndex];

			if (GroupState.bIsActive)
			{
				// Whether or not we have an active member from a previous update
				bool bHasOldActiveMember = false;

				for (int32 GroupMemberIndex = GroupState.Members.Num() - 1; GroupMemberIndex >= 0; GroupMemberIndex--)
				{
					FSyncGroupMember& GroupMember = GroupState.Members[GroupMemberIndex];
					if (!GroupMember.bIsActive)
					{
#if UE_DEBUG_SYNC_GROUPS
						if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
						{
							UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
								TEXT("[%s] [%p] Left"),
								*GroupState.GroupName.ToString(),
								GroupMember.TraitPtr.GetNodeInstance());
						}
#endif

						// This member is no longer active, remove it
						GroupState.Members.RemoveAtSwap(GroupMemberIndex, EAllowShrinking::No);
					}
					else if (!GroupMember.bJustJoined)
					{
						bHasOldActiveMember = true;
					}
				}

				// If we don't have any lingering members from a previous update, then we are considered
				// as a fresh new group
				if (!bHasOldActiveMember && !GroupState.Members.IsEmpty() && !GroupState.bJustFormed)
				{
#if UE_DEBUG_SYNC_GROUPS
					if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
					{
						UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
							TEXT("[%s] Refreshed"),
							*GroupState.GroupName.ToString());
					}
#endif

					GroupState.bJustFormed = true;
				}

				MaxNumMembers = FMath::Max<uint32>(MaxNumMembers, GroupState.Members.Num());
			}
			else
			{
#if UE_DEBUG_SYNC_GROUPS
				if (CDebugVarEnableSyncLog.GetValueOnAnyThread())
				{
					for (int32 GroupMemberIndex = GroupState.Members.Num() - 1; GroupMemberIndex >= 0; GroupMemberIndex--)
					{
						const FSyncGroupMember& GroupMember = GroupState.Members[GroupMemberIndex];

						UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
							TEXT("[%s] [%p] Left"),
							*GroupState.GroupName.ToString(),
							GroupMember.TraitPtr.GetNodeInstance());
					}

					UE_VLOG_UELOG(Context.GetHostObject(), LogAnimMarkerSync, Verbose,
						TEXT("[%s] Released"),
						*GroupState.GroupName.ToString());
				}
#endif

				// This group is no longer active, remove it
				SyncGroups.RemoveAtSwap(SyncGroupIndex, EAllowShrinking::No);
				bRebuildGroupIndexMap = true;
			}
		}

		if (bRebuildGroupIndexMap)
		{
			// We purged one or more inactive groups, rebuild our map
			SyncGroupMap.Reset();

			const int32 NumSyncGroups = SyncGroups.Num();
			for (int32 SyncGroupIndex = 0; SyncGroupIndex < NumSyncGroups; ++SyncGroupIndex)
			{
				const FSyncGroupState& GroupState = SyncGroups[SyncGroupIndex];
				SyncGroupMap.Add(GroupState.GroupName, SyncGroupIndex);
			}
		}
	}

	TArray<FSyncGroupMemberContext, TMemStackAllocator<>> MemberContexts;
	MemberContexts.AddDefaulted(MaxNumMembers);

	FSyncGroupContext GroupContext;

	// Now that we have discovered all groups and their memberships, we can perform synchronization
	for (FSyncGroupState& GroupState : SyncGroups)
	{
		// Initialize our context for this new group
		InitGroup(GroupState, MemberContexts, GroupContext);

		// Find our leader for this group
		FindLeaderIndex(Context, GroupContext);

		// If the leader has sync markers, collect and filter them
		// We'll retain only markers common to all members
		BuildGroupState(Context, GroupContext);

		// Advance the leader position as it determines the position of its followers
		AdvanceLeader(Context, GroupContext);

		// Advance the follower positions based on their leader
		AdvanceFollowers(Context, GroupContext);

		// Retain the leader progress as we might need it if a new leader joins the group during the next update
		GroupState.PreviousLeaderTimelineState = GroupContext.LeaderTimelineState;
		GroupState.PreviousLeaderSyncMarkers = GroupContext.Members[GroupContext.LeaderIndex].SyncMarkers;
		GroupState.PreviousLeaderPhasePosition = GroupContext.LeaderPhaseEnd;

#if UE_DEBUG_SYNC_GROUPS
		GroupState.PreviousLeaderIndex = GroupContext.LeaderIndex;
#endif
	}
}


#undef UE_DEBUG_SYNC_GROUPS
