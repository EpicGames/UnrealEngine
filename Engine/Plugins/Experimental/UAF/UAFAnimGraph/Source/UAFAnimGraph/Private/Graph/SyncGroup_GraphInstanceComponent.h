// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitPtr.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/UAFGraphInstanceComponent.h"
#include "SyncGroup_GraphInstanceComponent.generated.h"

namespace UE::UAF
{
	namespace Private
	{
		struct FSyncGroupState;
		struct FSyncGroupUniqueName;
	}

	struct FSyncGroupParameters;
}

/**
 * FSyncGroupGraphInstanceComponent
 *
 * This component maintains the necessary state to support group based synchronization.
 */
USTRUCT()
struct FSyncGroupGraphInstanceComponent : public FUAFGraphInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// Defaulted constructors/copy-operators in cpp so fwd decls of members can work
	FSyncGroupGraphInstanceComponent();
	FSyncGroupGraphInstanceComponent(FSyncGroupGraphInstanceComponent&&);
	FSyncGroupGraphInstanceComponent(const FSyncGroupGraphInstanceComponent&);
	FSyncGroupGraphInstanceComponent& operator=(const FSyncGroupGraphInstanceComponent&);
	FSyncGroupGraphInstanceComponent& operator=(FSyncGroupGraphInstanceComponent&&);

	virtual ~FSyncGroupGraphInstanceComponent();

	// Registers the specified trait with group based synchronization
	void RegisterWithGroup(const UE::UAF::FSyncGroupParameters& GroupParameters, const UE::UAF::FWeakTraitPtr& TraitPtr, const UE::UAF::FTraitUpdateState& TraitState);

	// Create a unique group name suitable for spawned sub-graphs to self-synchronize
	// Unique group names are a limited resource, when no longed needed they must be released
	FName CreateUniqueGroupName();

	// Releases a unique group name that is no longer needed. It will be recycled the
	// next time one is needed
	void ReleaseUniqueGroupName(FName GroupName);

	// FGraphInstanceComponent impl
	virtual void PreUpdate(UE::UAF::FExecutionContext& Context) override;
	virtual void PostUpdate(UE::UAF::FExecutionContext& Context) override;

private:
	// A map of sync group name -> group index
	TMap<FName, int32> SyncGroupMap;

	// A list of groups and their data
	TArray<UE::UAF::Private::FSyncGroupState> SyncGroups;

	// The first free unique group name
	UE::UAF::Private::FSyncGroupUniqueName* FirstFreeUniqueGroupName = nullptr;

	// The map of currently used group names
	TMap<FName, UE::UAF::Private::FSyncGroupUniqueName*> UsedUniqueGroupNames;

	// A counter tracking the next unique group name to allocate
	int32 UniqueGroupNameCounter = 0;
};

