// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/UAFGraphInstanceComponent.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Graph/AnimGraphTaskContext.h"
#include "UAFAnimGraphInitializerComponent.generated.h"

namespace UE::UAF
{
struct FExecutionContext;
}

/** Internal component used for initial UAF graph setup */
USTRUCT(meta=(Hidden, Abstract))
struct FUAFAnimGraphInitializerComponent : public FUAFGraphInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	FUAFAnimGraphInitializerComponent() = default;
	explicit FUAFAnimGraphInitializerComponent(FAnimNextFactoryParams&& InFactoryParams)
		: FactoryParams(MoveTemp(InFactoryParams))
	{}

	// Factory params used to construct the graph instance
	FAnimNextFactoryParams FactoryParams;

	// Tasks queue to be applied on construction
	TArray<UE::UAF::FUniqueAnimGraphTask> TaskQueue;
};

template<>
struct TStructOpsTypeTraits<FUAFAnimGraphInitializerComponent> : public TStructOpsTypeTraitsBase2<FUAFAnimGraphInitializerComponent>
{
	enum
	{
		WithCopy = false
	};
};

