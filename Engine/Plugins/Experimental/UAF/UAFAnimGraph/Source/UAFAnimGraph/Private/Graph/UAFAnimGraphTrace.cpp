// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/UAFAnimGraphTrace.h"

#include "EvaluationVM/EvaluationProgram.h"
#include "EvaluationVM/SerializableEvaluationProgram.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "ObjectTrace.h"
#include "ObjectAsTraceIdProxyArchive.h"
#include "Serialization/MemoryWriter.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/IHierarchy.h"

#if UAF_TRACE_ENABLED

UE_TRACE_EVENT_BEGIN(UAFAnimGraph, EvaluationProgram)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, OuterObjectId)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint8[], ProgramData)
UE_TRACE_EVENT_END()

namespace UE::UAF
{
	void TraceGraphInstances(const FAnimNextGraphInstance& RootGraph)
	{
		bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(UAFChannel);
    	if (!bChannelEnabled)
    	{
    		return;
    	}

		UObject* OuterObject = RootGraph.GetModuleInstance() ? RootGraph.GetModuleInstance()->GetObject() : nullptr;

		FExecutionContext ExecutionContext;
		TArray<FWeakTraitPtr, TInlineAllocator<8, TMemStackAllocator<>>> Traits;
		TSet<uint64> TracedInstances;
		
		FMemStack& MemStack = FMemStack::Get();
		FMemMark Mark(MemStack);
		
		ExecutionContext.BindTo(RootGraph.GetGraphRootPtr());

		Traits.Push(RootGraph.GetGraphRootPtr());

		TTraitBinding<IHierarchy> HierarchyTrait;

		while (!Traits.IsEmpty())
		{
			FWeakTraitPtr Trait = Traits.Pop();
			if (Trait.IsValid())
			{
				FAnimNextGraphInstance& Graph = Trait.GetNodeInstance()->GetOwner();
				if (!TracedInstances.Contains(Graph.GetUniqueId()))
				{
					TRACE_UAF_VARIABLES(&Graph, Graph.GetModuleInstance() ? Graph.GetModuleInstance()->GetObject() : nullptr);
					TracedInstances.Add(Graph.GetUniqueId());
				}

				FTraitStackBinding TraitStack;
				ExecutionContext.GetStack(Trait, TraitStack);
				
				if (TraitStack.GetInterface(HierarchyTrait))
				{
					IHierarchy::GetStackChildren(ExecutionContext, TraitStack, Traits);
				}
			}
		}
	}

	void TraceEvaluationProgram(const FEvaluationProgram& Program,  const FAnimNextGraphInstance& RootGraph)
	{
		bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(UAFChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		uint64 OuterObjectId = 0;
		if (FAnimNextModuleInstance* ModuleInstance = RootGraph.GetModuleInstance())
		{
			TraceEvaluationProgram(Program, RootGraph, ModuleInstance->GetObject());
		}
	}
		
	void TraceEvaluationProgram(const FEvaluationProgram& Program,  const FUAFAssetInstance& RootGraph, const UObject* OuterObject)
	{
		uint64 InstanceId = RootGraph.GetUniqueId();
		
		double RecordingTime = 0;
		if (OuterObject)
		{
			RecordingTime = FObjectTrace::GetWorldElapsedTime(OuterObject->GetWorld());
		}
		
		uint64 HostId = RootGraph.GetHost() ? RootGraph.GetHost()->GetUniqueId() : FObjectTrace::GetObjectId(OuterObject);

		TArray<uint8> ArchiveData;
		FMemoryWriter WriterArchive(ArchiveData);
		FObjectAsTraceIdProxyArchive Archive(WriterArchive);
		
		static const FSerializableEvaluationProgram Defaults;
		FSerializableEvaluationProgram SerializableProgram(Program);
		FSerializableEvaluationProgram::StaticStruct()->SerializeItem(Archive, &SerializableProgram, &Defaults);
		
		UE_TRACE_LOG(UAFAnimGraph, EvaluationProgram, UAFChannel)
				<< EvaluationProgram.Cycle(FPlatformTime::Cycles64())
				<< EvaluationProgram.OuterObjectId(HostId)
				<< EvaluationProgram.RecordingTime(RecordingTime)
				<< EvaluationProgram.InstanceId(InstanceId)
				<< EvaluationProgram.ProgramData(ArchiveData.GetData(), ArchiveData.Num());
	}
}
#endif
