// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/UAFGraphInstanceComponent.h"
#include "Graph/AnimNextGraphInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFGraphInstanceComponent)

FAnimNextGraphInstance& FUAFGraphInstanceComponent::GetGraphInstance()
{
	return static_cast<FAnimNextGraphInstance&>(*Instance);
}

const FAnimNextGraphInstance& FUAFGraphInstanceComponent::GetGraphInstance() const
{
	return static_cast<const FAnimNextGraphInstance&>(*Instance);
}
