// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/VirtualValueBundle_ExternalGraph.h"

namespace UE::UAF
{

const FWeakAnimGraphReference* FVirtualValueBundle_ExternalGraph::GetAnimGraphReference() const
{
	return &WeakGraphReference;
}

}
