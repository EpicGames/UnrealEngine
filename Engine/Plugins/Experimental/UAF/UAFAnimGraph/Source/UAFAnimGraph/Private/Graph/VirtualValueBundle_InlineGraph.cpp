// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/VirtualValueBundle_InlineGraph.h"

namespace UE::UAF
{

const FTraitPtr* FVirtualValueBundle_InlineGraph::GetInlineGraph() const
{
	return &TraitPtr;
}

}
