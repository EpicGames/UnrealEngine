// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/VirtualValueBundle_ValueBundle.h"

namespace UE::UAF
{

const FAnimNextGraphLODPose* FVirtualValueBundle_ValueBundle::GetLODPose() const
{
	return &Pose;
}

const FValueBundleHeap* FVirtualValueBundle_ValueBundle::GetValueBundle() const
{
	return &ValueBundle;
}

}
