// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/WeakAnimGraphReference.h"

#include "Graph/AnimGraphReference.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"

namespace UE::UAF
{

FWeakAnimGraphReference::FWeakAnimGraphReference(const FAnimGraphReference& InAnimGraphReference)
{
	Ptr = InAnimGraphReference.Ptr;
}

FWeakAnimGraphReference::FWeakAnimGraphReference(const TSharedPtr<FAnimNextGraphInstance>& InAnimGraphInstance)
{
	Ptr = InAnimGraphInstance;
}

void FWeakAnimGraphReference::Reset()
{
	Ptr.Reset();
}

bool FWeakAnimGraphReference::IsValid() const
{
	return Ptr.IsValid();
}

const UUAFAnimGraph* FWeakAnimGraphReference::GetAnimGraph() const
{
	if (TSharedPtr<FAnimNextGraphInstance> PinnedPtr = Ptr.Pin())
	{
		return PinnedPtr->GetAnimationGraph();
	}
	return nullptr;
}

TSharedPtr<FAnimNextGraphInstance> FWeakAnimGraphReference::GetOrAllocate(const FExecutionContext& InContext, const FTraitBinding& InBinding) const
{
	if (TSharedPtr<FAnimNextGraphInstance> PinnedPtr = Ptr.Pin())
	{
		PinnedPtr->AllocateVariablesInternal(nullptr, InBinding.GetTraitPtr().GetNodeInstance()->GetOwner().AsShared());
		PinnedPtr->AllocateGraphInternal();
		return PinnedPtr;
	}
	return TSharedPtr<FAnimNextGraphInstance>();
}

void FWeakAnimGraphReference::QueueTask(FUniqueAnimGraphTask&& InTaskFunction) const
{
	if (TSharedPtr<FAnimNextGraphInstance> PinnedPtr = Ptr.Pin())
	{
		if(FAnimNextModuleInstance* ModuleInstance = PinnedPtr->GetModuleInstance())
		{
			ModuleInstance->QueueTask([TaskFunction = MoveTemp(InTaskFunction), WeakAnimGraph = Ptr](const UE::UAF::FModuleTaskContext& InContext) mutable
			{
				if (TSharedPtr<FAnimNextGraphInstance> PinnedAnimGraph = WeakAnimGraph.Pin())
				{
					PinnedAnimGraph->RunOrQueueTask(MoveTemp(TaskFunction));
				}
			});
		}
	}
}

void FWeakAnimGraphReference::SetDebugName(FName InName) const
{
	if (TSharedPtr<FAnimNextGraphInstance> PinnedPtr = Ptr.Pin())
	{
		PinnedPtr->SetDebugName(InName);
	}
}

void FWeakAnimGraphReference::AddReferencedObjects(FReferenceCollector& InCollector) const
{
	if (TSharedPtr<FAnimNextGraphInstance> PinnedPtr = Ptr.Pin())
	{
		InCollector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), PinnedPtr.Get());
	}
}

}
