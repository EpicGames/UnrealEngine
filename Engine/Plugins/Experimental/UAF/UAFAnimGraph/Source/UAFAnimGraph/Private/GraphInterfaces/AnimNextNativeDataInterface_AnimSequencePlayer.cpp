// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphInterfaces/AnimNextNativeDataInterface_AnimSequencePlayer.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextNativeDataInterface_AnimSequencePlayer)

void FAnimNextNativeDataInterface_AnimSequencePlayer::BindToFactoryObject(const FBindToFactoryObjectContext& InContext)
{
	AnimSequence = CastChecked<UAnimSequence>(InContext.FactoryObject);
}
