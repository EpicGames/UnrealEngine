// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataInterface/AnimNextNativeDataInterface.h"
#include "Traits/SequencePlayerTraitData.h"
#include "AnimNextNativeDataInterface_AnimSequencePlayer.generated.h"

class UAnimSequence;

/**
 * DEPRECATED - no longer in use
 */
USTRUCT()
struct FAnimNextNativeDataInterface_AnimSequencePlayer : public FAnimNextNativeDataInterface
{
	GENERATED_BODY()

	// FAnimNextNativeDataInterface interface
	UAFANIMGRAPH_API virtual void BindToFactoryObject(const FBindToFactoryObjectContext& InContext) override;
#if WITH_EDITORONLY_DATA
	virtual const UScriptStruct* GetUpgradeTraitStruct() const { return FAnimNextSequencePlayerTraitSharedData::StaticStruct(); }
#endif

	// The animation object to play
	UPROPERTY()	// Not editable to hide it in cases where we would end up duplicating factory source asset
	TObjectPtr<const UAnimSequence> AnimSequence;

	// The play rate to use
	UPROPERTY(EditAnywhere, Category = "Anim Sequence", meta = (EnableCategories))
	double PlayRate = 1.0f;

	// The timeline's start position
	UPROPERTY(EditAnywhere, Category = "Anim Sequence", meta = (EnableCategories))
	double StartPosition = 0.0f;

	// Whether to loop the animation
	UPROPERTY(EditAnywhere, Category = "Anim Sequence", meta = (EnableCategories))
	bool Loop = false;
};
