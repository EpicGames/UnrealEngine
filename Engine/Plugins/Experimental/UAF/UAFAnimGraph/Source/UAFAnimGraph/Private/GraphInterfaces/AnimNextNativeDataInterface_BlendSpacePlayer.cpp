// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphInterfaces/AnimNextNativeDataInterface_BlendSpacePlayer.h"
#include "Animation/BlendSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextNativeDataInterface_BlendSpacePlayer)

void FAnimNextNativeDataInterface_BlendSpacePlayer::BindToFactoryObject(const FBindToFactoryObjectContext& InContext)
{
	BlendSpace = CastChecked<UBlendSpace>(InContext.FactoryObject);
}
