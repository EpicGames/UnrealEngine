// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataInterface/AnimNextNativeDataInterface.h"
#include "Traits/BlendSpacePlayerTraitData.h"
#include "AnimNextNativeDataInterface_BlendSpacePlayer.generated.h"

class UBlendSpace;

/**
 * DEPRECATED - no longer in use
 */
USTRUCT()
struct FAnimNextNativeDataInterface_BlendSpacePlayer : public FAnimNextNativeDataInterface
{
	GENERATED_BODY()

	// FAnimNextNativeDataInterface interface
	UAFANIMGRAPH_API virtual void BindToFactoryObject(const FBindToFactoryObjectContext& InContext) override;
#if WITH_EDITORONLY_DATA
	virtual const UScriptStruct* GetUpgradeTraitStruct() const { return FAnimNextBlendSpacePlayerTraitSharedData::StaticStruct(); }
#endif

	// The animation object to play
	UPROPERTY()	// Not editable to hide it in cases where we would end up duplicating factory source asset
	TObjectPtr<const UBlendSpace> BlendSpace;

	// The location on the x-axis to sample.
	UPROPERTY(EditAnywhere, Category = "Blend Space", meta = (EnableCategories))
	double XAxisSamplePoint = 0.0f;

	// The location on the y-axis to sample.
	UPROPERTY(EditAnywhere, Category = "Blend Space", meta = (EnableCategories))
	double YAxisSamplePoint = 0.0f;

	// The play rate to use
	UPROPERTY(EditAnywhere, Category = "Blend Space", meta = (EnableCategories))
	double PlayRate = 1.0f;

	// The timeline's start position. This is normalized in the [0,1] range.
	UPROPERTY(EditAnywhere, Category = "Blend Space", meta = (EnableCategories))
	double StartPosition = 0.0f;

	// Whether to loop the animation
	UPROPERTY(EditAnywhere, Category = "Blend Space", meta = (EnableCategories))
	bool Loop = false;
};
