// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataInterface/AnimNextNativeDataInterface.h"
#include "TraitInterfaces/IGroupSynchronization.h"
#include "Traits/SynchronizeUsingGroupsTraitData.h"
#include "AnimNextNativeDataInterface_SynchronizeUsingGroups.generated.h"


/**
 * DEPRECATED - no longer in use
 */
USTRUCT()
struct FAnimNextNativeDataInterface_SynchronizeUsingGroups : public FAnimNextNativeDataInterface
{
	GENERATED_BODY()

	// FAnimNextNativeDataInterface interface 
#if WITH_EDITORONLY_DATA
	virtual const UScriptStruct* GetUpgradeTraitStruct() const { return FAnimNextSynchronizeUsingGroupsTraitSharedData::StaticStruct(); }
#endif
	
	// The group name
	// If no name is provided, no synchronization occurs
	UPROPERTY(EditAnywhere, Category = "Group Synchronization", meta = (EnableCategories))
	FName GroupName;

	// The role this player can assume within the group
	UPROPERTY(EditAnywhere, Category = "Group Synchronization", meta = (EnableCategories))
	EAnimGroupSynchronizationRole GroupRole = EAnimGroupSynchronizationRole::CanBeLeader;

	// The synchronization mode
	UPROPERTY(EditAnywhere, Category = "Group Synchronization", meta = (EnableCategories))
	EAnimGroupSynchronizationMode SyncMode = EAnimGroupSynchronizationMode::NoSynchronization;

	// Whether or not to match the group sync point when joining as leader
	// When disabled, the leader will snap the position of the group to its current position
	UPROPERTY(EditAnywhere, Category = "Group Synchronization", meta = (EnableCategories))
	bool MatchSyncPoint = true;
};
