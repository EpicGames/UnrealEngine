// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/AnimNextModuleInjectionComponent.h"

#include "InjectionEvents.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/ModuleTaskContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleInjectionComponent)

void FAnimNextModuleInjectionComponent::OnBindToInstance()
{
	FAnimNextModuleInstance& ModuleInstance = GetModuleInstance();
	InjectionInfo = UE::UAF::FInjectionInfo(ModuleInstance);

	// Register re-injection for each (user) tick function
	for(UE::UAF::FModuleEventTickFunction& TickFunction : ModuleInstance.GetTickFunctions())
	{
		if(TickFunction.bUserEvent)
		{
			TickFunction.OnPreModuleEvent.AddStatic(&FAnimNextModuleInjectionComponent::OnReapplyInjection);
		}
	}
}

void FAnimNextModuleInjectionComponent::OnTraitEvent(FAnimNextTraitEvent& Event)
{
	if(UE::UAF::FInjection_InjectEvent* InjectionEvent = Event.AsType<UE::UAF::FInjection_InjectEvent>())
	{
		OnInjectionEvent(*InjectionEvent);
	}
	else if(UE::UAF::FInjection_UninjectEvent* UninjectionEvent = Event.AsType<UE::UAF::FInjection_UninjectEvent>())
	{
		OnUninjectionEvent(*UninjectionEvent);
	}
}

void FAnimNextModuleInjectionComponent::OnInjectionEvent(UE::UAF::FInjection_InjectEvent& InEvent)
{
	using namespace UE::UAF;

	if(InEvent.IsHandled())
	{
		return;
	}

	const FInjectionRequestArgs& RequestArgs = InEvent.Request->GetArgs();
	
	FAnimNextVariableReference FoundSite = InjectionInfo.FindInjectionSite(RequestArgs.Site);
	if(FoundSite.IsNone())
	{
		UE_LOGFMT(LogAnimation, Warning, "Could not find injection site '{SiteName}' for injection request", RequestArgs.Site.DesiredSite.GetName());
		InEvent.MarkConsumed();
		return;
	}

	// Correct the name found above as we may have targeted None (any)
	if(RequestArgs.Site.DesiredSite != FoundSite)
	{
		InEvent.Request->GetMutableArgs().Site = FInjectionSite(FoundSite);
	}

	// Mark as handled so any additional trait events dont get processed at the module level
	InEvent.MarkHandled();

	// Store request as it will need to be re-applied each frame to ensure that bindings do not override it
	FInjectionRecord& InjectionRecord = CurrentRequests.FindOrAdd(InEvent.Request->GetArgs().Site.DesiredSite);

	FAnimNextModuleInstance& ModuleInstance = GetModuleInstance();
	EPropertyBagResult Result = ModuleInstance.AccessVariable<FAnimNextAnimGraph>(FoundSite, [this, &RequestArgs, &InEvent, &InjectionRecord](FAnimNextAnimGraph& InGraph)
	{
		switch(RequestArgs.Type)
		{
		case EAnimNextInjectionType::InjectObject:
			// Bump serial number to identify this injection routing
			ensureAlways(RequestArgs.Object != nullptr);
			InGraph.InjectionData.InjectionSerialNumber = IncrementSerialNumber();
			InEvent.SerialNumber = InGraph.InjectionData.InjectionSerialNumber;
			InjectionRecord.SerialNumber = InGraph.InjectionData.InjectionSerialNumber;
			InjectionRecord.GraphRequest = InEvent.Request;

			// Note we dont consume here, as we want the event to forward to the injection site trait
			break;
		case EAnimNextInjectionType::EvaluationModifier:
			// We dont increment serial number when applying evaluation modifiers as we dont want to trigger a graph instantiation
			ensureAlways(RequestArgs.EvaluationModifier != nullptr);
			InGraph.InjectionData.EvaluationModifier = RequestArgs.EvaluationModifier;
			InjectionRecord.ModifierRequest = InEvent.Request;

			// Update status
			{
				auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
				StatusUpdateEvent->Request = InEvent.Request;
				StatusUpdateEvent->Status = EInjectionStatus::Playing;
				GetModuleInstance().QueueOutputTraitEvent(StatusUpdateEvent);
			}

			// Evaluation modifiers are consumed straight away
			InEvent.MarkConsumed();
			break;
		}
	});

	ensureAlways(Result == EPropertyBagResult::Success);
}

void FAnimNextModuleInjectionComponent::OnUninjectionEvent(UE::UAF::FInjection_UninjectEvent& InEvent)
{
	using namespace UE::UAF;

	const FInjectionRequestArgs& RequestArgs = InEvent.Request->GetArgs();

	// Injection modifiers can have private access to take over low-level tasks
	FAnimNextVariableReference FoundSite = InjectionInfo.FindInjectionSite(RequestArgs.Site);
	if(FoundSite.IsNone())
	{
		UE_LOGFMT(LogAnimation, Warning, "Could not find injection site '{SiteName}' for injection request", RequestArgs.Site.DesiredSite.GetName());
		InEvent.MarkConsumed();
		return;
	}

	FInjectionRecord& InjectionRecord = CurrentRequests.FindOrAdd(FoundSite);

	// Update graph and bump serial number to identify this un-injection routing
	FAnimNextModuleInstance& ModuleInstance = GetModuleInstance();
	EPropertyBagResult Result = ModuleInstance.AccessVariable<FAnimNextAnimGraph>(FoundSite, [this, &RequestArgs, &InEvent, &InjectionRecord](FAnimNextAnimGraph& InGraph)
	{
		switch(RequestArgs.Type)
		{
		case EAnimNextInjectionType::InjectObject:
			// Update graph and bump serial number to identify this injection routing
			InGraph.InjectionData.InjectionSerialNumber = IncrementSerialNumber();
			InEvent.SerialNumber = InGraph.InjectionData.InjectionSerialNumber;
			InjectionRecord.GraphRequest.Reset();

			// Note we dont consume here, as we want the event to forward to the injection site trait
			break;
		case EAnimNextInjectionType::EvaluationModifier:
			// We dont increment serial number when applying evaluation modifiers as we dont want to trigger a graph instantiation
			InGraph.InjectionData.EvaluationModifier = nullptr;
			InjectionRecord.ModifierRequest.Reset();

			// Update status
			{
				auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
				StatusUpdateEvent->Request = InEvent.Request;
				StatusUpdateEvent->Status = EInjectionStatus::Completed;
				GetModuleInstance().QueueOutputTraitEvent(StatusUpdateEvent);
			}
	
			// Evaluation modifiers are consumed straight away
			InEvent.MarkConsumed();
			break;
		}
	});

	ensureAlways(Result == EPropertyBagResult::Success);

	if(!InjectionRecord.IsValid())
	{
		CurrentRequests.Remove(InEvent.Request->GetArgs().Site.DesiredSite);
	}
}

void FAnimNextModuleInjectionComponent::OnReapplyInjection(const UE::UAF::FModuleTaskContext& InContext)
{
	using namespace UE::UAF;
	
	FAnimNextModuleInjectionComponent& Component = InContext.ModuleInstance.GetOrAddComponent<FAnimNextModuleInjectionComponent>();

	for(const TPair<FAnimNextVariableReference, FInjectionRecord>& RequestPair : Component.CurrentRequests)
	{
		// Re-apply this request, as it may have been overwritten by subsequent bindings/calculations
		EPropertyBagResult Result = InContext.ModuleInstance.AccessVariable<FAnimNextAnimGraph>(RequestPair.Key, [&RequestPair](FAnimNextAnimGraph& InGraph)
		{
			if(RequestPair.Value.GraphRequest.IsValid())
			{
				InGraph.InjectionData.InjectionSerialNumber = RequestPair.Value.SerialNumber;
			}
			if(RequestPair.Value.ModifierRequest.IsValid())
			{
				const FInjectionRequestArgs& ModifierRequestArgs = RequestPair.Value.ModifierRequest->GetArgs();
				InGraph.InjectionData.EvaluationModifier = ModifierRequestArgs.EvaluationModifier;
			}
		});

		ensureAlways(Result == EPropertyBagResult::Success);
	}
}

uint32 FAnimNextModuleInjectionComponent::IncrementSerialNumber()
{
	// Avoid zero as this is 'invalid' and  will ensure at injection sites (indicating incorrect routing)
	if(++SerialNumber == 0)
	{
		++SerialNumber;
	}
	return SerialNumber;
}

void FAnimNextModuleInjectionComponent::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	for (TPair<FAnimNextVariableReference, FInjectionRecord>& RequestPair : CurrentRequests)
	{
		RequestPair.Value.AddReferencedObjects(Collector);
	}
}

void FAnimNextModuleInjectionComponent::FInjectionRecord::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (GraphRequest.IsValid())
	{
		GraphRequest->AddReferencedObjects(Collector);
	}
	if (ModifierRequest.IsValid())
	{
		ModifierRequest->AddReferencedObjects(Collector);
	}
}
