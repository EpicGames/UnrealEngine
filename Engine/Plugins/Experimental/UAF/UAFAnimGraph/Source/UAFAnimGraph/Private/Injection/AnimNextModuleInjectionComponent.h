// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InjectionInfo.h"
#include "Injection/InjectionRequest.h"
#include "Module/UAFModuleInstanceComponent.h"
#include "UObject/GCObject.h"
#include "AnimNextModuleInjectionComponent.generated.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FInjection_InjectEvent;
	struct FInjection_UninjectEvent;
	struct FModuleTaskContext;
}

// Module component that holds info about injection sites and routes injection requests
USTRUCT()
struct FAnimNextModuleInjectionComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// Get the cached injection info for our module 
	const UE::UAF::FInjectionInfo& GetInjectionInfo() const { return InjectionInfo; }

	void AddStructReferencedObjects(class FReferenceCollector& Collector);

private:
	// FUAFModuleInstanceComponent interface
	virtual void OnTraitEvent(FAnimNextTraitEvent& Event) override;

	void OnInjectionEvent(UE::UAF::FInjection_InjectEvent& InEvent);
	void OnUninjectionEvent(UE::UAF::FInjection_UninjectEvent& InEvent);

	static void OnReapplyInjection(const UE::UAF::FModuleTaskContext& InContext);

	uint32 IncrementSerialNumber();

private:
	// FUAFAssetInstanceComponent interface
	virtual void OnBindToInstance() override;

	// Info for injection
	UE::UAF::FInjectionInfo InjectionInfo;

	struct FInjectionRecord
	{
		bool IsValid() const
		{
			return GraphRequest.IsValid() || ModifierRequest.IsValid();
		}

		void AddReferencedObjects(FReferenceCollector& Collector);

		TSharedPtr<UE::UAF::FInjectionRequest> GraphRequest;
		TSharedPtr<UE::UAF::FInjectionRequest> ModifierRequest;
		uint32 SerialNumber = 0;
	};
	
	// Currently-injected requests
	TMap<FAnimNextVariableReference, FInjectionRecord> CurrentRequests;

	// Serial number used to identify forwarded requests
	uint32 SerialNumber = 0;
};

template<>
struct TStructOpsTypeTraits<FAnimNextModuleInjectionComponent> : public TStructOpsTypeTraitsBase2<FAnimNextModuleInjectionComponent>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API