// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InjectionInfo.h"
#include "Graph/UAFGraphInstanceComponent.h"
#include "GraphInstanceInjectionComponent.generated.h"

/**
 * FGraphInstanceInjectionComponent
 * This component maintains injection info for a graph
 */
USTRUCT()
struct FGraphInstanceInjectionComponent : public FUAFGraphInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	const UE::UAF::FInjectionInfo& GetInjectionInfo() const
	{
		return InjectionInfo;
	}

private:
	// FUAFAssetInstanceComponent interface
	virtual void OnBindToInstance() override;
	
	UE::UAF::FInjectionInfo InjectionInfo;
};
