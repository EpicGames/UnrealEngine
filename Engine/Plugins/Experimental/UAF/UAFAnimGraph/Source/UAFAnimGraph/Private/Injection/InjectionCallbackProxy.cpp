// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionCallbackProxy.h"
#include "Animation/AnimSequence.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Component/AnimNextComponent.h"
#include "GraphInterfaces/AnimNextNativeDataInterface_AnimSequencePlayer.h"
#include "Injection/InjectionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InjectionCallbackProxy)

UInjectionCallbackProxy* UInjectionCallbackProxy::CreateProxyObjectForInjection(
	UUAFComponent* AnimNextComponent,
	FAnimNextVariableReference Site,
	UObject* Object,
	FAnimNextFactoryParams FactoryParams,
	UE::UAF::FInjectionBlendSettings BlendInSettings,
	UE::UAF::FInjectionBlendSettings BlendOutSettings)
{
	UInjectionCallbackProxy* Proxy = NewObject<UInjectionCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->Inject(AnimNextComponent, Site, Object, MoveTemp(FactoryParams), BlendInSettings, BlendOutSettings);
	return Proxy;
}

bool UInjectionCallbackProxy::Inject(
	UUAFComponent* AnimNextComponent,
	FAnimNextVariableReference Site,
	UObject* Object,
	FAnimNextFactoryParams&& FactoryParams,
	const UE::UAF::FInjectionBlendSettings& BlendInSettings,
	const UE::UAF::FInjectionBlendSettings& BlendOutSettings)
{
	if (AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "UInjectionCallbackProxy::Inject: Failed to inject - UAFComponent is not valid.");
		return false;
	}

	UE::UAF::FInjectionRequestArgs RequestArgs;
	RequestArgs.Site = FAnimNextInjectionSite(Site);
	RequestArgs.Object = Object;
	RequestArgs.BlendInSettings = BlendInSettings;
	RequestArgs.BlendOutSettings = BlendOutSettings;
	RequestArgs.FactoryParams = MoveTemp(FactoryParams);

	UE::UAF::FInjectionLifetimeEvents LifetimeEvents;
	LifetimeEvents.OnCompleted.BindUObject(this, &UInjectionCallbackProxy::OnInjectionCompleted);
	LifetimeEvents.OnInterrupted.BindUObject(this, &UInjectionCallbackProxy::OnInjectionInterrupted);
	LifetimeEvents.OnBlendingOut.BindUObject(this, &UInjectionCallbackProxy::OnInjectionBlendingOut);

	PlayingRequest = UE::UAF::FInjectionUtils::Inject(AnimNextComponent, AnimNextComponent->GetSystemReference(), MoveTemp(RequestArgs), MoveTemp(LifetimeEvents));

	bWasInterrupted = false;

	if (!PlayingRequest.IsValid())
	{
		OnInterrupted.Broadcast();
		Reset();
	}

	return PlayingRequest.IsValid();
}

EUninjectionResult UInjectionCallbackProxy::Uninject()
{
	if(!PlayingRequest.IsValid())
	{
		return EUninjectionResult::Failed;
	}

	UE::UAF::FInjectionUtils::Uninject(PlayingRequest);

	return EUninjectionResult::Succeeded;
}

void UInjectionCallbackProxy::SetVariable(const FAnimNextVariableReference& Variable, const int32& Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UInjectionCallbackProxy::execSetVariable)
{
	using namespace UE::UAF;

	if(!P_THIS->PlayingRequest.IsValid())
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			NSLOCTEXT("InjectionCallbackProxy", "InjectionCallbackProxy_SetVariableInvalidRequestWarning", "Invalid injection request when calling Set Variable")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}
	
	P_GET_STRUCT_REF(FAnimNextVariableReference, Variable);

	const FProperty* VariableProperty = Variable.ResolveProperty();
	if (VariableProperty == nullptr)
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			FText::Format(NSLOCTEXT("InjectionCallbackProxy", "InjectionCallbackProxy_SetVariablePropertyError", "Failed to resolve the property {0} of variable reference"), FText::FromName(Variable.GetName()))
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	if (Variable.GetName() == NAME_None || Variable.GetObject() == nullptr)
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			NSLOCTEXT("InjectionCallbackProxy", "InjectionCallbackProxy_SetVariableInvalidWarning", "Invalid variable supplied to Set Variable")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;

	TUniquePtr<uint8> TempStorage((uint8*)FMemory::Malloc(VariableProperty->GetElementSize(), VariableProperty->GetMinAlignment()));
	VariableProperty->InitializeValue(TempStorage.Get());

	Stack.StepCompiledIn(TempStorage.Get(), VariableProperty->GetClass());

	P_FINISH;

	P_NATIVE_BEGIN;

	P_THIS->PlayingRequest->QueueTask([Variable, Storage = MoveTemp(TempStorage)](const FInstanceTaskContext& InContext)
	{
		const FProperty* VariableProperty = Variable.ResolveProperty();
		check(VariableProperty);	// If this fires the property has been destroyed before we can destroy the temp memory storage

		FAnimNextParamType Type = FAnimNextParamType::FromProperty(VariableProperty);
		InContext.SetVariableInternal(Variable, Type, TConstArrayView<uint8>(Storage.Get(), VariableProperty->GetElementSize()));

		// Clean up
		VariableProperty->DestroyValue(Storage.Get());
	});

	P_NATIVE_END;
}

void UInjectionCallbackProxy::Cancel()
{
	Super::Cancel();
	Uninject();
}

void UInjectionCallbackProxy::OnInjectionCompleted(const UE::UAF::FInjectionRequest& Request)
{
	if (!bWasInterrupted)
	{
		const UE::UAF::EInjectionStatus Status = Request.GetStatus();
		check(!EnumHasAnyFlags(Status, UE::UAF::EInjectionStatus::Interrupted));

		if (EnumHasAnyFlags(Status, UE::UAF::EInjectionStatus::Expired))
		{
			OnInterrupted.Broadcast();
		}
		else
		{
			OnCompleted.Broadcast();
		}
	}

	Reset();
}

void UInjectionCallbackProxy::OnInjectionInterrupted(const UE::UAF::FInjectionRequest& Request)
{
	bWasInterrupted = true;

	OnInterrupted.Broadcast();
}

void UInjectionCallbackProxy::OnInjectionBlendingOut(const UE::UAF::FInjectionRequest& Request)
{
	if (!bWasInterrupted)
	{
		OnBlendOut.Broadcast();
	}
}

void UInjectionCallbackProxy::Reset()
{
	PlayingRequest = nullptr;
	bWasInterrupted = false;
}

void UInjectionCallbackProxy::BeginDestroy()
{
	Reset();

	Super::BeginDestroy();
}
