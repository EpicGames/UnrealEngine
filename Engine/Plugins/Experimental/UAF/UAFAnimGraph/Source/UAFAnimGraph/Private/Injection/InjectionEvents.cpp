// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionEvents.h"

#include "TraitCore/TraitEventList.h"

namespace UE::UAF
{
	void FInjection_InjectEvent::OnExpired(FTraitEventList& OutputEventList)
	{
		auto ActionEvent = MakeTraitEvent<FAnimNextModule_ActionEvent>();
		ActionEvent->ActionFunction = [Request = Request]()
			{
				Request->OnStatusUpdate(EInjectionStatus::Expired);
			};

		OutputEventList.Push(MoveTemp(ActionEvent));
	}

	void FInjection_StatusUpdateEvent::Execute() const
	{
		Request->OnStatusUpdate(Status);
	}

	void FInjection_TimelineUpdateEvent::Execute() const
	{
		Request->OnTimelineUpdate(TimelineState);
	}
}
