// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Injection/InjectionRequest.h"
#include "Injection/InjectionStatus.h"
#include "Module/ModuleEvents.h"
#include "TraitCore/TraitEvent.h"
#include "TraitInterfaces/ITimeline.h"

namespace UE::UAF
{
	/**
	 * Injection Inject Event
	 *
	 * Event raised when an injection request is made.
	 * It encapsulates everything needed to service an injection request.
	 *
	 * If no valid data is provided, this event will request that the input source plays instead.
	 */
	struct FInjection_InjectEvent : public FAnimNextTraitEvent
	{
		DECLARE_ANIM_TRAIT_EVENT(FInjection_PlayEvent, FAnimNextTraitEvent)

		FInjectionRequestPtr Request;

		int32 SerialNumber = 0;

	protected:
		// FAnimNextTraitEvent impl
		virtual void OnExpired(FTraitEventList& OutputEventList) override;
	};

	/**
	 * Injection Uninject Event
	 *
	 * Event raised when an uninject request is made.
	 */
	struct FInjection_UninjectEvent : public FAnimNextTraitEvent
	{
		DECLARE_ANIM_TRAIT_EVENT(FInjection_StopEvent, FAnimNextTraitEvent)

		FInjectionRequestPtr Request;

		int32 SerialNumber = 0;
	};

	/**
	 * Injection Status Update
	 *
	 * Event raised when the status of a request changes.
	 */
	struct FInjection_StatusUpdateEvent : public FAnimNextModule_ActionEvent
	{
		DECLARE_ANIM_TRAIT_EVENT(FInjection_StatusUpdateEvent, FAnimNextModule_ActionEvent)

		// FAnimNextSchedule_ActionEvent impl
		virtual void Execute() const override;

		// The request to update
		FInjectionRequestPtr Request;

		// The current request status
		EInjectionStatus Status = EInjectionStatus::None;
	};

	/**
	 * Injection Timeline Update
	 *
	 * Event raised when a request is playing with its updated timeline progress.
	 */
	struct FInjection_TimelineUpdateEvent : public FAnimNextModule_ActionEvent
	{
		DECLARE_ANIM_TRAIT_EVENT(FInjection_TimelineUpdateEvent, FAnimNextModule_ActionEvent)

		// FAnimNextSchedule_ActionEvent impl
		virtual void Execute() const override;

		// The request to update
		FInjectionRequestPtr Request;

		// The current request timeline state
		FTimelineState TimelineState;
	};
}
