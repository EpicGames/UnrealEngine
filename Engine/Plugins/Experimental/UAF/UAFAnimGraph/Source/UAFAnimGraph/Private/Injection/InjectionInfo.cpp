// Copyright Epic Games, Inc. All Rights Reserved.

#include "InjectionInfo.h"

#include "AnimNextRigVMAsset.h"
#include "Injection/InjectionSite.h"
#include "UAFAssetInstance.h"
#include "Graph/AnimNextAnimGraph.h"

namespace UE::UAF
{

FInjectionInfo::FInjectionInfo(const FUAFAssetInstance& InInstance)
	: Instance(&InInstance)
{
	CacheInfo();
}

void FInjectionInfo::CacheInfo() const
{
	check(Instance);
	check(Instance->GetAsset<UUAFRigVMAsset>());
	DefaultInjectionSite = Instance->GetAsset<UUAFRigVMAsset>()->DefaultInjectionSite;

	InjectionSites.Reset();
	Instance->GetAllVariablesOfType<FAnimNextAnimGraph>(InjectionSites);

	if(DefaultInjectionSite.IsNone() && InjectionSites.Num() > 0)
	{
		DefaultInjectionSite = InjectionSites[0];
	}
}

FAnimNextVariableReference FInjectionInfo::FindInjectionSite(const FInjectionSite& InSite) const
{
	if(!InSite.DesiredSite.IsNone())
	{
		// Linear search all the injection sites
		for(const FAnimNextVariableReference& InjectionSite : InjectionSites)
		{
			if(InSite.DesiredSite == InjectionSite)
			{
				return InjectionSite;
			}
		}
	}

	// Not found - optionally use fallback
	const bool bUseModuleDefault = InSite.DesiredSite.IsNone() || InSite.bUseModuleFallback;
	if(bUseModuleDefault && !DefaultInjectionSite.IsNone())
	{
		return DefaultInjectionSite;
	}

	return FAnimNextVariableReference();
}

}
