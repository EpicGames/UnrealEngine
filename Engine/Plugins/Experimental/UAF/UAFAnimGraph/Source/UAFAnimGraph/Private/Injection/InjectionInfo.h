// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Variables/AnimNextVariableReference.h"

struct FUAFAssetInstance;
struct FAnimNextAnimGraph;
struct FAnimNextModuleInjectionComponent;
struct FAnimNextInjectionSite;

namespace UE::UAF
{
using FInjectionSite = FAnimNextInjectionSite;

// Info used to track injection sites for an AnimNext instance (graph, module etc)
struct FInjectionInfo
{
	FInjectionInfo() = default;

	explicit FInjectionInfo(const FUAFAssetInstance& InInstance);

	// Get the default injection site
	FAnimNextVariableReference GetDefaultInjectionSite() const
	{
		return DefaultInjectionSite;
	}

	// Find an injectable graph variable by name.
	// @param   InSite            The injection site.
	// @return The actual injection site we found (in the case an injection site of None was passed)
	FAnimNextVariableReference FindInjectionSite(const FInjectionSite& InSite) const;

private:
	void CacheInfo() const;

private:
	friend ::FAnimNextModuleInjectionComponent;

	// All injection site variables
	mutable TArray<FAnimNextVariableReference> InjectionSites;

	// Instance we are tracking
	const FUAFAssetInstance* Instance = nullptr;

	// The default injection site for our instance (user-adjustable via SetDefaultInjectionSite) 
	mutable FAnimNextVariableReference DefaultInjectionSite;
};

}
