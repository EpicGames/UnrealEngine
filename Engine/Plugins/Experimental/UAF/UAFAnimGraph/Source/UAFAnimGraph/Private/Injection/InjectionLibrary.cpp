// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionLibrary.h"

#include "Animation/AnimMontage.h"
#include "Component/AnimNextComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InjectionLibrary)

bool UUAFInjectionLibrary::IsSlotActive(const UUAFComponent* UAFComponent, FName SlotNodeName)
{
	if (UAFComponent && SlotNodeName != NAME_None)
	{
		if (const AActor* UAFActor = UAFComponent->GetOwner())
		{
			if (const USkeletalMeshComponent* SkeletalMeshComponent = UAFActor->FindComponentByClass<USkeletalMeshComponent>())
			{
				if (SkeletalMeshComponent->bEnableAnimation)
				{
					return false;
				}
				
				if (const UAnimInstance* UsedAnimInstance = SkeletalMeshComponent->GetAnimInstance())
				{
					for (const FAnimMontageInstance* MontageInstance : UsedAnimInstance->MontageInstances)
					{
						if (MontageInstance->IsActive()
							&& MontageInstance->Montage->IsValidSlot(SlotNodeName))
						{
							return true;
						}
					}
				}
			}
		}
	}
	
	return false;
}