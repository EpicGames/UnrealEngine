// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionRequest.h"

#include "Injection/InjectionEvents.h"
#include "Component/AnimNextComponent.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Logging/StructuredLog.h"
#include "Engine/World.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFWeakSystemReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InjectionRequest)

namespace UE::UAF
{
	bool FInjectionRequest::Inject(FInjectionRequestArgs&& InRequestArgs, FInjectionLifetimeEvents&& InLifetimeEvents, UObject* InHost, FUAFWeakSystemReference InReference)
	{
		check(IsInGameThread());
		if (InHost == nullptr || !InReference.IsValid())
		{
			UE_LOGF(LogAnimation, Warning, "FInjectionRequest::Inject: Failed to inject - Provided host or system reference are not valid.");
			return false;	// Nothing to play
		}

		if (Status != EInjectionStatus::None)
		{
			UE_LOGF(LogAnimation, Log, "FInjectionRequest::Inject: Failed to inject - Request is already playing, can not play again.");
			return false;	// Already playing, cannot play again
		}

		if(!ValidateArgs(InRequestArgs))
		{
			UE_LOGF(LogAnimation, Warning, "FInjectionRequest::Inject: Failed to inject - Args are not valid.");
			return false;	// Invalid args
		}

		UWorld* World = InHost->GetWorld();
		if(World == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "FInjectionRequest::Inject: Failed to inject - No valid World to play in.");
			return false;	// No world to play in
		}

		RequestArgs = MoveTemp(InRequestArgs);
		LifetimeEvents = MoveTemp(InLifetimeEvents);
		WeakHost = InHost;
		Reference = InReference;
		Status = EInjectionStatus::Pending;

		auto InjectEvent = MakeTraitEvent<FInjection_InjectEvent>();
		InjectEvent->Request = AsShared();

		Reference.QueueInputTraitEvent(InjectEvent);

		InjectionEvent = InjectEvent;

		return true;
	}

	void FInjectionRequest::Uninject()
	{
		check(IsInGameThread());
		if (!EnumHasAnyFlags(Status, EInjectionStatus::Playing))
		{
			UE_LOGF(LogAnimation, Log, "FInjectionRequest::Uninject: Could not uninject - The request is not playing.");
			return;	// Not playing
		}
		else if (EnumHasAnyFlags(Status, EInjectionStatus::Interrupted))
		{
			UE_LOGF(LogAnimation, Log, "FInjectionRequest::Uninject: Could not uninject - The request has already been interrupted.");
			return;	// We already got interrupted
		}

		// Cancel our (possibly persistent) event
		if (TSharedPtr<FAnimNextTraitEvent> PinnedInjectionEvent = InjectionEvent.Pin())
		{
			PinnedInjectionEvent->MarkConsumed();
			InjectionEvent = nullptr;
		}

		auto UninjectEvent = MakeTraitEvent<FInjection_UninjectEvent>();
		UninjectEvent->Request = AsShared();

		Reference.QueueInputTraitEvent(UninjectEvent);
	}

	bool FInjectionRequest::ValidateArgs(const FInjectionRequestArgs& InRequestArgs)
	{
		switch(InRequestArgs.Type)
		{
		case EAnimNextInjectionType::InjectObject:
			if(InRequestArgs.Site.DesiredSite.IsNone())
			{
				UE_LOGFMT(LogAnimation, Warning, "FInjectionRequest::ValidateArgs: Missing injection site");
				return false;
			}
			if(InRequestArgs.Object == nullptr)
			{
				UE_LOGFMT(LogAnimation, Warning, "FInjectionRequest::ValidateArgs: Missing object");
				return false;
			}
			break;
		case EAnimNextInjectionType::EvaluationModifier:
			if(InRequestArgs.Object != nullptr)
			{
				UE_LOGFMT(LogAnimation, Warning, "FInjectionRequest::ValidateArgs: Object provided when injecting evaluation modifier");
				return false;
			}
			break;
		}

		return true;
	}

	const FInjectionRequestArgs& FInjectionRequest::GetArgs() const
	{
		return RequestArgs;
	}

	FInjectionRequestArgs& FInjectionRequest::GetMutableArgs()
	{
		return RequestArgs;
	}

	const FInjectionLifetimeEvents& FInjectionRequest::GetLifetimeEvents() const
	{
		return LifetimeEvents;
	}

	FInjectionLifetimeEvents& FInjectionRequest::GetMutableLifetimeEvents()
	{
		return LifetimeEvents;
	}

	EInjectionStatus FInjectionRequest::GetStatus() const
	{
		check(IsInGameThread());
		return Status;
	}

	const FTimelineState& FInjectionRequest::GetTimelineState() const
	{
		check(IsInGameThread());
		ensureMsgf(RequestArgs.bTrackTimelineProgress, TEXT("Attempting to query the timeline state of an injection request that isn't tracking the timeline progress"));
		return TimelineState;
	}

	bool FInjectionRequest::HasExpired() const
	{
		check(IsInGameThread());
		return EnumHasAnyFlags(Status, EInjectionStatus::Expired);
	}

	bool FInjectionRequest::HasCompleted() const
	{
		check(IsInGameThread());
		return EnumHasAnyFlags(Status, EInjectionStatus::Completed);
	}

	bool FInjectionRequest::IsPlaying() const
	{
		check(IsInGameThread());
		return EnumHasAnyFlags(Status, EInjectionStatus::Playing);
	}

	bool FInjectionRequest::IsBlendingOut() const
	{
		check(IsInGameThread());
		return EnumHasAnyFlags(Status, EInjectionStatus::BlendingOut);
	}

	bool FInjectionRequest::WasInterrupted() const
	{
		check(IsInGameThread());
		return EnumHasAnyFlags(Status, EInjectionStatus::Interrupted);
	}

	void FInjectionRequest::OnStatusUpdate(EInjectionStatus NewStatus)
	{
		check(IsInGameThread());
		switch (NewStatus)
		{
		case EInjectionStatus::Playing:
			ensureMsgf(Status == EInjectionStatus::Pending, TEXT("Expected Injection status to be pending, found: %u"), (uint32)Status);
			Status = NewStatus;
			LifetimeEvents.OnStarted.ExecuteIfBound(*this);
			break;
		case EInjectionStatus::Playing | EInjectionStatus::Interrupted:
			ensureMsgf(EnumHasAnyFlags(Status, EInjectionStatus::Playing), TEXT("Expected Injection status to be playing, found: %u"), (uint32)Status);
			EnumAddFlags(Status, EInjectionStatus::Interrupted);
			LifetimeEvents.OnInterrupted.ExecuteIfBound(*this);
			break;
		case EInjectionStatus::BlendingOut:
			ensureMsgf(EnumHasAnyFlags(Status, EInjectionStatus::Playing), TEXT("Expected Injection status to be playing, found: %u"), (uint32)Status);
			EnumAddFlags(Status, EInjectionStatus::BlendingOut);
			LifetimeEvents.OnBlendingOut.ExecuteIfBound(*this);
			break;
		case EInjectionStatus::Completed:
			ensureMsgf(EnumHasAnyFlags(Status, EInjectionStatus::Playing), TEXT("Expected Injection status to be playing, found: %u"), (uint32)Status);
			// Maintain our interrupted status if it was present
			Status = EInjectionStatus::Completed | (Status & EInjectionStatus::Interrupted);
			LifetimeEvents.OnCompleted.ExecuteIfBound(*this);
			break;
		case EInjectionStatus::Expired:
			ensureMsgf(Status == EInjectionStatus::Pending, TEXT("Expected Injection status to be pending, found: %u"), (uint32)Status);
			Status = NewStatus;
			LifetimeEvents.OnCompleted.ExecuteIfBound(*this);
			break;
		default:
			ensureMsgf(false, TEXT("Unsupported Injection status update value: %u"), (uint32)NewStatus);
			break;
		}
	}

	void FInjectionRequest::OnTimelineUpdate(const FTimelineState& NewTimelineState)
	{
		check(IsInGameThread());
		TimelineState = NewTimelineState;
	}

	void FInjectionRequest::ExternalAddReferencedObjects(FReferenceCollector& Collector)
	{
		AddReferencedObjects(Collector);
	}

	void FInjectionRequest::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddPropertyReferencesWithStructARO(FInjectionRequestArgs::StaticStruct(), &RequestArgs);
	}

	void FInjectionRequest::QueueTask(FUniqueInstanceTask&& InTask)
	{
		TaskQueue.Enqueue(MoveTemp(InTask));
	}

	void FInjectionRequest::FlushTasks(const FInstanceTaskContext& InContext)
	{
		// Flush any queued tasks
		while (!TaskQueue.IsEmpty())
		{
			TOptional<FUniqueInstanceTask> Task = TaskQueue.Dequeue();
			check(Task.IsSet());
			Task.GetValue()(InContext);
		}
	}
}
