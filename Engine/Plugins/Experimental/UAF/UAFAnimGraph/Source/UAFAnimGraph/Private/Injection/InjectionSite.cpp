// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionSite.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InjectionSite)

#if WITH_EDITORONLY_DATA
bool FAnimNextInjectionSite::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	return false;
}

void FAnimNextInjectionSite::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DesiredSite = FAnimNextVariableReference(DesiredSiteName_DEPRECATED);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}
#endif
