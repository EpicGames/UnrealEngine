// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionSiteTrait.h"

#include "AnimNextModuleInjectionComponent.h"
#include "InstanceTaskContext.h"
#include "Factory/AnimGraphFactory.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/ITimeline.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Injection/IEvaluationModifier.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModuleInstance.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IInertializerBlend.h"
#include "TraitInterfaces/ISmoothBlend.h"
#include "InjectionEvents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InjectionSiteTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FInjectionSiteTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IGarbageCollection) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	// Trait required interfaces implementation boilerplate
	#define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacroRequired) \
		GeneratorMacroRequired(IBlendStack) \

	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FInjectionSiteTrait::OnInjectEvent) \
		GeneratorMacro(FInjectionSiteTrait::OnUninjectEvent) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FInjectionSiteTrait, TRAIT_INTERFACE_ENUMERATOR, TRAIT_REQUIRED_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR

	void FInjectionSiteTrait::FInjectionTracker::Initialize(FInjectionRequestPtr InRequest, EInjectionTrackerState InState)
	{
		Request = InRequest;
		State = InState;
	}

	void FInjectionSiteTrait::FInjectionTracker::Reset()
	{
		Request = FInjectionRequestPtr();
		State = EInjectionTrackerState::Inactive;
	}

	void FInjectionSiteTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		if(FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
		{
			(void)ModuleInstance->GetOrAddComponent<FAnimNextModuleInjectionComponent>();
		}
	}

	void FInjectionSiteTrait::TrackPendingInjectionRequest(int32 InChildIndex, FInstanceData& InstanceData)
	{
		// Assume that we have a pending tracker before we call this function
		if(ensure(InstanceData.PendingTracker.State != EInjectionTrackerState::Inactive))
		{
			InstanceData.InjectionTrackers.SetNum(FMath::Max(InstanceData.InjectionTrackers.Num(), InChildIndex + 1), EAllowShrinking::No);
			InstanceData.InjectionTrackers[InChildIndex] = InstanceData.PendingTracker;
			InstanceData.PendingTracker.Reset();
		}
	}

	ETraitStackPropagation FInjectionSiteTrait::OnInjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_InjectEvent& Event) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FAnimNextAnimGraph& Graph = SharedData->GetGraph(Binding);

		// Check to see if this injection event was routed to us
		ensure(Event.SerialNumber != 0);
		if (Graph.InjectionData.InjectionSerialNumber == Event.SerialNumber)
		{
			const FAnimNextInjectionRequestArgs& RequestArgs = Event.Request->GetArgs();
			ensureMsgf(!InstanceData->PendingRequest.IsValid(), TEXT("Injection site %s already contained a pending request, it will be overwritten"), *RequestArgs.Site.DesiredSite.GetName().ToString());

			// Overwrite any request we might have, we'll pick it up on the next update
			InstanceData->PendingRequest.Reset();
			InstanceData->PendingRequest.Request = Event.Request;

			Event.MarkConsumed();
		}

		return ETraitStackPropagation::Continue;
	}

	ETraitStackPropagation FInjectionSiteTrait::OnUninjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_UninjectEvent& Event) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FAnimNextAnimGraph& Graph = SharedData->GetGraph(Binding);

		// Check to see if this uninjection event was routed to us
		ensure(Event.SerialNumber != 0);
		if (Graph.InjectionData.InjectionSerialNumber == Event.SerialNumber)
		{
			// Reset any pending request we might have, and cancel it
			InstanceData->PendingRequest.Reset();
			InstanceData->PendingRequest.bStop = true;

			Event.MarkConsumed();
		}

		return ETraitStackPropagation::Continue;
	}

	void FInjectionSiteTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		auto TranslateBlendMode = [](EAnimNextInjectionBlendMode InBlendMode)
		{
			switch(InBlendMode)
			{
			case EAnimNextInjectionBlendMode::Standard:
				return IBlendStack::EBlendMode::Standard;
			case EAnimNextInjectionBlendMode::Inertialization:
				return IBlendStack::EBlendMode::Inertialization;
			default:
				checkNoEntry();
				break;
			}
			return IBlendStack::EBlendMode::Standard;
		};

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IBlendStack> BlendStackTrait;
		ensure(Binding.GetStackInterface(BlendStackTrait));

		IBlendStack::FGraphRequestPtr ActiveGraph = nullptr;
		int32 ActiveChildIndex = BlendStackTrait.GetActiveGraph(Context, ActiveGraph);
		const bool bHasActiveSubGraph = ActiveGraph != nullptr;

		const FAnimNextAnimGraph& Graph = SharedData->GetGraph(Binding);
		const bool bInjectionSiteChanged = !Graph.IsEqualForInjectionSiteChange(InstanceData->CachedGraph);
		if(bInjectionSiteChanged)
		{
			InstanceData->CachedGraph = Graph;
		}

		bool bJustTransitioned = false;
		if (InstanceData->PendingRequest.IsValid() || bInjectionSiteChanged || !bHasActiveSubGraph)
		{
			// Grab and clear our pending request, if any
			const FInjectionRequestPtr Request = InstanceData->PendingRequest.Request;
			InstanceData->PendingRequest.Reset();

			bool bRequestValid = false;
			IBlendStack::FGraphRequest BlendStackRequest;
			EInjectionTrackerState State = EInjectionTrackerState::Inactive;
			if (Request && Request->GetArgs().Object)
			{
				// Push request from an event
				State = EInjectionTrackerState::ActiveEvent;

				FAnimNextInjectionRequestArgs& RequestArgs = Request->GetMutableArgs();
				BlendStackRequest.Type = IBlendStack::EGraphRequestType::Owned;
				BlendStackRequest.FactoryParams = MoveTemp(RequestArgs.FactoryParams);
				BlendStackRequest.AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, RequestArgs.Object, BlendStackRequest.FactoryParams);
				bRequestValid = BlendStackRequest.AnimationGraph != nullptr;
				BlendStackRequest.BlendArgs = RequestArgs.BlendInSettings.Blend;
				BlendStackRequest.BlendMode = TranslateBlendMode(RequestArgs.BlendInSettings.BlendMode);
			}
			else if(Graph.Asset)
			{
				// Push request from a pin-value change
				State = EInjectionTrackerState::ActivePin;

				BlendStackRequest.Type = IBlendStack::EGraphRequestType::Owned;
				BlendStackRequest.AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, Graph.Asset, BlendStackRequest.FactoryParams);
				bRequestValid = BlendStackRequest.AnimationGraph != nullptr;
				const FAnimNextInjectionBlendSettings& DefaultBlendInSettings = SharedData->GetDefaultBlendInSettings(Binding);
				BlendStackRequest.BlendArgs = DefaultBlendInSettings.Blend;
				BlendStackRequest.BlendMode = TranslateBlendMode(DefaultBlendInSettings.BlendMode);
			}
			else
			{
				// No request, or invalid/unhandled, so push our source child
				State = EInjectionTrackerState::ActiveSource;

				bRequestValid = true;
				BlendStackRequest.Type = IBlendStack::EGraphRequestType::Child;
				BlendStackRequest.ChildPtr = Context.AllocateNodeInstance(Binding, SharedData->Source);

				const FAnimNextInjectionBlendSettings& DefaultBlendInSettings = SharedData->GetDefaultBlendInSettings(Binding);
				BlendStackRequest.BlendArgs = DefaultBlendInSettings.Blend;
				BlendStackRequest.BlendMode = TranslateBlendMode(DefaultBlendInSettings.BlendMode);
			}

			if (bRequestValid)
			{
				if (bHasActiveSubGraph)
				{
					ensure(InstanceData->InjectionTrackers.IsValidIndex(ActiveChildIndex));

					// Queue our status update
					const FInjectionTracker& ActiveInjectionTracker = InstanceData->InjectionTrackers[ActiveChildIndex];
					if (ActiveInjectionTracker.State == EInjectionTrackerState::ActiveEvent)
					{
						auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
						StatusUpdateEvent->Request = ActiveInjectionTracker.Request;
						StatusUpdateEvent->Status = EInjectionStatus::Playing | EInjectionStatus::Interrupted;

						Context.RaiseOutputTraitEvent(StatusUpdateEvent);
					}
				}
				else
				{
					// No current active subgraph, so blend instantly so we either:
					// - Ensure that source doesnt blend from refpose on first becoming relevant
					// - Dont blend from source when we first become relevant with a valid request
					BlendStackRequest.BlendArgs.BlendTime = 0.0f;
				}

				// Track a new request
				InstanceData->PendingTracker.Initialize(Request, State);
				ActiveChildIndex = BlendStackTrait.PushGraph(Context, MoveTemp(BlendStackRequest));

				// Make sure we tracked it after blending
				ensure(!InstanceData->PendingTracker.IsRequestValid());
				ensure(InstanceData->InjectionTrackers.IsValidIndex(ActiveChildIndex) &&
					InstanceData->InjectionTrackers[ActiveChildIndex].Request == Request && 
					InstanceData->InjectionTrackers[ActiveChildIndex].State == State);

				bJustTransitioned = true;
			}
		}

		float CurrentRequestTimeLeft = FLT_MAX;

		const int32 NumInjectionTrackers = InstanceData->InjectionTrackers.Num();
		for (int32 TrackerIndex = 0; TrackerIndex < NumInjectionTrackers; ++TrackerIndex)
		{
			const FInjectionTracker& InjectionTracker = InstanceData->InjectionTrackers[TrackerIndex];
			if (InjectionTracker.State != EInjectionTrackerState::ActiveEvent)
			{
				continue;	// We don't care about this injection request
			}

			// Flush any pending tasks (allows variable sets etc. queued from GT)
			FAnimNextGraphInstance* GraphInstance = BlendStackTrait.GetGraphInstance(Context, TrackerIndex);
			check(GraphInstance != nullptr);
			FInstanceTaskContext TaskContext(*GraphInstance);
			InjectionTracker.Request->FlushTasks(TaskContext);

			// Broadcast our timeline progress
			TTraitBinding<ITimeline> TimelineTrait;
			if (Binding.GetStackInterface(TimelineTrait))
			{
				FTimelineState ChildState;
				ensure(TimelineTrait.GetState(Context, ChildState));

				// Compute how much time is left before the timeline ends (can be negative if we overshoot)
				// Note when looping/infinite we dont have a 'time left', so we leave it at FLT_MAX
				if (ActiveChildIndex == TrackerIndex && !(ChildState.IsLooping() || !ChildState.IsFinite()))
				{
					const float ChildCurrentPosition = ChildState.GetPosition();
					const float ChildNextPosition = ChildCurrentPosition + (TraitState.GetDeltaTime() * ChildState.GetPlayRate());
					const float TimeToEnd =
						ChildNextPosition >= ChildCurrentPosition ?	// Is moving forward?
						ChildState.GetDuration() - ChildNextPosition :
						ChildNextPosition;

					CurrentRequestTimeLeft = TimeToEnd;
				}

				// Only raise a timeline update event if we care about it
				if (InjectionTracker.Request->GetArgs().bTrackTimelineProgress)
				{
					auto TimelineUpdateEvent = MakeTraitEvent<FInjection_TimelineUpdateEvent>();
					TimelineUpdateEvent->Request = InjectionTracker.Request;

					// We don't have too many options here:
					//    - We can have one frame delay (as we do now)
					//    - We could use the speculative estimate (from above) as our new state, but this may not be fully accurate (e.g. ignores sync groups)
					//    - We could query the timeline during PostUpdate, but this would ignore sync groups
					//    - We could add a new graph instance component and hook PostUpdate, but then we have an ordering issue with the sync group component
					//    - To be fully accurate, the timeline would need to broadcast when it changes, and so we would need to register callbacks on it and manage them
					TimelineUpdateEvent->TimelineState = ChildState;

					Context.RaiseOutputTraitEvent(TimelineUpdateEvent);
				}
			}
		}

		// Check if we are blending out
		if (!bJustTransitioned && ActiveChildIndex != INDEX_NONE)
		{
			const FInjectionTracker& ActiveInjectionTracker = InstanceData->InjectionTrackers[ActiveChildIndex];
			if(ActiveInjectionTracker.State == EInjectionTrackerState::ActiveEvent)
			{
				const FAnimNextInjectionRequestArgs& RequestArgs = ActiveInjectionTracker.Request->GetArgs();

				const float BlendOutTime = RequestArgs.BlendOutSettings.Blend.BlendTime;

				// Only do this if we are automatically blending out
				if (ActiveInjectionTracker.Request->GetArgs().LifetimeType == EAnimNextInjectionLifetimeType::Auto &&
					CurrentRequestTimeLeft <= BlendOutTime)
				{
					// We are ready to start blending out
					{
						auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
						StatusUpdateEvent->Request = ActiveInjectionTracker.Request;
						StatusUpdateEvent->Status = EInjectionStatus::BlendingOut;

						Context.RaiseOutputTraitEvent(StatusUpdateEvent);
					}

					// Blend to source child or do we still have an active pin input?
					bool bBlendedToPin = false;
					if(Graph.Asset != nullptr)
					{
						// Add a request to blend in our fallback graph using blend out time
						bool bRequestValid = true;
						IBlendStack::FGraphRequest BlendStackRequest;
						BlendStackRequest.Type = IBlendStack::EGraphRequestType::Owned;
						BlendStackRequest.AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, Graph.Asset, BlendStackRequest.FactoryParams);
						bRequestValid = BlendStackRequest.AnimationGraph != nullptr;
						BlendStackRequest.BlendArgs.BlendTime = BlendOutTime;
						BlendStackRequest.BlendMode = TranslateBlendMode(RequestArgs.BlendOutSettings.BlendMode);

						if (bRequestValid)
						{
							InstanceData->PendingTracker.Initialize(FInjectionRequestPtr(), EInjectionTrackerState::ActivePin);
							ActiveChildIndex = BlendStackTrait.PushGraph(Context, MoveTemp(BlendStackRequest));

							// Make sure we tracked it after blending
							ensure(InstanceData->InjectionTrackers.IsValidIndex(ActiveChildIndex) &&
								InstanceData->InjectionTrackers[ActiveChildIndex].Request == FInjectionRequestPtr() && 
								InstanceData->InjectionTrackers[ActiveChildIndex].State == EInjectionTrackerState::ActivePin);

							bBlendedToPin = true;
						}
					}

					if (!bBlendedToPin)
					{
						// Add a request to blend in our source using blend out time
						IBlendStack::FGraphRequest BlendStackRequest;
						BlendStackRequest.Type = IBlendStack::EGraphRequestType::Child;
						BlendStackRequest.ChildPtr = Context.AllocateNodeInstance(Binding, SharedData->Source);
						BlendStackRequest.BlendArgs.BlendTime = BlendOutTime;

						InstanceData->PendingTracker.Initialize(FInjectionRequestPtr(), EInjectionTrackerState::ActiveSource);
						ActiveChildIndex = BlendStackTrait.PushGraph(Context, MoveTemp(BlendStackRequest));

						// Make sure we tracked it after blending
						ensure(InstanceData->InjectionTrackers.IsValidIndex(ActiveChildIndex) &&
							InstanceData->InjectionTrackers[ActiveChildIndex].Request == FInjectionRequestPtr() && 
							InstanceData->InjectionTrackers[ActiveChildIndex].State == EInjectionTrackerState::ActiveSource);
					}
				}
			}
		}

		// Update traits below us
		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FInjectionSiteTrait::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		TrackPendingInjectionRequest(ChildIndex, *InstanceData);
		if (ensure(InstanceData->InjectionTrackers.IsValidIndex(ChildIndex)))
		{
			FInjectionTracker& InjectionTracker = InstanceData->InjectionTrackers[ChildIndex];
			if (InjectionTracker.State == EInjectionTrackerState::ActiveEvent)
			{
				// Queue our status update for this newly playing injection
				auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
				StatusUpdateEvent->Request = InjectionTracker.Request;
				StatusUpdateEvent->Status = EInjectionStatus::Playing;
				Context.RaiseOutputTraitEvent(StatusUpdateEvent);
			}
		}

		// Update traits below us
		IDiscreteBlend::OnBlendInitiated(Context, Binding, ChildIndex);
	}

	void FInjectionSiteTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (ChildIndex != INDEX_NONE && ensure(InstanceData->InjectionTrackers.IsValidIndex(ChildIndex)))
		{
			FInjectionTracker& InjectionTracker = InstanceData->InjectionTrackers[ChildIndex];
			if (InjectionTracker.State == EInjectionTrackerState::ActiveEvent)
			{
				// Queue our status update
				auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
				StatusUpdateEvent->Request = InjectionTracker.Request;
				StatusUpdateEvent->Status = EInjectionStatus::Completed;
				Context.RaiseOutputTraitEvent(StatusUpdateEvent);
			}

			// This can release our request instance's last reference
			InjectionTracker.Reset();
		}

		// Update traits below us
		IDiscreteBlend::OnBlendTerminated(Context, Binding, ChildIndex);
	}

	void FInjectionSiteTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->PendingRequest.Request)
		{
			InstanceData->PendingRequest.Request->AddReferencedObjects(Collector);
		}

		for (FInjectionTracker& InjectionTracker : InstanceData->InjectionTrackers)
		{
			if (InjectionTracker.Request)
			{
				InjectionTracker.Request->AddReferencedObjects(Collector);
			}
		}
	}

	void FInjectionSiteTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		// Forward to traits below us to ensure we have tasks populated before we modify
		IEvaluate::PreEvaluate(Context, Binding);

		// Apply evaluation injection if present
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FAnimNextAnimGraph& Graph = SharedData->GetGraph(Binding);
		if(TSharedPtr<IEvaluationModifier> EvaluationModifier = Graph.InjectionData.EvaluationModifier.Pin())
		{
			EvaluationModifier->PreEvaluate(Context);
		}
	}

	void FInjectionSiteTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		// Forward to traits below us to ensure we have tasks populated before we modify
		IEvaluate::PostEvaluate(Context, Binding);

		// Apply evaluation injection if present
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FAnimNextAnimGraph& Graph = SharedData->GetGraph(Binding);
		if(TSharedPtr<IEvaluationModifier> EvaluationModifier = Graph.InjectionData.EvaluationModifier.Pin())
		{
			EvaluationModifier->PostEvaluate(Context);
		}
	}
}
