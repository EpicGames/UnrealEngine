// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/InjectionUtils.h"
#include "GraphInterfaces/AnimNextNativeDataInterface_AnimSequencePlayer.h"
#include "Animation/AnimSequence.h"
#include "Component/AnimNextComponent.h"
#include "Logging/StructuredLog.h"

namespace UE::UAF
{
	FInjectionRequestPtr FInjectionUtils::Inject(UUAFComponent* InComponent, FInjectionRequestArgs&& InArgs, FInjectionLifetimeEvents&& InLifetimeEvents)
	{
		return Inject(InComponent, InComponent->GetSystemReference(), MoveTemp(InArgs), MoveTemp(InLifetimeEvents));
	}

	FInjectionRequestPtr FInjectionUtils::Inject(UObject* InHost, FUAFWeakSystemReference InSystemReference, FInjectionRequestArgs&& InArgs, FInjectionLifetimeEvents&& InLifetimeEvents)
	{
		check(IsInGameThread());

		FInjectionRequestPtr Request = MakeInjectionRequest();
		if(Request->Inject(MoveTemp(InArgs), MoveTemp(InLifetimeEvents), InHost, InSystemReference))
		{
			return Request;
		}
		return nullptr;
	}

	void FInjectionUtils::Uninject(FInjectionRequestPtr InInjectionRequest)
	{
		if(!InInjectionRequest.IsValid())
		{
			return;
		}
		InInjectionRequest->Uninject();
	}

	FInjectionRequestPtr FInjectionUtils::PlayAnim(
		UUAFComponent* InComponent,
		const FInjectionSite& InSite,
		UAnimSequence* InAnimSequence,
		FPlayAnimArgs&& InArgs,
		const FInjectionBlendSettings& InBlendInSettings,
		const FInjectionBlendSettings& InBlendOutSettings,
		FInjectionLifetimeEvents&& InLifetimeEvents)
	{
		return PlayAnimHandle(
			InComponent,
			InComponent->GetSystemReference(),
			InSite,
			InAnimSequence,
			MoveTemp(InArgs),
			InBlendInSettings,
			InBlendOutSettings,
			MoveTemp(InLifetimeEvents));
	}

	FInjectionRequestPtr FInjectionUtils::PlayAnimHandle(
		UObject* InHost,
		FUAFWeakSystemReference InSystemReference,
		const FInjectionSite& InSite,
		UAnimSequence* InAnimSequence,
		FPlayAnimArgs&& InArgs,
		const FInjectionBlendSettings& InBlendInSettings,
		const FInjectionBlendSettings& InBlendOutSettings,
		FInjectionLifetimeEvents&& InLifetimeEvents)
	{
		check(IsInGameThread());

		FInjectionRequestArgs RequestArgs;
		RequestArgs.Site = InSite;
		RequestArgs.Object = InAnimSequence;
		RequestArgs.Type = EAnimNextInjectionType::InjectObject;
		RequestArgs.BlendInSettings = InBlendInSettings;
		RequestArgs.BlendOutSettings = InBlendOutSettings;
		RequestArgs.LifetimeType = InArgs.LifetimeType;

		if (!InArgs.FactoryParams.IsValid())
		{
			RequestArgs.FactoryParams.AddTraitStruct<FSequencePlayerData>(ETraitVariableMapping::All, 0);
			RequestArgs.FactoryParams.AddInitializeTask([InAnimSequence, PlayRate = InArgs.PlayRate, StartPosition = InArgs.StartPosition, LoopMode = InArgs.LoopMode](const FInstanceTaskContext& InInstance)
			{
				InInstance.AccessVariablesStruct<FSequencePlayerData>([InAnimSequence, PlayRate, StartPosition, LoopMode](FSequencePlayerData& AnimSequencePlayer)
				{
					AnimSequencePlayer.AnimSequence = InAnimSequence;
					AnimSequencePlayer.PlayRate = PlayRate;
					AnimSequencePlayer.StartPosition = StartPosition;
					AnimSequencePlayer.LoopMode = LoopMode;
				});
			});
		}
		else
		{
			RequestArgs.FactoryParams = MoveTemp(InArgs.FactoryParams);
		}

		return Inject(InHost, InSystemReference, MoveTemp(RequestArgs), MoveTemp(InLifetimeEvents));
	}

	FInjectionRequestPtr FInjectionUtils::InjectAsset(
		UUAFComponent* InComponent,
		const FInjectionSite& InSite,
		UObject* InAsset,
		FAnimNextFactoryParams&& InFactoryParams,
		const FInjectionBlendSettings& InBlendInSettings,
		const FInjectionBlendSettings& InBlendOutSettings,
		FInjectionLifetimeEvents&& InLifetimeEvents)
	{
		return InjectAsset(
			InComponent,
			InComponent->GetSystemReference(),
			InSite,
			InAsset,
			MoveTemp(InFactoryParams),
			InBlendInSettings,
			InBlendOutSettings,
			MoveTemp(InLifetimeEvents));
	}

	FInjectionRequestPtr FInjectionUtils::InjectAsset(
		UObject* InHost,
		FUAFWeakSystemReference InSystemReference,
		const FInjectionSite& InSite,
		UObject* InAsset,
		FAnimNextFactoryParams&& InFactoryParams,
		const FInjectionBlendSettings& InBlendInSettings,
		const FInjectionBlendSettings& InBlendOutSettings,
		FInjectionLifetimeEvents&& InLifetimeEvents)
	{
		check(IsInGameThread());

		FInjectionRequestArgs RequestArgs;
		RequestArgs.Site = InSite;
		RequestArgs.Object = InAsset;
		RequestArgs.Type = EAnimNextInjectionType::InjectObject;
		RequestArgs.BlendInSettings = InBlendInSettings;
		RequestArgs.BlendOutSettings = InBlendOutSettings;
		RequestArgs.FactoryParams = MoveTemp(InFactoryParams);

		return Inject(InHost, InSystemReference, MoveTemp(RequestArgs), MoveTemp(InLifetimeEvents));
	}

	FInjectionRequestPtr FInjectionUtils::InjectEvaluationModifier(
		UUAFComponent* InComponent,
		const TSharedRef<IEvaluationModifier>& InEvaluationModifier,
		const FInjectionSite& InSite)
	{
		return InjectEvaluationModifier(
			InComponent,
			InComponent->GetSystemReference(),
			InEvaluationModifier,
			InSite);
	}

	FInjectionRequestPtr FInjectionUtils::InjectEvaluationModifier(
		UObject* InHost,
		FUAFWeakSystemReference InSystemReference,
		const TSharedRef<IEvaluationModifier>& InEvaluationModifier,
		const FInjectionSite& InSite)
	{
		check(IsInGameThread());

		FInjectionRequestArgs RequestArgs;
		RequestArgs.Site = InSite;
		RequestArgs.Type = EAnimNextInjectionType::EvaluationModifier;
		RequestArgs.EvaluationModifier = InEvaluationModifier;

		return Inject(InHost, InSystemReference, MoveTemp(RequestArgs));
	}
}
