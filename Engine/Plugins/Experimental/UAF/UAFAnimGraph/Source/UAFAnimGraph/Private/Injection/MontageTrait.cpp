// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/MontageTrait.h"

#include "InstanceTaskContext.h"
#include "Factory/AnimGraphFactory.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "EvaluationVM/Tasks/PushAnimSequenceKeyframe.h"
#include "Animation/AnimMontage.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"
#include "EvaluationVM/Tasks/PushReferenceKeyframe.h"
#include "Module/UAFNotifyDispatcherComponent.h"
#include "Traits/NotifyFilterTrait.h"
#include "Injection/UAFMontageComponent.h"
#include "Graph/SyncGroup_GraphInstanceComponent.h"
#include "EvaluationVM/Tasks/ApplyAdditiveKeyframe.h"
#include "EvaluationVM/Tasks/BlendKeyframesPerBone.h"
#include "EvaluationVM/Tasks/ApplyAdditiveKeyframePerBone.h"
#include "UAFLogging.h"
#include "Engine/SkeletalMesh.h"
#include "Traits/Inertialization.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FMontageTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdateTraversal) \
		GeneratorMacro(ITimeline) \
		GeneratorMacro(IGroupSynchronization) \
		GeneratorMacro(IGarbageCollection) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMontageTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FMontageTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		if(FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
		{
			// Cache the montage component and create it lazily if it hasn't been created yet. 
			MontageComponent = &ModuleInstance->GetOrAddComponent<FUAFMontageComponent>();
		}

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (!InstanceData->Source.IsValid())
		{
			InstanceData->Source = Context.AllocateNodeInstance(Binding, SharedData->Source);
		}

		InstanceData->SyncGroupParams.bMatchSyncPoint = false;
		InstanceData->SyncGroupParams.SyncMode = EAnimGroupSynchronizationMode::SynchronizeUsingGroupName;
		// Montage will always be the leader because they cannot sync to any other animation within UAF as they are advanced outside of UAF
		// Instead we solely use montages to drive synchronization of other assets in the UAF system
		InstanceData->SyncGroupParams.GroupRole = EAnimGroupSynchronizationRole::AlwaysLeader;
	}

	void FMontageTrait::FInstanceData::ResetData()
	{
		SourceWeight = 1.0f;
		PoseWeights.Reset();
		AdditivePoseWeights.Reset();
		EvalTaskDataForSlot.Reset();
		AdditiveEvalTaskDataForSlot.Reset();
		BlendDataForBlendProfiles.Reset();
		AdditiveBlendDataForBlendProfiles.Reset();
		SkeletonDataForBlendProfiles.Reset();
		bReturnPreUpdateState = true;
		SyncCandidateIndex = INDEX_NONE;
	}

	void FMontageTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PreUpdate(Context, Binding, TraitState);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		InstanceData->DeltaTime = TraitState.GetDeltaTime();
		InstanceData->LastUpdateSourceWeight = InstanceData->SourceWeight;
		InstanceData->ResetData();

		const FUAFMontageComponent* MontageComponent = InstanceData->MontageComponent;
		if (MontageComponent == nullptr)
		{
			UAF_TRAIT_LOG(Warning, TEXT("FMontageTrait::PreUpdate: Could not update montages - There is no valid MontageComponent."));
			return;
		}
		
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FName CurrentSlotName = SharedData->GetSlotName(Binding);
		if (CurrentSlotName.IsNone())
		{
			UAF_TRAIT_LOG(Warning, TEXT("FMontageTrait::PreUpdate: Could not update montages - Trait has not valid slot name."));
			return;
		}
		
		// Raise Inertialization events if applicable
		if (MontageComponent->HasActiveInertializationRequests())
		{
			const USkeleton* Skeleton =	Context.GetBindingObject().IsValid() && Context.GetBindingObject()->GetSkeletalMeshAsset() 
										? Context.GetBindingObject()->GetSkeletalMeshAsset()->GetSkeleton() 
										: nullptr;
			
			if (Skeleton)
			{
				const FName SlotGroupName = Skeleton->GetSlotGroupName(CurrentSlotName);
				if (const ::FInertializationRequest* ActiveRequest = MontageComponent->GetInertializationRequestForGroupName(SlotGroupName))
				{
					TSharedPtr<FAnimNextInertializationRequestEvent> InertializationRequestEvent = MakeTraitEvent<FAnimNextInertializationRequestEvent>();
					InertializationRequestEvent->Request.BlendTime = ActiveRequest->Duration;
					Context.RaiseOutputTraitEvent(InertializationRequestEvent);
				}
			}
		}

		if (MontageComponent->MontageEvalData.IsEmpty())
		{
			return;
		}
		
		InstanceData->bUsesBlendProfiles = false;
		for (const FUAFMontageEvalData& EvalData : MontageComponent->MontageEvalData)
		{
			if (EvalData.ActiveBlendProfile != nullptr)
			{
				InstanceData->bUsesBlendProfiles = true;
				break;
			}
		}

		int32 NumBones = 0;
		if (InstanceData->bUsesBlendProfiles)
		{
			const TWeakObjectPtr<const USkeletalMeshComponent> RefSkelMeshComp = Context.GetBindingObject();
			if (RefSkelMeshComp.IsValid())
			{
				NumBones = RefSkelMeshComp->GetNumBones();
			}
		}
		
		const bool bIsSyncDisabled = SharedData->GetbDisableSynchronization(Binding);
		float TotalMontageWeight = 0.0f;
		float TotalNonAdditiveMontageWeight = 0.0f;
		float SyncCandidate_HeighestWeight = -1.0f;
		int32 SyncCandidate_HeighestWeightIndex = INDEX_NONE;
		for (int32 EvalDataIndex = 0; EvalDataIndex < MontageComponent->MontageEvalData.Num(); ++EvalDataIndex)
		{
			const FUAFMontageEvalData& EvalData = MontageComponent->MontageEvalData[EvalDataIndex];
			if (EvalData.Montage != nullptr)
			{
				const UAnimMontage* const Montage = EvalData.Montage.Get();
				if (Montage->IsValidSlot(CurrentSlotName))
				{
					if (const FAnimTrack* AnimTrack = EvalData.Montage->GetAnimationData(CurrentSlotName))
					{
						const float ClampedTime = FMath::Clamp(EvalData.MontagePosition, 0.f, AnimTrack->GetLength());
						if (const FAnimSegment* AnimSegment = AnimTrack->GetSegmentAtTime(ClampedTime))
						{
							float AnimPosition = 0;
							if (UAnimSequence* AnimSegmentSequence = Cast<UAnimSequence>(AnimSegment->GetAnimationData(ClampedTime, AnimPosition)))
							{
								const float MontageWeight = EvalData.BlendInfo.GetBlendedValue();
								TotalMontageWeight += MontageWeight;
								
								if (AnimTrack->IsAdditive())
								{
									InstanceData->AdditiveEvalTaskDataForSlot.Emplace(AnimPosition, AnimSegmentSequence);
									if (!InstanceData->bUsesBlendProfiles)
									{
										InstanceData->AdditivePoseWeights.Add(MontageWeight);
									}
								}
								else
								{
									InstanceData->EvalTaskDataForSlot.Emplace(AnimPosition, AnimSegmentSequence);
									TotalNonAdditiveMontageWeight += MontageWeight;
									if (!InstanceData->bUsesBlendProfiles)
									{
										InstanceData->PoseWeights.Add(MontageWeight);
									}
								}

								if (InstanceData->bUsesBlendProfiles)
								{
									FBlendSampleData& SampleBlendData = AnimTrack->IsAdditive() ? InstanceData->AdditiveBlendDataForBlendProfiles.AddDefaulted_GetRef() : InstanceData->BlendDataForBlendProfiles.AddDefaulted_GetRef();
									SampleBlendData.TotalWeight = MontageWeight;
									SampleBlendData.PerBoneBlendData.AddZeroed(NumBones);

									const UBlendProfile* BlendProfile = EvalData.ActiveBlendProfile;
									InstanceData->SkeletonDataForBlendProfiles.Add(BlendProfile ? BlendProfile->GetSkeleton() : nullptr);

;									for (int32 PerBoneIndex = 0; PerBoneIndex < SampleBlendData.PerBoneBlendData.Num(); ++PerBoneIndex)
									{
										// If we have a active blend profile we calculate the bone weight according to its settings
										// Otherwise we fall back to use the pose weight as bone weight
										if (BlendProfile)
										{
											const float BoneBlendScale = BlendProfile->GetBoneBlendScale(PerBoneIndex);
											const float BoneWeight = UBlendProfile::CalculateBoneWeight(BoneBlendScale, BlendProfile->GetMode(), EvalData.BlendInfo, EvalData.BlendStartAlpha, MontageWeight, false);
											SampleBlendData.PerBoneBlendData[PerBoneIndex] = BoneWeight;
										}
										else
										{
											SampleBlendData.PerBoneBlendData[PerBoneIndex] = MontageWeight;
										}
									}
								}

								if (!bIsSyncDisabled)
								{
									const bool bIsSynchronizationCandidate = !EvalData.Montage->SyncGroup.IsNone() && EvalData.BlendInfo.GetDesiredValue() == 0 && EvalData.BlendInfo.IsComplete() == false;
									if (bIsSynchronizationCandidate && MontageWeight > SyncCandidate_HeighestWeight)
									{
										SyncCandidate_HeighestWeight = MontageWeight;
										SyncCandidate_HeighestWeightIndex = EvalDataIndex;
									}
								}
							}
						}
					}
				}
			}
		}

		// Re-Normalize pose weights 
		if (InstanceData->bUsesBlendProfiles)
		{
			if (TotalNonAdditiveMontageWeight > (1.0f + ZERO_ANIMWEIGHT_THRESH))
			{
				for (FBlendSampleData& BlendSampleEntry : InstanceData->BlendDataForBlendProfiles)
				{
					BlendSampleEntry.TotalWeight /= TotalNonAdditiveMontageWeight;
				}
			}
		}
		else
		{
			if (TotalMontageWeight > (1.f + ZERO_ANIMWEIGHT_THRESH))
			{
				// If we have to re-normalize the weights, we also have to recalculate the total non-additive weight 
				TotalNonAdditiveMontageWeight = 0.0f;
				for (int32 Index = 0; Index < InstanceData->PoseWeights.Num(); ++Index)
				{
					InstanceData->PoseWeights[Index] /= TotalMontageWeight;
					TotalNonAdditiveMontageWeight += InstanceData->PoseWeights[Index];
				}

				// To maintain parity with the current ABP implementation we normalize additives when no blend profiles are used 
				for (int32 Index = 0; Index < InstanceData->AdditivePoseWeights.Num(); ++Index)
				{
					InstanceData->AdditivePoseWeights[Index] /= TotalMontageWeight;
				}
			}	
		}	
		
		// Calculate weight for the source pose 
		InstanceData->SourceWeight = 1.0f - FMath::Clamp(TotalNonAdditiveMontageWeight, 0.0f, 1.0f);

		TArray<float> TotalBoneWeights;
		TotalBoneWeights.SetNumZeroed(NumBones);
		if (InstanceData->bUsesBlendProfiles)
		{
			// Calculate non-additive total bone weights 
			for (const FBlendSampleData& BlendSampleEntry : InstanceData->BlendDataForBlendProfiles)
			{
				for (int32 PerBoneIndex = 0; PerBoneIndex < NumBones; ++PerBoneIndex)
				{
					if (ensureMsgf(BlendSampleEntry.PerBoneBlendData.IsValidIndex(PerBoneIndex), TEXT("Montage Bone Weight Normalization: Invalid per bone blend data setup!")))
					{
						TotalBoneWeights[PerBoneIndex] += BlendSampleEntry.PerBoneBlendData[PerBoneIndex];
					}
				}
			}

			// Re-Normalize non-additive bone weights 
			const float NormalizeThreshold = FAnimWeight::IsRelevant(InstanceData->SourceWeight) ? (1.f + ZERO_ANIMWEIGHT_THRESH) : ZERO_ANIMWEIGHT_THRESH;
			for (int32 PerBoneIndex = 0; PerBoneIndex < NumBones; ++PerBoneIndex)
			{
				float NormalizedBoneTotal = 0.0f;
				const float TotalBoneWeight = TotalBoneWeights[PerBoneIndex];
				const int32 NumBlendSamples = InstanceData->BlendDataForBlendProfiles.Num();
				for (FBlendSampleData& BlendSampleEntry : InstanceData->BlendDataForBlendProfiles)
				{
					if (TotalBoneWeight > NormalizeThreshold)
					{
						BlendSampleEntry.PerBoneBlendData[PerBoneIndex] /= TotalBoneWeight;
					}
					else
					{
						if (!FAnimWeight::IsRelevant(InstanceData->SourceWeight))
						{
							BlendSampleEntry.PerBoneBlendData[PerBoneIndex] = 1.0f / NumBlendSamples;
						}
					}

					NormalizedBoneTotal += BlendSampleEntry.PerBoneBlendData[PerBoneIndex];
				}

				TotalBoneWeights[PerBoneIndex] = NormalizedBoneTotal;
			}
		}
		
		// If we are only playing additive animations we do not add the source weight as we want to apply 100% of it and not blend it in
		const bool bOnlyPlayingAdditives = InstanceData->EvalTaskDataForSlot.Num() == 0 && InstanceData->AdditiveEvalTaskDataForSlot.Num() > 0;
		const bool bAlwaysUpdateSource = SharedData->GetbAlwaysUpdateSource(Binding);
		if ((FAnimWeight::IsRelevant(InstanceData->SourceWeight) || bAlwaysUpdateSource) && !bOnlyPlayingAdditives)
		{
			// Add the pose weight to blend in the source with
			if (InstanceData->bUsesBlendProfiles)
			{
				InstanceData->SkeletonDataForBlendProfiles.Add(nullptr);
				FBlendSampleData& SourceSampleBlendData = InstanceData->BlendDataForBlendProfiles.AddDefaulted_GetRef();
				SourceSampleBlendData.TotalWeight = InstanceData->SourceWeight;
				SourceSampleBlendData.PerBoneBlendData.SetNumZeroed(NumBones);
				for (int32 PerBoneIndex = 0; PerBoneIndex < NumBones; ++PerBoneIndex)
				{
					SourceSampleBlendData.PerBoneBlendData[PerBoneIndex] = 1.0f - TotalBoneWeights[PerBoneIndex];
				}
			}
			else
			{
				InstanceData->PoseWeights.Add(InstanceData->SourceWeight);
			}
		}

		// Queue up anim notify events
		if (FNotifyFilterTrait::AreNotifiesEnabledInScope(Context))
		{
			TArray<FAnimNotifyEventReference> NotifiesForSlot;
			if (MontageComponent->GetActiveNotifiesForSlot(CurrentSlotName, NotifiesForSlot))
			{
				if (FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
				{
					// Ensure we have a handler component on the module
					(void)ModuleInstance->GetOrAddComponent<FUAFNotifyDispatcherComponent>();
				}

				TSharedPtr<FNotifyDispatchEvent> NotifyDispatchEvent = MakeTraitEvent<FNotifyDispatchEvent>();
				NotifyDispatchEvent->Notifies = MoveTemp(NotifiesForSlot);
				NotifyDispatchEvent->Weight = 1.0f;
				Context.RaiseOutputTraitEvent(NotifyDispatchEvent);
			}
		}

		// Handle Group Synchronization data and registering
		if (MontageComponent->MontageEvalData.IsValidIndex(SyncCandidate_HeighestWeightIndex))
		{
			const FUAFMontageEvalData& SyncCandidateEvalData = MontageComponent->MontageEvalData[SyncCandidate_HeighestWeightIndex];
			const float MontageLength = SyncCandidateEvalData.Montage->CalculateSequenceLength();

			// On the first activation of synchronization the timeline state will be invalid, so we create it manually once 
			if (InstanceData->PostMontageUpdateState.GetDuration() == 0.0f)
			{
				const float PreUpdatePosition = FMath::Clamp(SyncCandidateEvalData.MontagePosition - InstanceData->DeltaTime, 0.0f, MontageLength);
				InstanceData->PreMontageUpdateState = FTimelineState(PreUpdatePosition, MontageLength, SyncCandidateEvalData.MontagePlayRate, false).WithDebugName(SyncCandidateEvalData.Montage->GetFName());
			}
			else
			{
				InstanceData->PreMontageUpdateState = InstanceData->PostMontageUpdateState;
			}

			InstanceData->PostMontageUpdateState = FTimelineState(SyncCandidateEvalData.MontagePosition, MontageLength, SyncCandidateEvalData.MontagePlayRate, false).WithDebugName(SyncCandidateEvalData.Montage->GetFName());
			InstanceData->SyncGroupParams.GroupName = SyncCandidateEvalData.Montage->SyncGroup;
			InstanceData->SyncCandidateIndex = SyncCandidate_HeighestWeightIndex;

			// Register with synchronization component
			const FSyncGroupParameters GroupParameters = InstanceData->SyncGroupParams;
			FSyncGroupGraphInstanceComponent& SyncComponent = Context.GetOrAddComponent<FSyncGroupGraphInstanceComponent>();
			SyncComponent.RegisterWithGroup(GroupParameters, Binding.GetTraitPtr(), TraitState);
		}
	}

	void FMontageTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const bool bAlwaysUpdateSource = SharedData->GetbAlwaysUpdateSource(Binding);
		const bool bOnlyPlayingAdditives = InstanceData->EvalTaskDataForSlot.Num() == 0 && InstanceData->AdditiveEvalTaskDataForSlot.Num() > 0;

		// To be consistent with ABP behavior if there is nothing connected to the Source Input we push a ref pose onto the stack to blend against
		// Another option would be to not blend if there isn't a source, however that would differ from ABP montage behavior
		if (!InstanceData->Source.IsValid() && (FAnimWeight::IsRelevant(InstanceData->SourceWeight) || bAlwaysUpdateSource || bOnlyPlayingAdditives))
		{
			Context.AppendTask(FAnimNextPushReferenceKeyframeTask::MakeFromSkeleton());
		}

		if (InstanceData->EvalTaskDataForSlot.IsEmpty() && InstanceData->AdditiveEvalTaskDataForSlot.IsEmpty())
		{
			return;
		}

		// First we push all non-additive animation keyframes onto the stack
		const float DeltaTime = InstanceData->DeltaTime;
		for (int32 EvalTaskDataIndex = InstanceData->EvalTaskDataForSlot.Num() -1; EvalTaskDataIndex >= 0; --EvalTaskDataIndex)
		{
			const FMontageEvalTaskData& TaskData = InstanceData->EvalTaskDataForSlot[EvalTaskDataIndex];
			if (TaskData.WeakAnimationSequence.IsValid())
			{
				FAnimNextAnimSequenceKeyframeTask Task;
				Task.bInterpolate = true;
				Task.bLooping = false; 
				Task.bExtractTrajectory = true;	/*Output.AnimInstanceProxy->ShouldExtractRootMotion()*/
				Task.AnimSequence = TaskData.WeakAnimationSequence.Get();
				Task.SampleTime = TaskData.AnimPosition;
				Task.DeltaTimeRecord = FDeltaTimeRecord(DeltaTime);

				Context.AppendTask(Task);
			}
		}

		// Then we blend the non-additive poses and source pose (if applicable) together 
		if (InstanceData->bUsesBlendProfiles)
		{
			if (InstanceData->BlendDataForBlendProfiles.Num() > 0)
			{
				const FBlendSampleData& SampleBlendData = InstanceData->BlendDataForBlendProfiles[0];
				const USkeleton* BlendProfileSkeleton = InstanceData->SkeletonDataForBlendProfiles[0];

				Context.AppendTask(FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask::Make(BlendProfileSkeleton, SampleBlendData, SampleBlendData.GetClampedWeight()));

				for (int32 BlendDataIndex = 1; BlendDataIndex < InstanceData->BlendDataForBlendProfiles.Num(); ++BlendDataIndex)
				{
					const FBlendSampleData& SampleBlendDataA = InstanceData->BlendDataForBlendProfiles[BlendDataIndex];
					const FBlendSampleData& SampleBlendDataB = InstanceData->BlendDataForBlendProfiles[BlendDataIndex - 1];
					const USkeleton* BlendProfileSkeletonA = InstanceData->SkeletonDataForBlendProfiles[BlendDataIndex];

					Context.AppendTask(FAnimNextBlendAddKeyframePerBoneWithScaleTask::Make(BlendProfileSkeletonA, SampleBlendDataA, SampleBlendDataB, SampleBlendDataA.GetClampedWeight()));
				}

				// Ensure that rotations are normalized after blending 
				Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
			}
		}
		else
		{
			if (InstanceData->PoseWeights.Num() > 0)
			{
				Context.AppendTask(FAnimNextBlendOverwriteKeyframeWithScaleTask::Make(InstanceData->PoseWeights[0]));
				for (int32 PoseWeightIndex = 1; PoseWeightIndex < InstanceData->PoseWeights.Num(); ++PoseWeightIndex)
				{
					Context.AppendTask(FAnimNextBlendAddKeyframeWithScaleTask::Make(InstanceData->PoseWeights[PoseWeightIndex]));
				}

				// Ensure that rotations are normalized after blending 
				Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
			}
		}

		// Finally, we add the additive poses and blend them in
		if (InstanceData->AdditiveEvalTaskDataForSlot.Num() > 0)
		{
			const bool bValidAdditiveData = InstanceData->bUsesBlendProfiles ?
				InstanceData->AdditiveBlendDataForBlendProfiles.Num() == InstanceData->AdditiveEvalTaskDataForSlot.Num() :
				InstanceData->AdditivePoseWeights.Num() == InstanceData->AdditiveEvalTaskDataForSlot.Num();

			if (ensureMsgf(bValidAdditiveData, TEXT("Additive montage pose data and blend data does not match! Will skip additives.")))
			{
				for (int32 AdditivePoseIndex = 0; AdditivePoseIndex < InstanceData->AdditiveEvalTaskDataForSlot.Num(); ++AdditivePoseIndex)
				{
					const FMontageEvalTaskData& TaskData = InstanceData->AdditiveEvalTaskDataForSlot[AdditivePoseIndex];
					if (TaskData.WeakAnimationSequence.IsValid())
					{
						FAnimNextAnimSequenceKeyframeTask Task;
						Task.bInterpolate = true;
						Task.bLooping = false;
						Task.bExtractTrajectory = true;	/*Output.AnimInstanceProxy->ShouldExtractRootMotion()*/
						Task.AnimSequence = TaskData.WeakAnimationSequence.Get();
						Task.SampleTime = TaskData.AnimPosition;
						Task.DeltaTimeRecord = FDeltaTimeRecord(DeltaTime);

						Context.AppendTask(Task);

						if (InstanceData->bUsesBlendProfiles)
						{
							const FBlendSampleData& AdditiveSampleBlendData = InstanceData->AdditiveBlendDataForBlendProfiles[AdditivePoseIndex];
							Context.AppendTask(FUAFApplyAdditiveKeyframePerBoneTask::Make(AdditiveSampleBlendData.GetClampedWeight(), AdditiveSampleBlendData, true));
						}
						else
						{
							Context.AppendTask(FAnimNextApplyAdditiveKeyframeTask::Make(InstanceData->AdditivePoseWeights[AdditivePoseIndex]));
						}

					}
				}
			}
		}
	}

	uint32 FMontageTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 1;
	}

	void FMontageTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the child, even if the handle is empty
		Children.Add(InstanceData->Source);
	}

	void FMontageTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		const bool bAlwaysUpdateSource = SharedData->GetbAlwaysUpdateSource(Binding);
		const bool bOnlyPlayingAdditives = InstanceData->EvalTaskDataForSlot.Num() == 0 && InstanceData->AdditiveEvalTaskDataForSlot.Num() > 0;
		if (InstanceData->Source.IsValid() && (FAnimWeight::IsRelevant(InstanceData->SourceWeight) || bAlwaysUpdateSource || bOnlyPlayingAdditives))
		{
			const bool bIsBlendingOut = InstanceData->LastUpdateSourceWeight > InstanceData->SourceWeight;
			FTraitUpdateState TraitStateSource = TraitState
				.WithWeight(InstanceData->SourceWeight)
				.AsBlendingOut(bIsBlendingOut);

			TraversalQueue.Push(InstanceData->Source, TraitStateSource);
		}
	}

	void FMontageTrait::GetSyncMarkers(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineSyncMarkerArray& OutSyncMarkers) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->SyncCandidateIndex == INDEX_NONE || InstanceData->MontageComponent == nullptr)
		{
			return;
		}

		if (InstanceData->MontageComponent->MontageEvalData.IsValidIndex(InstanceData->SyncCandidateIndex))
		{
			const FUAFMontageEvalData& SyncCandidateEvalData = InstanceData->MontageComponent->MontageEvalData[InstanceData->SyncCandidateIndex];
			if (ensureMsgf(SyncCandidateEvalData.Montage, TEXT("Get Montage SyncMarkers: SyncCandidateIndex referred to eval data with an invalid montage!")))
			{
				const int32 NumSyncMarkers = SyncCandidateEvalData.Montage->MarkerData.AuthoredSyncMarkers.Num();
				if (NumSyncMarkers > 0)
				{
					OutSyncMarkers.Reserve(NumSyncMarkers);
					for (const FAnimSyncMarker& SyncMarker : SyncCandidateEvalData.Montage->MarkerData.AuthoredSyncMarkers)
					{
						OutSyncMarkers.Add(FTimelineSyncMarker(SyncMarker.MarkerName, SyncMarker.Time));
					}
				}
			}
		}
	}

	bool FMontageTrait::GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		OutTimelineState = InstanceData->bReturnPreUpdateState ? InstanceData->PreMontageUpdateState : InstanceData->PostMontageUpdateState;
		return true;
	}

	void FMontageTrait::AdvanceBy(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		// This function gets called by the SyncGroupComponent and would usually advance the timeline of this trait
		// Because montages get currently advanced outside of UAF we mimic this behaviour by swapping which state we return in GetState()
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		InstanceData->bReturnPreUpdateState = false;
	}

	FSyncGroupParameters FMontageTrait::GetGroupParameters(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->SyncGroupParams;
	}

	void FMontageTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		Collector.AddReferencedObjects(InstanceData->SkeletonDataForBlendProfiles);
	}
}

