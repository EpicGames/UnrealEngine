// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/UAFMontageComponent.h"

#include "UAFLogging.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Component/AnimNextComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFNotifyDispatcherComponent.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFMontageComponent)

void FUAFUpdateMontageTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (!WeakAnimInstance.IsValid() || !MontageComponent)
	{
		return;
	}

	USkeletalMeshComponent* SkelMeshComponent = WeakAnimInstance->GetSkelMeshComponent();
	if (!ensureMsgf(SkelMeshComponent != nullptr, TEXT("UAF Tick Montages: The SkelMeshComponent was unexpectedly null. Montages won't get ticked.")))
	{
		return;
	}

	// If the skelmesh component hasn't been ticked already (which can happen in CMC's PerformMovement if the animation has root motion) we make sure to tick it now to update montages 
	SkelMeshComponent->TickMontagesOnly(DeltaTime, true);

	FAnimNextModuleInstance* ModuleInstance = MontageComponent->GetModuleInstancePtr();
	if (ensureMsgf(ModuleInstance != nullptr, TEXT("UAF TickMontages: Module Instance was not valid during tick, cannot send notify trait events!")))
	{
		// Queue up montage notifies, these will be fired no matter the slot weight. 
		if (WeakAnimInstance->NotifyQueue.AnimNotifies.Num() > 0)
		{
			TSharedPtr<UE::UAF::FNotifyDispatchEvent> NotifyDispatchEvent = MakeTraitEvent<UE::UAF::FNotifyDispatchEvent>();
			NotifyDispatchEvent->Notifies = WeakAnimInstance->NotifyQueue.AnimNotifies;
			NotifyDispatchEvent->Weight = 1.0f;
			ModuleInstance->QueueOutputTraitEvent(NotifyDispatchEvent);
		}
	}
	
	// Cache montage animation notifies, these will be fired only if the animation has weight. 
	// Each slot will handle their notifies within the trait. 
	// If you have more than one slot with the same name, these notifies might fire multiple times and may require manual filtering.
	MontageComponent->ActiveSlotNotifies.Reset();
	MontageComponent->ActiveSlotNotifies = WeakAnimInstance->NotifyQueue.UnfilteredMontageAnimNotifies;

	// Copy any active inertialization requests from the anim instance and then clear them manually since UpdateMontageEvaluationData won't be called
	MontageComponent->ActiveInertializationRequests = WeakAnimInstance->GetActiveInertializationRequests();
	WeakAnimInstance->ClearActiveInertializationRequests();
	
	MontageComponent->MontageEvalData.Reset();
	for (const FAnimMontageInstance* MontageInstance : WeakAnimInstance->MontageInstances)
	{
		// although montage can advance with 0.f weight, it is fine to filter by weight here
		// because we don't want to evaluate them if 0 weight
		if (MontageInstance->Montage && MontageInstance->GetWeight() > ZERO_ANIMWEIGHT_THRESH)
		{
			MontageComponent->MontageEvalData.Add(
				FUAFMontageEvalData
				(
					MontageInstance->Montage,
					MontageInstance->GetPosition(),
					MontageInstance->GetPlayRate(),
					MontageInstance->GetBlendStartAlpha(),
					MontageInstance->GetBlend(),
					MontageInstance->GetActiveBlendProfile()
				));
		}
	}
}

FString FUAFUpdateMontageTickFunction::DiagnosticMessage()
{
	return GetFullNameSafe(Target.Get()) + TEXT("[UAFUpdateMontageTickFunction]");
}

FName FUAFUpdateMontageTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("UAFUpdateMontageTickFunction/%s"), *GetFullNameSafe(Target.Get())));
	}

	static const FName UAFUpdateMontageTickFunctionFName(TEXT("UAFUpdateMontageTickFunction"));
	return UAFUpdateMontageTickFunctionFName;
}

FUAFMontageComponent::~FUAFMontageComponent()
{
	if (bHasTickDependencyRegistered && IsInGameThread())
	{
		if (FAnimNextModuleInstance* ModuleInstance = GetModuleInstancePtr())
		{
			if (UUAFComponent* UAFComponent = Cast<UUAFComponent>(ModuleInstance->GetObject()))
			{
				// Unregister movement component prerequisite
				UCharacterMovementComponent* CharacterMovementComponent = UAFComponent->GetOwner() ? UAFComponent->GetOwner()->FindComponentByClass<UCharacterMovementComponent>() : nullptr;
				if(CharacterMovementComponent)
				{
					MontageTickFunction.RemovePrerequisite(CharacterMovementComponent, CharacterMovementComponent->PrimaryComponentTick);
				}
				
				// Unregister module prerequisite
				for (UE::UAF::FModuleEventTickFunction& TickFunction : ModuleInstance->GetTickFunctions())
				{
					if (TickFunction.Event.GetEventName() == TickDependencyEventName)
					{
						TickFunction.RemovePrerequisite(UAFComponent, MontageTickFunction);
						bHasTickDependencyRegistered = false;
						break;
					}
				}
			}
		}
	}
}

bool FUAFMontageComponent::IsSlotActive(FName SlotNodeName) const
{
	if (SlotNodeName != NAME_None)
	{
		for (const FUAFMontageEvalData& EvalData : MontageEvalData)
		{
			if (EvalData.IsActive() && EvalData.GetMontage()->IsValidSlot(SlotNodeName))
			{
				return true;
			}
		}
	}

	return false;
}

float FUAFMontageComponent::GetSlotLocalWeight(FName SlotNodeName) const
{
	if (SlotNodeName != NAME_None)
	{
		float MontageLocalWeight = 0.0f;
		for (const FUAFMontageEvalData& EvalData : MontageEvalData)
		{
			const UAnimMontage* const Montage = EvalData.GetMontage();
			if (Montage && Montage->IsValidSlot(SlotNodeName))
			{
				const float MontageWeight = EvalData.GetBlendInfo().GetBlendedValue();
				MontageLocalWeight += MontageWeight;
			}
		}
		return FMath::Clamp(MontageLocalWeight, 0.0f, 1.0f);
	}
	return 0.0f;
}

TObjectPtr<const UAnimMontage> FUAFMontageComponent::GetActiveMontageForSlot(FName SlotNodeName) const
{
	if (SlotNodeName != NAME_None)
	{
		for (const FUAFMontageEvalData& EvalData : MontageEvalData)
		{
			if (EvalData.IsActive() && EvalData.GetMontage()->IsValidSlot(SlotNodeName))
			{
				return EvalData.GetMontage();
			}
		}
	}
	return nullptr;
}

bool FUAFMontageComponent::HasActiveInertializationRequests() const
{
	return !ActiveInertializationRequests.IsEmpty();
}

const FInertializationRequest* FUAFMontageComponent::GetInertializationRequestForGroupName(FName SlotGroupName) const
{
	return ActiveInertializationRequests.Find(SlotGroupName);
}

bool FUAFMontageComponent::GetActiveNotifiesForSlot(FName SlotName, TArray<FAnimNotifyEventReference>& OutNotifies) const
{
	const FAnimNotifyArray* FoundEntry = ActiveSlotNotifies.Find(SlotName);
	if (FoundEntry && FoundEntry->Notifies.Num() > 0)
	{
		OutNotifies = FoundEntry->Notifies;
		return true;
	}

	return false;
}

void FUAFMontageComponent::OnBindToInstance()
{
	FAnimNextModuleInstance* ModuleInstance = GetModuleInstancePtr();
	if (!ModuleInstance)
	{
		UE_LOGF(LogAnimation, Warning, "FUAFMontageComponent::OnBindToInstance: Could not bind to instance - Module Instance is not valid.");
		return;
	}

	UUAFComponent* UAFComponent = Cast<UUAFComponent>(ModuleInstance->GetObject());
	if (!UAFComponent)
	{
		UE_LOGF(LogAnimation, Warning, "FUAFMontageComponent::OnBindToInstance: Could not bind to instance - UAF Component is not valid.");
		return;
	}

	TickDependencyEventName = TickMontagesBeforeEventName.IsNone() ? DefaultTickDependencyEventName : TickMontagesBeforeEventName;

	// Depending on init order the SystemReference in the UAFComponnet might not be valid yet, 
	// so we have to manually find the correct module event tick function and enqueue our prerequisite.
	UE::UAF::FModuleEventTickFunction* ModuleEventTickFunction = nullptr;
	for (UE::UAF::FModuleEventTickFunction& TickFunction : ModuleInstance->GetTickFunctions())
	{
		if (TickFunction.Event.GetEventName() == TickDependencyEventName)
		{
			ModuleEventTickFunction = &TickFunction;
			break;
		}
	}

	if (ModuleEventTickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "UAFMontageComponent: No valid module event tick function found for {EventName}", TickDependencyEventName);
		return;
	}

	FAnimNextModuleInstance::RunTaskOnGameThread(
		[WeakUAFComponent = TWeakObjectPtr<UUAFComponent>(UAFComponent), ModuleEventTickFunction, MontageComponent = this]()
		{
			UUAFComponent* UAFComponent = WeakUAFComponent.Get();
			if (!UAFComponent)
			{
				UE_LOGF(LogAnimation, Warning, "FUAFMontageComponent: Could not setup montage tick function - UAF Component is not valid.");
				return;
			}

			AActor* UAFActor = UAFComponent->GetOwner();
			if (!UAFActor)
			{
				UE_LOGF(LogAnimation, Warning, "FUAFMontageComponent: Could not setup montage tick function - Owner of UAF Component [%ls] is not valid.", *UAFComponent->GetName());
				return;
			}

			USkeletalMeshComponent* SkelMeshComponent = UAFActor->FindComponentByClass<USkeletalMeshComponent>();
			if (!SkelMeshComponent || SkelMeshComponent->bEnableAnimation)
			{
				UE_LOGF(LogAnimation, Warning, "FUAFMontageComponent: Could not setup montage tick function - [%ls] does not have a valid SkeletalMeshComponent or Animations are not disabled.", *UAFActor->GetName());
				return;
			}

			// Create a new anim instance if the skelmesh component doesn't have one already (which it should not if UAF is used) 
			UAnimInstance* UsedAnimInstance = SkelMeshComponent->GetAnimInstance();
			if (UsedAnimInstance == nullptr)
			{

				UClass* MontageInstanceClass = MontageComponent->LegacyMontageAnimInstanceClass;
				if (MontageInstanceClass == nullptr)
				{
					MontageInstanceClass= UAnimInstance::StaticClass();
				}

				// Create a new AnimInstance that we can use to update our montages. 
				SkelMeshComponent->AnimClass = MontageInstanceClass;
				SkelMeshComponent->ClearAnimScriptInstance();
				SkelMeshComponent->AnimScriptInstance = NewObject<UAnimInstance>(SkelMeshComponent, MontageInstanceClass);

				UsedAnimInstance = SkelMeshComponent->GetAnimInstance();
			}

			// Do a lightweight init of the AnimInstance for montages only
			UsedAnimInstance->InitializeMontageOnly();

			// Setup and register tick function
			MontageComponent->MontageTickFunction.bRunOnAnyThread = false;
			MontageComponent->MontageTickFunction.bCanEverTick = true;
			MontageComponent->MontageTickFunction.bStartWithTickEnabled = true;
			MontageComponent->MontageTickFunction.MontageComponent = MontageComponent;
			MontageComponent->MontageTickFunction.Target = UAFComponent;
			MontageComponent->MontageTickFunction.WeakAnimInstance = UsedAnimInstance;

			if (UWorld* World = UAFComponent->GetWorld())
			{
				ULevel* Level = World->PersistentLevel;
				MontageComponent->MontageTickFunction.RegisterTickFunction(Level);

				ModuleEventTickFunction->AddPrerequisite(UAFComponent, MontageComponent->MontageTickFunction);
				
				// If the character has a character movement component, we have to ensure we tick after that otherwise you can end up with ticking the montages twice or in the wrong order
				if (UCharacterMovementComponent* CharacterMovementComponent = UAFActor->FindComponentByClass<UCharacterMovementComponent>())
				{
					MontageComponent->MontageTickFunction.AddPrerequisite(CharacterMovementComponent, CharacterMovementComponent->PrimaryComponentTick);
				}
				MontageComponent->bHasTickDependencyRegistered = true;
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "FUAFMontageComponent: Could not setup montage tick function - [%ls] has no valid World.", *UAFComponent->GetName());
			}
		});
}

FRigUnit_IsSlotActive_Execute()
{
	Result = false;
	if (SlotNodeName == NAME_None)
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("[%s] is not a valid slot name."), *SlotNodeName.ToString());
		return;
	}
	
	FUAFAssetInstance& Instance = ExecuteContext.GetContextData<FUAFAssetContextData>().GetInstance();
	FAnimNextModuleInstance* RootInstance = Instance.GetRootInstance();
	const FUAFMontageComponent* MontageComponent = RootInstance ? RootInstance->TryGetComponent<FUAFMontageComponent>() : nullptr;
	if (MontageComponent)
	{
		Result = MontageComponent->IsSlotActive(SlotNodeName);
	}
}

FRigUnit_GetSlotLocalWeight_Execute()
{
	LocalWeight = 0.0f;
	if (SlotNodeName == NAME_None)
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("[%s] is not a valid slot name."), *SlotNodeName.ToString());
		return;
	}
	
	FUAFAssetInstance& Instance = ExecuteContext.GetContextData<FUAFAssetContextData>().GetInstance();
	FAnimNextModuleInstance* RootInstance = Instance.GetRootInstance();
	const FUAFMontageComponent* MontageComponent = RootInstance ? RootInstance->TryGetComponent<FUAFMontageComponent>() : nullptr;
	if (MontageComponent)
	{
		LocalWeight = MontageComponent->GetSlotLocalWeight(SlotNodeName);
	}
}

