// Copyright Epic Games, Inc. All Rights Reserved.

#include "Native/UAFSimplePrePhysicsGraphComponent.h"

#include "Graph/RigUnit_UAFRunAsset.h"
#include "Injection/InjectionSiteTrait.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Script/UAFScriptContextData.h"
#include "Module/ModuleTaskContext.h"
#include "Variables/RigUnit_CopyModuleProxyVariables.h"
#include "Graph/RigUnit_UAFWriteSystemOutput.h"

TConstArrayView<UE::UAF::FScriptEventInfo> FUAFSimplePrePhysicsGraphComponent::GetScriptEvents()
{
	static const UE::UAF::FScriptEventInfo Events[] = 
	{
		{ .Event = UE::UAF::FScriptEvent(&FUAFSimplePrePhysicsGraphComponent::PrePhysics, FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName) }
	};

	return Events;
}

void FUAFSimplePrePhysicsGraphComponent::PrePhysics(TStructView<FUAFScriptComponent> InScriptComponent, TConstStructView<FUAFScriptContextData> InContextData)
{
	using namespace UE::UAF;
	
	FUAFSimplePrePhysicsGraphComponent& This = InScriptComponent.Get<FUAFSimplePrePhysicsGraphComponent>();
	
	const FAnimNextModuleContextData& ModuleContextData = InContextData.Get<FAnimNextModuleContextData>();
	FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();

	FAnimNextExecuteContext Context;
	Context.SetDeltaTime(ModuleContextData.GetDeltaTime());
	Context.SetOwningObject(ModuleInstance.GetObject());

	FScopedExecuteContextData ScopedExecuteContextData(Context, InContextData);

	FRigUnit_CopyModuleProxyVariables::StaticExecute(Context);

	FRigUnit_UAFRunAsset::StaticExecute(
		Context,
		*This.Graph,
		FAnimNextVariableOverridesCollection(),
		NAME_None,
		This.Output,
		This.WorkData);

	FRigUnit_UAFWriteSystemOutput::StaticExecute(
		Context,
		This.Output);
}

void FUAFSimplePrePhysicsGraphComponent::CallEventByName(TConstStructView<FUAFScriptContextData> InContextData)
{
	if (InContextData.Get<FUAFScriptContextData>().GetEventName() == FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName)
	{
		PrePhysics(*this, InContextData);
	}
}

void FUAFSimplePrePhysicsGraphComponent::OnBindToInstance()
{
	using namespace UE::UAF;

	// This interacts with actors/components at present, so needs to be called from the GT
	check(IsInGameThread());

	FAnimNextModuleInstance& SystemInstance = GetAssetInstance().As<FAnimNextModuleInstance>();
	SystemInstance.AccessVariablesStruct<FInjectionSiteTraitData>([this](FInjectionSiteTraitData& InjectionSiteTraitData)
	{
		Graph = &InjectionSiteTraitData.Graph;
	});

	check(Graph);
}