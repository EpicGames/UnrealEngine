// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayAnim/PlayAnimCallbackProxy.h"
#include "Animation/AnimSequence.h"
#include "Component/AnimNextComponent.h"
#include "GraphInterfaces/AnimNextNativeDataInterface_AnimSequencePlayer.h"
#include "Injection/InjectionUtils.h"
#include "Variables/AnimNextVariableReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayAnimCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UPlayAnimCallbackProxy

UPlayAnimCallbackProxy::UPlayAnimCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPlayAnimCallbackProxy* UPlayAnimCallbackProxy::CreateProxyObjectForPlayAnim(
	UUAFComponent* AnimNextComponent,
	FName SiteName,
	UAnimSequence* AnimSequence,
	float PlayRate,
	float StartPosition,
	UE::UAF::FInjectionBlendSettings BlendInSettings,
	UE::UAF::FInjectionBlendSettings BlendOutSettings)
{
	UPlayAnimCallbackProxy* Proxy = NewObject<UPlayAnimCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->Play(AnimNextComponent, SiteName, AnimSequence, PlayRate, StartPosition, BlendInSettings, BlendOutSettings);
	return Proxy;
}

bool UPlayAnimCallbackProxy::Play(
	UUAFComponent* AnimNextComponent,
	FName SiteName,
	UAnimSequence* AnimSequence,
	float PlayRate,
	float StartPosition,
	const UE::UAF::FInjectionBlendSettings& BlendInSettings,
	const UE::UAF::FInjectionBlendSettings& BlendOutSettings)
{
	bool bPlayedSuccessfully = false;
	if (AnimNextComponent != nullptr)
	{
		UE::UAF::FInjectionLifetimeEvents LifetimeEvents;
		LifetimeEvents.OnCompleted.BindUObject(this, &UPlayAnimCallbackProxy::OnPlayAnimCompleted);
		LifetimeEvents.OnInterrupted.BindUObject(this, &UPlayAnimCallbackProxy::OnPlayAnimInterrupted);
		LifetimeEvents.OnBlendingOut.BindUObject(this, &UPlayAnimCallbackProxy::OnPlayAnimBlendingOut);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PlayingRequest = UE::UAF::FInjectionUtils::PlayAnim(
			AnimNextComponent,
			UE::UAF::FInjectionSite(FAnimNextVariableReference(SiteName)),
			AnimSequence,
			{ 
				.PlayRate = PlayRate,
				.StartPosition = StartPosition
			},
			BlendInSettings,
			BlendOutSettings,
			MoveTemp(LifetimeEvents));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bWasInterrupted = false;
	}

	if (!PlayingRequest.IsValid())
	{
		OnInterrupted.Broadcast();
		Reset();
	}

	return bPlayedSuccessfully;
}

void UPlayAnimCallbackProxy::OnPlayAnimCompleted(const UE::UAF::FInjectionRequest& Request)
{
	if (!bWasInterrupted)
	{
		const UE::UAF::EInjectionStatus Status = Request.GetStatus();
		check(!EnumHasAnyFlags(Status, UE::UAF::EInjectionStatus::Interrupted));

		if (EnumHasAnyFlags(Status, UE::UAF::EInjectionStatus::Expired))
		{
			OnInterrupted.Broadcast();
		}
		else
		{
			OnCompleted.Broadcast();
		}
	}

	Reset();
}

void UPlayAnimCallbackProxy::OnPlayAnimInterrupted(const UE::UAF::FInjectionRequest& Request)
{
	bWasInterrupted = true;

	OnInterrupted.Broadcast();
}

void UPlayAnimCallbackProxy::OnPlayAnimBlendingOut(const UE::UAF::FInjectionRequest& Request)
{
	if (!bWasInterrupted)
	{
		OnBlendOut.Broadcast();
	}
}

void UPlayAnimCallbackProxy::Reset()
{
	PlayingRequest = nullptr;
	bWasInterrupted = false;
}

void UPlayAnimCallbackProxy::BeginDestroy()
{
	Reset();

	Super::BeginDestroy();
}
