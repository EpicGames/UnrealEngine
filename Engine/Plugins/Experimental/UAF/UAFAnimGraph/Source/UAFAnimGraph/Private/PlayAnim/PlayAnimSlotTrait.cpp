// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayAnim/PlayAnimSlotTrait.h"

#include "Animation/AnimSequence.h"
#include "Factory/AnimGraphFactory.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/ITimeline.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModuleInstance.h"
#include "TraitInterfaces/IGraphFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayAnimSlotTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FPlayAnimSlotTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IGarbageCollection) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(ISmoothBlend) \
		GeneratorMacro(IInertializerBlend) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FPlayAnimSlotTrait::OnInjectEvent) \
		GeneratorMacro(FPlayAnimSlotTrait::OnUninjectEvent) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FPlayAnimSlotTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR

	void FPlayAnimSlotTrait::FPlayAnimSlotRequest::Initialize(FInjectionRequestPtr InRequest, const FInjectionBlendSettings& InBlendSettings, const UUAFAnimGraph* InAnimationGraph)
	{
		Request = InRequest;
		BlendSettings = InBlendSettings;
		AnimationGraph = InAnimationGraph;

		// If no input is provided, we'll use the source
		State = InAnimationGraph != nullptr ? EPlayAnimRequestState::Active : EPlayAnimRequestState::ActiveSource;
		bWasRelevant = false;
	}

	int32 FPlayAnimSlotTrait::FindFreeRequestIndexOrAdd(FInstanceData& InstanceData)
	{
		// Find an empty request we can use
		const int32 NumRequests = InstanceData.SlotRequests.Num();
		for (int32 RequestIndex = 0; RequestIndex < NumRequests; ++RequestIndex)
		{
			if (InstanceData.SlotRequests[RequestIndex].State == EPlayAnimRequestState::Inactive)
			{
				// This request is inactive, we can re-use it
				return RequestIndex;
			}
		}

		// All requests are in use, add a new one
		return InstanceData.SlotRequests.AddDefaulted();
	}

	ETraitStackPropagation FPlayAnimSlotTrait::OnInjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_InjectEvent& Event) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FAnimNextVariableReference InjectionSite(SharedData->GetSlotName(Binding));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const FAnimNextInjectionRequestArgs& RequestArgs = Event.Request->GetArgs();
		if (InjectionSite == RequestArgs.Site.DesiredSite)
		{
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			ensureMsgf(!InstanceData->PendingRequest.IsValid(), TEXT("PlayAnim slot %s already contained a pending request, it will be overwritten"), *InjectionSite.GetName().ToString());

			// Overwrite any request we might have, we'll pick it up on the next update
			InstanceData->PendingRequest.Reset();
			InstanceData->PendingRequest.Request = Event.Request;

			Event.MarkConsumed();
		}

		return ETraitStackPropagation::Continue;
	}

	ETraitStackPropagation FPlayAnimSlotTrait::OnUninjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_UninjectEvent& Event) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FAnimNextVariableReference InjectionSite(SharedData->GetSlotName(Binding));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const FAnimNextInjectionRequestArgs& RequestArgs = Event.Request->GetArgs();
		if (InjectionSite == RequestArgs.Site.DesiredSite)
		{
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			// Reset any pending request we might have, and cancel it
			InstanceData->PendingRequest.Reset();
			InstanceData->PendingRequest.bStop = true;

			Event.MarkConsumed();
		}

		return ETraitStackPropagation::Continue;
	}

	uint32 FPlayAnimSlotTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->SlotRequests.Num();
	}

	void FPlayAnimSlotTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (const FPlayAnimSlotRequest& SlotRequest : InstanceData->SlotRequests)
		{
			// Even if the request is inactive, we queue an empty handle
			Children.Add(SlotRequest.GetChildPtr());
		}
	}

	void FPlayAnimSlotTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		const bool bHasActiveSubGraph = InstanceData->CurrentlyActiveRequestIndex != INDEX_NONE;
		if (bHasActiveSubGraph)
		{
			InstanceData->SlotRequests[InstanceData->CurrentlyActiveRequestIndex].bWasRelevant = true;
		}

		bool bJustTransitioned = false;
		if (InstanceData->PendingRequest.IsValid() || !bHasActiveSubGraph)
		{
			const FInjectionRequestPtr Request = InstanceData->PendingRequest.Request;

			// Clear it out now in case we early out below
			InstanceData->PendingRequest.Reset();

			FInjectionBlendSettings BlendSettings;
			const UUAFAnimGraph* AnimationGraph = nullptr;

			if (Request)
			{
				// This is a new pending request, lookup the sub-graph to use with any factory trait we have and the desired animation object
				FAnimNextInjectionRequestArgs& RequestArgs = Request->GetMutableArgs();
				if (RequestArgs.Object)
				{
					AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, RequestArgs.Object, RequestArgs.FactoryParams);
					if (AnimationGraph != nullptr)
					{
						// Check for re-entrancy and early-out if we are linking back to the current instance or one of its parents
						const FName EntryPoint = AnimationGraph->DefaultEntryPoint;
						const FAnimNextGraphInstance* OwnerGraphInstance = &Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
						while (OwnerGraphInstance != nullptr)
						{
							if (OwnerGraphInstance->UsesAnimationGraph(AnimationGraph) && OwnerGraphInstance->UsesEntryPoint(EntryPoint))
							{
								UE_LOGFMT(LogAnimation, Warning, "Ignoring PlayAnim request for {0}, re-entrancy detected", AnimationGraph->GetFName());
								return;
							}

							OwnerGraphInstance = OwnerGraphInstance->GetParentGraphInstance();
						}
					}

					BlendSettings = RequestArgs.BlendInSettings;
				}
			}

			if (bHasActiveSubGraph)
			{
				// Queue our status update
				const FPlayAnimSlotRequest& OldSlotRequest = InstanceData->SlotRequests[InstanceData->CurrentlyActiveRequestIndex];
				if (OldSlotRequest.State == EPlayAnimRequestState::Active)
				{
					auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
					StatusUpdateEvent->Request = OldSlotRequest.Request;
					StatusUpdateEvent->Status = EInjectionStatus::Playing | EInjectionStatus::Interrupted;

					Context.RaiseOutputTraitEvent(StatusUpdateEvent);
				}
			}

			// Find an empty request we can use
			const int32 FreeRequestIndex = FindFreeRequestIndexOrAdd(*InstanceData);

			FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[FreeRequestIndex];
			SlotRequest.Initialize(Request, BlendSettings, AnimationGraph);

			const int32 OldChildIndex = InstanceData->CurrentlyActiveRequestIndex;
			const int32 NewChildIndex = FreeRequestIndex;

			InstanceData->CurrentlyActiveRequestIndex = FreeRequestIndex;

			DiscreteBlendTrait.OnBlendTransition(Context, OldChildIndex, NewChildIndex);

			bJustTransitioned = true;
		}

		float CurrentRequestTimeLeft = 0.0f;

		// Broadcast our timeline progress
		const int32 NumSlotRequests = InstanceData->SlotRequests.Num();
		for (int32 RequestIndex = 0; RequestIndex < NumSlotRequests; ++RequestIndex)
		{
			const FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[RequestIndex];
			if (SlotRequest.State != EPlayAnimRequestState::Active)
			{
				continue;	// We don't care about this slot request
			}

			FTraitStackBinding ChildStack;
			ensure(Context.GetStack(SlotRequest.GetChildPtr(), ChildStack));

			TTraitBinding<ITimeline> ChildTimelineTrait;
			ensure(ChildStack.GetInterface(ChildTimelineTrait));

			FTimelineState ChildState;
			ensure(ChildTimelineTrait.GetState(Context, ChildState));

			if (InstanceData->CurrentlyActiveRequestIndex == RequestIndex)
			{
				const float ChildCurrentPosition = ChildState.GetPosition();
				const float ChildNextPosition = ChildCurrentPosition + (TraitState.GetDeltaTime() * ChildState.GetPlayRate());

				// Compute how much time is left before the timeline ends (can be negative if we overshoot)
				const float TimeToEnd =
					ChildNextPosition >= ChildCurrentPosition ?	// Is moving forward?
					ChildState.GetDuration() - ChildNextPosition :
					ChildNextPosition;

				CurrentRequestTimeLeft = TimeToEnd;
			}

			// Only raise a timeline update event if we care about it
			if (SlotRequest.Request->GetArgs().bTrackTimelineProgress)
			{
				auto TimelineUpdateEvent = MakeTraitEvent<FInjection_TimelineUpdateEvent>();
				TimelineUpdateEvent->Request = SlotRequest.Request;

				// We don't have too many options here:
				//    - We can have one frame delay (as we do now)
				//    - We could use the speculative estimate (from above) as our new state, but this may not be fully accurate (e.g. ignores sync groups)
				//    - We could query the timeline during PostUpdate, but this would ignore sync groups
				//    - We could add a new graph instance component and hook PostUpdate, but then we have an ordering issue with the sync group component
				//    - To be fully accurate, the timeline would need to broadcast when it changes, and so we would need to register callbacks on it and manage them
				TimelineUpdateEvent->TimelineState = ChildState;

				Context.RaiseOutputTraitEvent(TimelineUpdateEvent);
			}
		}

		// Check if we are blending out
		if (!bJustTransitioned && InstanceData->CurrentlyActiveRequestIndex != INDEX_NONE)
		{
			const FPlayAnimSlotRequest& ActiveSlotRequest = InstanceData->SlotRequests[InstanceData->CurrentlyActiveRequestIndex];

			if (ActiveSlotRequest.State == EPlayAnimRequestState::Active)
			{
				const FAnimNextInjectionRequestArgs& RequestArgs = ActiveSlotRequest.Request->GetArgs();

				const float BlendOutTime = RequestArgs.BlendOutSettings.Blend.BlendTime;
				if (CurrentRequestTimeLeft <= BlendOutTime)
				{
					// We are ready to start blending out
					{
						auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
						StatusUpdateEvent->Request = ActiveSlotRequest.Request;
						StatusUpdateEvent->Status = EInjectionStatus::BlendingOut;

						Context.RaiseOutputTraitEvent(StatusUpdateEvent);
					}

					// Find an empty request we can use
					const int32 FreeRequestIndex = FindFreeRequestIndexOrAdd(*InstanceData);

					FPlayAnimSlotRequest& FreeSlotRequest = InstanceData->SlotRequests[FreeRequestIndex];
					FreeSlotRequest.Initialize(FInjectionRequestPtr(), RequestArgs.BlendOutSettings, nullptr);

					const int32 OldChildIndex = InstanceData->CurrentlyActiveRequestIndex;
					const int32 NewChildIndex = FreeRequestIndex;

					InstanceData->CurrentlyActiveRequestIndex = FreeRequestIndex;

					DiscreteBlendTrait.OnBlendTransition(Context, OldChildIndex, NewChildIndex);
				}
			}
		}
	}

	void FPlayAnimSlotTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const int32 NumRequests = InstanceData->SlotRequests.Num();
		check(NumRequests != 0);	// Should never happen since the source is always present

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		for (int32 RequestIndex = 0; RequestIndex < NumRequests; ++RequestIndex)
		{
			const FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[RequestIndex];
			const float BlendWeight = DiscreteBlendTrait.GetBlendWeight(Context, RequestIndex);
			const bool bGraphHasNeverUpdated = SlotRequest.GraphInstance.IsValid() && !SlotRequest.GraphInstance->HasUpdated();

			FTraitUpdateState RequestSlotTraitState = TraitState
				.WithWeight(BlendWeight)
				.AsBlendingOut(RequestIndex != InstanceData->CurrentlyActiveRequestIndex)
				.AsNewlyRelevant(!SlotRequest.bWasRelevant || bGraphHasNeverUpdated);

			if (SlotRequest.GraphInstance.IsValid())
			{
				SlotRequest.GraphInstance->MarkAsUpdated();
			}

			TraversalQueue.Push(InstanceData->SlotRequests[RequestIndex].GetChildPtr(), RequestSlotTraitState);
		}
	}

	float FPlayAnimSlotTrait::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (ChildIndex == InstanceData->CurrentlyActiveRequestIndex)
		{
			return 1.0f;	// Active child has full weight
		}
		else if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			return 0.0f;	// Other children have no weight
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}

	int32 FPlayAnimSlotTrait::GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		return InstanceData->CurrentlyActiveRequestIndex;
	}

	void FPlayAnimSlotTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// We initiate immediately when we transition
		DiscreteBlendTrait.OnBlendInitiated(Context, NewChildIndex);

		// We terminate immediately when we transition
		DiscreteBlendTrait.OnBlendTerminated(Context, OldChildIndex);
	}

	void FPlayAnimSlotTrait::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			// Allocate our new request instance
			FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[ChildIndex];

			if (SlotRequest.State == EPlayAnimRequestState::Active)
			{
				const FName EntryPoint = SlotRequest.AnimationGraph->DefaultEntryPoint;
				FAnimNextGraphInstance& Owner = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
				SlotRequest.GraphInstance = SlotRequest.AnimationGraph->AllocateInstance(
					{
						.SystemReference = Owner.GetModuleInstance()->GetReference(),
						.ParentGraphInstance = Owner.AsShared(),
						.EntryPoint = EntryPoint
					});

				// TODO: Validate that our child implements the ITimeline interface

				{
					// Queue our status update
					auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
					StatusUpdateEvent->Request = SlotRequest.Request;
					StatusUpdateEvent->Status = EInjectionStatus::Playing;

					Context.RaiseOutputTraitEvent(StatusUpdateEvent);
				}
			}
			else if (SlotRequest.State == EPlayAnimRequestState::ActiveSource)
			{
				const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

				SlotRequest.ChildPtr = Context.AllocateNodeInstance(Binding, SharedData->Source);
			}
		}
	}

	void FPlayAnimSlotTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			// Deallocate our request instance
			FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[ChildIndex];

			if (SlotRequest.State == EPlayAnimRequestState::Active)
			{
				SlotRequest.GraphInstance.Reset();

				{
					// Queue our status update
					auto StatusUpdateEvent = MakeTraitEvent<FInjection_StatusUpdateEvent>();
					StatusUpdateEvent->Request = SlotRequest.Request;
					StatusUpdateEvent->Status = EInjectionStatus::Completed;

					Context.RaiseOutputTraitEvent(StatusUpdateEvent);
				}
			}

			SlotRequest.Request = nullptr;
			SlotRequest.ChildPtr.Reset();
			SlotRequest.State = EPlayAnimRequestState::Inactive;
			SlotRequest.bWasRelevant = false;
		}
	}

	float FPlayAnimSlotTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			const FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[ChildIndex];
			return SlotRequest.BlendSettings.Blend.BlendTime;
		}
		else
		{
			// Unknown child
			return 0.0f;
		}
	}

	EAlphaBlendOption FPlayAnimSlotTrait::GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			const FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[ChildIndex];
			return SlotRequest.BlendSettings.Blend.BlendOption;
		}
		else
		{
			// Unknown child
			return EAlphaBlendOption::Linear;
		}
	}

	UCurveFloat* FPlayAnimSlotTrait::GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			const FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[ChildIndex];
			return SlotRequest.BlendSettings.Blend.CustomCurve;
		}
		else
		{
			// Unknown child
			return nullptr;
		}
	}

	float FPlayAnimSlotTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->SlotRequests.IsValidIndex(ChildIndex))
		{
			const FPlayAnimSlotRequest& SlotRequest = InstanceData->SlotRequests[ChildIndex];
			if (SlotRequest.BlendSettings.BlendMode == EAnimNextInjectionBlendMode::Inertialization)
			{
				return SlotRequest.BlendSettings.Blend.BlendTime;
			}
			else
			{
				// Not an inertializing blend
				return 0.0f;
			}
		}
		else
		{
			// Unknown child
			return 0.0f;
		}
	}

	void FPlayAnimSlotTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->PendingRequest.Request)
		{
			InstanceData->PendingRequest.Request->AddReferencedObjects(Collector);
		}

		for (FPlayAnimSlotRequest& SlotRequest : InstanceData->SlotRequests)
		{
			if (SlotRequest.Request)
			{
				SlotRequest.Request->AddReferencedObjects(Collector);
			}

			Collector.AddReferencedObject(SlotRequest.AnimationGraph);
			if(FAnimNextGraphInstance* ImplPtr = SlotRequest.GraphInstance.Get())
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), ImplPtr);
			}
		}
	}
}
