// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IInertializerBlend.h"
#include "TraitInterfaces/ISmoothBlend.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Injection/InjectionEvents.h"

#include "PlayAnimSlotTrait.generated.h"

USTRUCT(meta = (DisplayName = "PlayAnim Slot", Hidden))
struct FAnimNextPlayAnimSlotTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** Default input when no animation request has been made on this slot. */
	UPROPERTY()
	FAnimNextTraitHandle Source;

	/** The name of this slot that the PlayAnim API refers to. */
	UPROPERTY(EditAnywhere, Category = "Default")
	FName SlotName;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(SlotName) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextPlayAnimSlotTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	/**
	 * FPlayAnimSlotTrait
	 * 
	 * A trait that provides a slot onto which a PlayAnim request can be processed.
	 * It allows for this trait to act as a pass-through when not actively used and
	 * when a PlayAnim request is made to start playing a child instance, we blend to it.
	 */
	struct FPlayAnimSlotTrait : FBaseTrait, IUpdate, IUpdateTraversal, IHierarchy, IDiscreteBlend, ISmoothBlend, IInertializerBlend, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FPlayAnimSlotTrait, FBaseTrait)

		// Slot request state
		enum class EPlayAnimRequestState : uint8
		{
			// Slot request is inactive
			Inactive,

			// Slot request is active and using a sub-graph
			Active,

			// Slot request is active and using the source input
			ActiveSource,
		};

		struct FPlayAnimPendingRequest
		{
			// The PlayAnim request
			FInjectionRequestPtr Request;

			// Whether or not a Stop request was issued
			bool bStop = false;

			// Returns whether or not we have a pending request
			bool IsValid() const { return !!Request || bStop; }

			// Resets the pending request
			void Reset()
			{
				Request = nullptr;
				bStop = false;
			}
		};

		struct FPlayAnimSlotRequest
		{
			// The PlayAnim request
			FInjectionRequestPtr Request;

			// The blend settings to use
			FInjectionBlendSettings BlendSettings;

			// The module used by the graph instance, as selected by the chooser
			TObjectPtr<const UUAFAnimGraph> AnimationGraph;

			// The graph instance
			TSharedPtr<FAnimNextGraphInstance> GraphInstance;

			// Our child handle
			// If we use the source input, this is a strong handle to it, otherwise we are a weak handle to the graph instance's root
			FTraitPtr ChildPtr;

			// The current request state
			EPlayAnimRequestState State = EPlayAnimRequestState::Inactive;

			// Whether or not this slot state was previously relevant
			bool bWasRelevant = false;

			// Initializes a request to begin playing
			void Initialize(FInjectionRequestPtr InRequest, const FInjectionBlendSettings& InBlendSettings, const UUAFAnimGraph* InAnimationGraph);

			FWeakTraitPtr GetChildPtr() const
			{
				if (State == EPlayAnimRequestState::ActiveSource)
				{
					return ChildPtr;
				}
				else if (GraphInstance)
				{
					return GraphInstance->GraphInstancePtr;
				}
				return FWeakTraitPtr();
			}
		};

		using FSharedData = FAnimNextPlayAnimSlotTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			// List of PlayAnim slot requests
			TArray<FPlayAnimSlotRequest> SlotRequests;

			// PlayAnim pending request
			FPlayAnimPendingRequest PendingRequest;

			// The index of the currently active request
			// All other requests are blending out
			int32 CurrentlyActiveRequestIndex = INDEX_NONE;
		};

		// Internal impl
		static int32 FindFreeRequestIndexOrAdd(FInstanceData& InstanceData);

		// Event handling impl
		ETraitStackPropagation OnInjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_InjectEvent& Event) const;
		ETraitStackPropagation OnUninjectEvent(const FExecutionContext& Context, FTraitBinding& Binding, FInjection_UninjectEvent& Event) const;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IDiscreteBlend impl
		virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual int32 GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const override;
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;
		virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;

		// ISmoothBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;
		virtual EAlphaBlendOption GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;
		virtual UCurveFloat* GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;

		// IInertializerBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};
}
