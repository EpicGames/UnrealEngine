// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/ExecutionContext.h"

#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitTemplate.h"
#include "TraitCore/NodeDescription.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Components/SceneComponent.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "Logging/StructuredLog.h"

namespace UE::UAF
{
	namespace Private
	{
		// Represents an entry for a scoped interface
		struct FScopedInterfaceEntry
		{
			// The trait stack, copied from the source when we push a scoped interface
			FTraitStackBinding Stack;

			// The trait that implements the interface
			// We lazily cache the binding of the scoped interface
			FTraitBinding Trait;

			// The scoped interface
			FTraitInterfaceUID InterfaceUID;

			// The trait index on the stack that implements our scoped interface
			uint8 TraitIndex = 0;

			// Whether or not the scoped interface trait binding has been cached
			bool bIsTraitCached = false;

			union
			{
				// Next entry in the stack of free entries
				FScopedInterfaceEntry* NextFreeEntry = nullptr;

				// The previous entry on the scoped interface stack
				FScopedInterfaceEntry* PrevScopedInterfaceStackEntry;
			};

			FScopedInterfaceEntry(const FTraitBinding& InTrait, FTraitInterfaceUID InInterfaceUID)
				: Stack(*InTrait.GetStack())
				, InterfaceUID(InInterfaceUID)
				, TraitIndex(InTrait.GetTraitIndex())
			{
			}

			// Lazily construct the trait binding to our scoped interface
			bool LazilyCacheTrait()
			{
				if (bIsTraitCached)
				{
					return true;	// Already cached
				}

				if (!Stack.GetTrait(TraitIndex, Trait))
				{
					return false;
				}

				if (!Trait.AsInterfaceImpl(InterfaceUID, Trait))
				{
					return false;
				}

				bIsTraitCached = true;
				return true;
			}
		};
	}

	FExecutionContext::FExecutionContext()
		: MemStack(FMemStack::Get())
		, NodeTemplateRegistry(FNodeTemplateRegistry::Get())
		, TraitRegistry(FTraitRegistry::Get())
	{
	}

	FExecutionContext::FExecutionContext(FAnimNextGraphInstance& InGraphInstance)
		: FExecutionContext()
	{
		BindTo(InGraphInstance);
	}

	void FExecutionContext::BindTo(FAnimNextGraphInstance& InGraphInstance)
	{
		FAnimNextGraphInstance* InRootGraphInstance = InGraphInstance.GetRootGraphInstance();
		if (RootGraphInstance == InRootGraphInstance)
		{
			return;	// Already bound to this root graph instance, nothing to do
		}

		RootGraphInstance = InRootGraphInstance;
	}

	void FExecutionContext::BindTo(const FWeakTraitPtr& TraitPtr)
	{
		if (const FNodeInstance* NodeInstance = TraitPtr.GetNodeInstance())
		{
			BindTo(NodeInstance->GetOwner());
		}
	}

	bool FExecutionContext::IsBound() const
	{
		return RootGraphInstance != nullptr;
	}

	bool FExecutionContext::IsBoundTo(const FAnimNextGraphInstance& InGraphInstance) const
	{
		return RootGraphInstance == InGraphInstance.GetRootGraphInstance();
	}

	FTraitPtr FExecutionContext::AllocateNodeInstance(FAnimNextGraphInstance& GraphInstance, FAnimNextTraitHandle ChildTraitHandle) const
	{
		LLM_SCOPE_BYNAME(TEXT("UAF/AnimGraph"));

		if (!ChildTraitHandle.IsValid())
		{
			UE_LOGF(LogAnimation, Log, "FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The provided trait handle is not valid.");
			return FTraitPtr();	// Attempting to allocate a node using an invalid trait handle
		}

		if (!ensureMsgf(IsBound(), TEXT("FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The execution context is not bound to a valid graph instance.")))
		{
			return FTraitPtr();	// The execution context must be bound to a valid graph instance
		}

		if (!ensureMsgf(GraphInstance.GetAnimationGraph() != nullptr, TEXT("FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The provided graph instance has no valid animation graph.")))
		{
			return FTraitPtr();	// We need a valid graph instance to allocate into
		}

		const FNodeHandle ChildNodeHandle = ChildTraitHandle.GetNodeHandle();
		const uint32 ChildTraitIndex = ChildTraitHandle.GetTraitIndex();

		const FNodeDescription& NodeDesc = GetNodeDescription(GraphInstance, ChildNodeHandle);
		const FNodeTemplate* NodeTemplate = GetNodeTemplate(NodeDesc);

		if (!ensureMsgf(NodeTemplate != nullptr, TEXT("FExecutionContext::AllocateNodeInstance: Could not allocate node instance - Node template wasn't found, node descriptor might be corrupted")))
		{
			return FTraitPtr();	// Node template wasn't found, node descriptor is perhaps corrupted
		}
		else if (ChildTraitIndex >= NodeTemplate->GetNumTraits())
		{
			UE_LOGF(LogAnimation, Warning, "FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The requested trait index doesn't exist on that node descriptor.");
			return FTraitPtr();	// The requested trait index doesn't exist on that node descriptor
		}

		// We need to allocate a new node instance
		const uint32 InstanceSize = NodeDesc.GetNodeInstanceDataSize();
		uint8* NodeInstanceBuffer = reinterpret_cast<uint8*>(FMemory::Malloc(InstanceSize, 16));
		FNodeInstance* NodeInstance = new(NodeInstanceBuffer) FNodeInstance(GraphInstance, ChildNodeHandle);

		// Manually bind our stack since we have everything we need
		FTraitStackBinding StackBinding;
		StackBinding.Context = this;
		StackBinding.NodeInstance = NodeInstance;
		StackBinding.NodeDescription = &NodeDesc;
		StackBinding.NodeTemplate = NodeTemplate;

		// Start construction with the base trait
		// We construct the whole node which will include one or more sub-stacks (each with their own base trait)
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs;
		const FTraitTemplate* EndDesc = TraitDescs + NodeTemplate->GetNumTraits();

		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; ++TraitDesc)
		{
			const FTrait* Trait = GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			const uint32 TraitIndex = TraitDesc - TraitDescs;

			if (TraitDesc->GetMode() == ETraitMode::Base)
			{
				// A new base trait, update our stack binding
				StackBinding.BaseTraitIndex = TraitIndex;
				StackBinding.TopTraitIndex = TraitDesc->GetNumStackTraits() - 1;
			}
			
			const FTraitBinding Binding(&StackBinding, Trait, TraitIndex);
			
			if (TraitDesc->IsGCAware())
			{
				GraphInstance.RegisterWithGC(Binding.GetTraitPtr());
			}

			Trait->ConstructTraitInstance(*this, Binding);
		}

		return FTraitPtr(NodeInstance, ChildTraitIndex);
	}

	FTraitPtr FExecutionContext::AllocateNodeInstance(const FWeakTraitPtr& ParentBinding, FAnimNextTraitHandle ChildTraitHandle) const
	{
		LLM_SCOPE_BYNAME(TEXT("UAF/AnimGraph"));
		
		if (!ChildTraitHandle.IsValid())
		{
			UE_LOGF(LogAnimation, Log, "FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The provided trait handle is not valid.");
			return FTraitPtr();	// Attempting to allocate a node using an invalid trait handle
		}

		if (!ensureMsgf(IsBound(), TEXT("FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The execution context is not bound to a valid graph instance.")))
		{
			return FTraitPtr();	// The execution context must be bound to a valid graph instance
		}

		if (!ensureMsgf(ParentBinding.IsValid(), TEXT("FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The provided parent binding was not valid.")))
		{
			return FTraitPtr();	// We need a parent binding to know which graph instance to allocate into
		}

		FNodeInstance* ParentNodeInstance = ParentBinding.GetNodeInstance();
		FAnimNextGraphInstance& GraphInstance = ParentNodeInstance->GetOwner();

		const FNodeHandle ChildNodeHandle = ChildTraitHandle.GetNodeHandle();
		const uint32 ChildTraitIndex = ChildTraitHandle.GetTraitIndex();

		const FNodeDescription& NodeDesc = GetNodeDescription(GraphInstance, ChildNodeHandle);
		const FNodeTemplate* NodeTemplate = GetNodeTemplate(NodeDesc);

		if (!ensureMsgf(NodeTemplate != nullptr, TEXT("FExecutionContext::AllocateNodeInstance: Could not allocate node instance - Node template wasn't found, node descriptor might be corrupted")))
		{
			return FTraitPtr();	// Node template wasn't found, node descriptor is perhaps corrupted
		}
		else if (ChildTraitIndex >= NodeTemplate->GetNumTraits())
		{
			UE_LOGF(LogAnimation, Warning, "FExecutionContext::AllocateNodeInstance: Could not allocate node instance - The requested trait index doesn't exist on that node descriptor.");
			return FTraitPtr();	// The requested trait index doesn't exist on that node descriptor
		}

		// If the trait we wish to allocate lives in the parent node, return a weak handle to it
		// We use a weak handle to avoid issues when multiple base traits live within the same node
		// When this happens, a trait can end up pointing to another within the same node causing
		// the reference count to never reach zero when all other handles are released
		if (ParentNodeInstance->GetNodeHandle() == ChildNodeHandle)
		{
			return FTraitPtr(ParentNodeInstance, FTraitPtr::IS_WEAK_BIT, ChildTraitIndex);
		}

		// We need to allocate a new node instance
		const uint32 InstanceSize = NodeDesc.GetNodeInstanceDataSize();
		uint8* NodeInstanceBuffer = reinterpret_cast<uint8*>(FMemory::Malloc(InstanceSize, 16));
		FNodeInstance* NodeInstance = new(NodeInstanceBuffer) FNodeInstance(GraphInstance, ChildNodeHandle);

		// Manually bind our stack since we have everything we need
		FTraitStackBinding StackBinding;
		StackBinding.Context = this;
		StackBinding.NodeInstance = NodeInstance;
		StackBinding.NodeDescription = &NodeDesc;
		StackBinding.NodeTemplate = NodeTemplate;

		// Start construction with the base trait
		// We construct the whole node which will include one or more sub-stacks (each with their own base trait)
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs;
		const FTraitTemplate* EndDesc = TraitDescs + NodeTemplate->GetNumTraits();

		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; ++TraitDesc)
		{
			const FTrait* Trait = GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			const uint32 TraitIndex = TraitDesc - TraitDescs;

			if (TraitDesc->GetMode() == ETraitMode::Base)
			{
				// A new base trait, update our stack binding
				StackBinding.BaseTraitIndex = TraitIndex;
				StackBinding.TopTraitIndex = TraitDesc->GetNumStackTraits() - 1;
			}

			const FTraitBinding Binding(&StackBinding, Trait, TraitIndex);
			
			// Auto-register this trait with garbage collection
			if (TraitDesc->IsGCAware())
			{
				const FWeakTraitPtr TraitPtr = Binding.GetTraitPtr();
				FAnimNextGraphInstance& Instance = TraitPtr.GetNodeInstance()->GetOwner();
				Instance.RegisterWithGC(TraitPtr);
			}

			Trait->ConstructTraitInstance(*this, Binding);
		}

		return FTraitPtr(NodeInstance, ChildTraitIndex);
	}

	void FExecutionContext::ReleaseNodeInstance(FTraitPtr& NodePtr) const
	{
		if (!NodePtr.IsValid())
		{
			return;
		}

		FNodeInstance* NodeInstance = NodePtr.GetNodeInstance();

		if (!ensureMsgf(IsBoundTo(NodeInstance->GetOwner()), TEXT("FExecutionContext::ReleaseNodeInstance: The execution context isn't bound to the right graph instance")))
		{
			return;	// The execution context isn't bound to the right graph instance
		}

		// Reset the handle here to simplify the multiple return statements below
		NodePtr.PackedPointerAndFlags = 0;
		NodePtr.TraitIndex = 0;

		if (NodeInstance->RemoveReference())
		{
			return;	// Node instance still has references, we can't release it yet
		}

		const FNodeDescription& NodeDesc = GetNodeDescription(*NodeInstance);
		const FNodeTemplate* NodeTemplate = GetNodeTemplate(NodeDesc);
		if (!ensureMsgf(NodeTemplate != nullptr, TEXT("FExecutionContext::ReleaseNodeInstance: Could not release node instance - Node template wasn't found, node descriptor is perhaps corrupted (we'll leak the node memory)")))
		{
			return;	// Node template wasn't found, node descriptor is perhaps corrupted (we'll leak the node memory)
		}

		// Manually bind our stack since we have everything we need
		FTraitStackBinding StackBinding;
		StackBinding.Context = this;
		StackBinding.NodeInstance = NodeInstance;
		StackBinding.NodeDescription = &NodeDesc;
		StackBinding.NodeTemplate = NodeTemplate;

		// Start destruction with the top trait
		// We destruct the whole node which will include one or more sub-stacks (each with their own base trait)
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs + NodeTemplate->GetNumTraits() - 1;
		const FTraitTemplate* EndDesc = TraitDescs - 1;
		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; --TraitDesc)
		{
			const FTrait* Trait = GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			// Always update our stack binding to make sure it points to the right sub-stack
			const FTraitTemplate* BaseTraitDesc = TraitDesc - TraitDesc->GetTraitIndex();
			StackBinding.BaseTraitIndex = BaseTraitDesc - TraitDescs;
			StackBinding.TopTraitIndex = StackBinding.BaseTraitIndex + BaseTraitDesc->GetNumStackTraits() - 1;
			
			const uint32 TraitIndex = TraitDesc - TraitDescs;
			const FTraitBinding Binding(&StackBinding, Trait, TraitIndex);
			
			// Auto-unregister this trait with garbage collection
			if (TraitDesc->IsGCAware())
			{
				const FWeakTraitPtr TraitPtr = Binding.GetTraitPtr();
				FAnimNextGraphInstance& Instance = TraitPtr.GetNodeInstance()->GetOwner();
				Instance.UnregisterWithGC(TraitPtr);
			}
			
			Trait->DestructTraitInstance(*this, Binding);
		}

		NodeInstance->~FNodeInstance();
		FMemory::Free(NodeInstance);
	}

	bool FExecutionContext::GetStack(const FWeakTraitPtr& TraitPtr, FTraitStackBinding& OutStackBinding) const
	{
		if (!TraitPtr.IsValid())
		{
			OutStackBinding.Reset();
			UE_LOGF(LogAnimation, Warning, "FExecutionContext::GetStack: Provided trait pointer was not valid.");
			return false;
		}

		if (!ensureMsgf(IsBoundTo(TraitPtr.GetNodeInstance()->GetOwner()), TEXT("FExecutionContext::GetStack: The execution context isn't bound to the right graph instance")))
		{
			OutStackBinding.Reset();
			return false;	// The execution context isn't bound to the right graph instance
		}

		OutStackBinding = FTraitStackBinding(*this, TraitPtr);
		return OutStackBinding.IsValid();	// Construction can fail in rare cases, see constructor
	}

	void FExecutionContext::PushScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding)
	{
		if (!Binding.IsValid())
		{
			UE_LOGF(LogAnimation, Warning, "FExecutionContext::PushScopedInterfaceImpl: Cannot push scoped interface - Provided trait binding is not valid.");
			return;	// Don't queue invalid pointers
		}

		// We don't have any specific handling for duplicate entries, if a scoped interface is pushed twice,
		// it must also be popped twice (if popped manually)

		Private::FScopedInterfaceEntry* ScopedEntry = FreeScopedInterfaceEntryStackHead;
		if (ScopedEntry != nullptr)
		{
			// We have a free entry, set our new head
			FreeScopedInterfaceEntryStackHead = ScopedEntry->NextFreeEntry;

			// Update our entry
			ScopedEntry->Stack = *Binding.GetStack();
			ScopedEntry->TraitIndex = Binding.GetTraitIndex();
			ScopedEntry->InterfaceUID = InterfaceUID;
			ScopedEntry->bIsTraitCached = false;
			ScopedEntry->NextFreeEntry = nullptr;		// Mark it as not being a member of any list
		}
		else
		{
			// Allocate a new entry
			ScopedEntry = new(MemStack) Private::FScopedInterfaceEntry(Binding, InterfaceUID);
		}

		ScopedEntry->PrevScopedInterfaceStackEntry = ScopedInterfaceStackHead;
		ScopedInterfaceStackHead = ScopedEntry;

#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		// In development builds, we lazily query right away to ensure the interface we push is present
		// In Test and Shipping, we'll do so only when/if the interface is actually queried
		ensure(ScopedEntry->LazilyCacheTrait());
#endif
	}

	bool FExecutionContext::PopScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding)
	{
		if (!Binding.IsValid())
		{
			UE_LOGF(LogAnimation, Warning, "FExecutionContext::PopScopedInterfaceImpl: Cannot pop scoped interface - Provided trait binding is not valid.");
			return false;
		}

		// We don't have any specific handling for duplicate entries, if a scoped interface is pushed twice,
		// it must also be popped twice (if popped manually)

		// Start searching at the top of the stack
		Private::FScopedInterfaceEntry* Entry = ScopedInterfaceStackHead;
		if (Entry->InterfaceUID == InterfaceUID &&				// Same interface
			Entry->Stack == *Binding.GetStack() &&				// Same stack
			Entry->TraitIndex == Binding.GetTraitIndex())		// Same trait
		{
			// We found the interface we were looking for, pop it
			// Add our entry to the free list
			Private::FScopedInterfaceEntry* PrevEntry = Entry->PrevScopedInterfaceStackEntry;
			Entry->NextFreeEntry = FreeScopedInterfaceEntryStackHead;
			FreeScopedInterfaceEntryStackHead = Entry;
			ScopedInterfaceStackHead = PrevEntry;

			return true;
		}

		return false;
	}

	bool FExecutionContext::PopStackScopedInterfaces(const FTraitStackBinding& StackBinding)
	{
		if (!StackBinding.IsValid())
		{
			UE_LOGF(LogAnimation, Warning, "FExecutionContext::PopStackScopedInterfaces: Cannot pop scoped interfaces - Provided stack binding is not valid.");
			return false;
		}

		bool bAnyPopped = false;

		// Start searching at the top of the stack
		Private::FScopedInterfaceEntry* Entry = ScopedInterfaceStackHead;
		while (Entry != nullptr)
		{
			if (Entry->Stack != StackBinding)
			{
				// This entry doesn't match our trait stack, stop searching
				break;
			}

			// We found a scoped interface owned by the trait stack, pop it
			// Add our entry to the free list
			Private::FScopedInterfaceEntry* PrevEntry = Entry->PrevScopedInterfaceStackEntry;
			Entry->NextFreeEntry = FreeScopedInterfaceEntryStackHead;
			FreeScopedInterfaceEntryStackHead = Entry;
			ScopedInterfaceStackHead = PrevEntry;
			bAnyPopped = true;

			// Continue execution in case this trait stack pushed multiple scoped interfaces

			// Move to the next entry on the stack
			Entry = PrevEntry;
		}

		return bAnyPopped;
	}

	bool FExecutionContext::GetScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, FTraitBinding& OutBinding) const
	{
		// Start searching at the top of the stack
		Private::FScopedInterfaceEntry* Entry = ScopedInterfaceStackHead;
		while (Entry != nullptr)
		{
			if (Entry->InterfaceUID == InterfaceUID)
			{
				// We found the interface we were looking for, return it
				Entry->LazilyCacheTrait();

				OutBinding = Entry->Trait;
				return true;
			}

			// Move to the next entry on the stack
			Entry = Entry->PrevScopedInterfaceStackEntry;
		}

		// We didn't find the interface we were looking for
		OutBinding.Reset();
		return false;
	}

	void FExecutionContext::ForEachScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, TFunctionRef<bool(FTraitBinding& Binding)> InFunction) const
	{
		// Start searching at the top of the stack
		Private::FScopedInterfaceEntry* Entry = ScopedInterfaceStackHead;
		while (Entry != nullptr)
		{
			if (Entry->InterfaceUID == InterfaceUID)
			{
				// We found the interface we were looking for, forward it to our callback
				Entry->LazilyCacheTrait();

				const bool bContinueSearching = InFunction(Entry->Trait);
				if (!bContinueSearching)
				{
					break;	// The callback returned false, we are done searching
				}
			}

			// Move to the next entry on the stack
			Entry = Entry->PrevScopedInterfaceStackEntry;
		}
	}

	void FExecutionContext::ForEachComponent(TFunctionRef<bool(FUAFGraphInstanceComponent&)> InFunction) const
	{
		check(IsBound());
		RootGraphInstance->ForEachComponent<FUAFGraphInstanceComponent>(InFunction);
	}

	void FExecutionContext::RaiseInputTraitEvent(FAnimNextTraitEventPtr Event)
	{
		ensureMsgf(false, TEXT("Raising input trait events is not supported in this context"));
	}

	void FExecutionContext::RaiseOutputTraitEvent(FAnimNextTraitEventPtr Event)
	{
		ensureMsgf(false, TEXT("Raising output trait events is not supported in this context"));
	}

	const FNodeDescription& FExecutionContext::GetNodeDescription(const FAnimNextGraphInstance& GraphInstance, FNodeHandle NodeHandle) const
	{
		// Grab the node description from the specified graph
		const UUAFAnimGraph* AnimationGraph = GraphInstance.GetAnimationGraph();
		return *reinterpret_cast<const FNodeDescription*>(&AnimationGraph->SharedDataBuffer[NodeHandle.GetSharedOffset()]);
	}

	const FNodeDescription& FExecutionContext::GetNodeDescription(const FNodeInstance& NodeInstance) const
	{
		// Grab the node description from the owning graph
		return GetNodeDescription(NodeInstance.GetOwner(), NodeInstance.GetNodeHandle());
	}

	const FNodeTemplate* FExecutionContext::GetNodeTemplate(const FNodeDescription& NodeDesc) const
	{
		check(NodeDesc.GetTemplateHandle().IsValid());
		return NodeTemplateRegistry.Find(NodeDesc.GetTemplateHandle());
	}

	const FTrait* FExecutionContext::GetTrait(const FTraitTemplate& Template) const
	{
		check(Template.GetRegistryHandle().IsValid());
		return TraitRegistry.Find(Template.GetRegistryHandle());
	}

	const UObject* FExecutionContext::GetHostObject() const
	{
		const FAnimNextModuleInstance* ModuleInstance = GetRootGraphInstance().GetModuleInstance();
		if (ModuleInstance)
		{
			return ModuleInstance->GetObject();
		}

		return nullptr;
	}

	TWeakObjectPtr<const USkeletalMeshComponent> FExecutionContext::GetBindingObject() const
	{
		// Pull the component from the system's output component
		if (IsBound())
		{
			if (FAnimNextModuleInstance* SystemInstance = RootGraphInstance->GetModuleInstance())
			{
				return SystemInstance->GetOrAddComponent<FUAFSystemOutputComponent>().GetSkeletalMeshComponent();
			}
		}
		
		return TWeakObjectPtr<const USkeletalMeshComponent>();
	}

#if UE_ENABLE_DEBUG_DRAWING
	FRigVMDrawInterface* FExecutionContext::GetDebugDrawInterface() const
	{
		check(IsBound());
		return RootGraphInstance->GetModuleInstance()->GetDebugDrawInterface();
	}

	const FTransform FExecutionContext::GetHostTransform() const
	{
		if (const UObject* ContextObject = GetHostObject())
		{
			if (const UActorComponent* ActorComponent = Cast<UActorComponent>(ContextObject))
			{
				if (const AActor* OwningActor = ActorComponent->GetOwner())
				{
					return OwningActor->GetActorTransform();
				}
			}
		}

		// @TODO: Add support for non-actors.
		return FTransform::Identity;
	}
#endif

	void FExecutionContext::Log(ELogVerbosity::Type Verbosity, const FString& InMessage, const FString& TraitName) const
	{
		const UObject* HostObject = GetHostObject();
		const UObject* SystemOwner = HostObject ? HostObject->GetOuter() : nullptr;
		const UObject* RootGraphAsset = IsBound() ? RootGraphInstance->GetAsset<UObject>() : nullptr;

		switch (Verbosity)
		{
		case ELogVerbosity::Fatal:
			UE_LOGFMT(LogAnimation, Fatal, "{TraitName}: {Message} - [Root Graph: {GraphPath}] [Owner: {OwnerPath}]", TraitName, InMessage, FSoftObjectPath(RootGraphAsset), FSoftObjectPath(SystemOwner));
			break;
		case ELogVerbosity::Error:
			UE_LOGFMT(LogAnimation, Error, "{TraitName}: {Message} - [Root Graph: {GraphPath}] [Owner: {OwnerPath}]", TraitName, InMessage, FSoftObjectPath(RootGraphAsset), FSoftObjectPath(SystemOwner));
			break;
		case ELogVerbosity::Warning:
			UE_LOGFMT(LogAnimation, Warning, "{TraitName}: {Message} - [Root Graph: {GraphPath}] [Owner: {OwnerPath}]", TraitName, InMessage, FSoftObjectPath(RootGraphAsset), FSoftObjectPath(SystemOwner));
			break;
		case ELogVerbosity::Display:
			UE_LOGFMT(LogAnimation, Display, "{TraitName}: {Message} - [Root Graph: {GraphPath}] [Owner: {OwnerPath}]", TraitName, InMessage, FSoftObjectPath(RootGraphAsset), FSoftObjectPath(SystemOwner));
			break;
		case ELogVerbosity::Log:
			UE_LOGFMT(LogAnimation, Log, "{TraitName}: {Message} - [Root Graph: {GraphPath}] [Owner: {OwnerPath}]", TraitName, InMessage, FSoftObjectPath(RootGraphAsset), FSoftObjectPath(SystemOwner));
			break;
		default:
			UE_LOGFMT(LogAnimation, Error, "Unsupported log category used! Message: {TraitName}: {Message} - [Root Graph: {GraphPath}] [Owner: {OwnerPath}]", TraitName, InMessage, FSoftObjectPath(RootGraphAsset), FSoftObjectPath(SystemOwner));
			break;
		}
	}

}
