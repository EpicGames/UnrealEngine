// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/NodeDescription.h"

#include "Serialization/Archive.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/NodeTemplateRegistry.h"

namespace UE::UAF
{
	void FNodeDescription::Serialize(FArchive& Ar)
	{
		const FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();

		Ar << NodeID;

		if (Ar.IsSaving())
		{
			const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(TemplateHandle);

			uint32 TemplateUID = NodeTemplate->GetUID();
			Ar << TemplateUID;
		}
		else if (Ar.IsLoading())
		{
			uint32 TemplateUID = 0;
			Ar << TemplateUID;

			TemplateHandle = NodeTemplateRegistry.Find(TemplateUID);
		}
		else
		{
			// Counting, etc
			int32 TemplateOffset = TemplateHandle.GetTemplateOffset();
			Ar << TemplateOffset;
		}

		// Use our template to serialize our traits
		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
		const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(TemplateHandle);

		const uint32 NumTraits = NodeTemplate->GetNumTraits();
		const FTraitTemplate* TraitTemplates = NodeTemplate->GetTraits();
		for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FTraitTemplate& TraitTemplate = TraitTemplates[TraitIndex];

			const FTraitRegistryHandle TraitHandle = TraitTemplate.GetRegistryHandle();
			FAnimNextTraitSharedData* SharedData = TraitTemplate.GetTraitDescription(*this);

			const FTrait* Trait = TraitRegistry.Find(TraitHandle);
			Trait->SerializeTraitSharedData(Ar, *SharedData);
		}
	}
}
