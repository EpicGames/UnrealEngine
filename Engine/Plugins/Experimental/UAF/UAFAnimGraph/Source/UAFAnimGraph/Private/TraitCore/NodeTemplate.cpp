// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/NodeTemplate.h"

#include "Serialization/Archive.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/NodeDescription.h"
#include "TraitCore/NodeInstance.h"

namespace UE::UAF
{
	namespace Private
	{
		static uint32 GetNumSubStackLatentProperties(const FTraitRegistry& TraitRegistry, const FTraitTemplate* TraitTemplates, uint32 NumTraits, uint32 BaseTraitIndex)
		{
			const FTraitTemplate& BaseTraitTemplate = TraitTemplates[BaseTraitIndex];
			check(BaseTraitTemplate.GetMode() == ETraitMode::Base);

			const uint32 NumSubStackTraits = BaseTraitTemplate.GetNumStackTraits();
			uint32 NumSubStackLatentProperties = 0;

			for (uint32 SubStackTraitIndex = 0; SubStackTraitIndex < NumSubStackTraits; ++SubStackTraitIndex)
			{
				const uint32 TraitIndex = BaseTraitIndex + SubStackTraitIndex;
				check(TraitIndex < NumTraits);

				const FTraitTemplate& TraitTemplate = TraitTemplates[TraitIndex];
				const FTraitUID TraitUID = TraitTemplate.GetUID();

				if (const FTrait* Trait = TraitRegistry.Find(TraitUID))
				{
					NumSubStackLatentProperties += Trait->GetNumLatentTraitProperties();
				}
			}

			return NumSubStackLatentProperties;
		}
	}

	void FNodeTemplate::Serialize(FArchive& Ar)
	{
		Ar << UID;
		Ar << NumTraits;

		const uint32 NumTraits_ = GetNumTraits();
		FTraitTemplate* TraitTemplates = GetTraits();

		for (uint32 TraitIndex = 0; TraitIndex < NumTraits_; ++TraitIndex)
		{
			TraitTemplates[TraitIndex].Serialize(Ar);
		}

		if (Ar.IsLoading())
		{
			// When loading, make sure to recompute all runtime dependent values (e.g. sizes and offsets)
			Finalize();
		}
	}

	void FNodeTemplate::Finalize()
	{
		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

		const uint32 NumTraits_ = GetNumTraits();
		FTraitTemplate* TraitTemplates = GetTraits();

		uint32 SharedDataOffset = sizeof(FNodeDescription);
		uint32 InstanceDataOffset = sizeof(FNodeInstance);
		uint32 SharedLatentPropertyHandlesOffset = 0;
		uint32 MinAlignment32 = alignof(FNodeDescription);

		for (uint32 TraitIndex = 0; TraitIndex < NumTraits_; ++TraitIndex)
		{
			FTraitTemplate& TraitTemplate = TraitTemplates[TraitIndex];
			const FTraitUID TraitUID = TraitTemplate.GetUID();

			uint32 NumLatentProperties = 0;
			uint32 TraitSharedDataOffset = 0;
			uint32 TraitSharedLatentPropertyHandlesOffset = 0;
			uint32 TraitInstanceDataOffset = 0;	// For instance data, 0 is an invalid offset since the data follows an instance of FNodeInstance

			uint32 NumSubStackLatentProperties = 0;
			if (TraitTemplate.GetMode() == ETraitMode::Base)
			{
				NumSubStackLatentProperties = Private::GetNumSubStackLatentProperties(TraitRegistry, TraitTemplates, NumTraits_, TraitIndex);
			}
			// Skip traits that we can't find
			// If a trait isn't loaded and we attempt to run the graph, it will be a no-op entry
			if (const FTrait* Trait = TraitRegistry.Find(TraitUID))
			{
				const FTraitMemoryLayout MemoryLayout = Trait->GetTraitMemoryDescription();

				// Align our data
				SharedDataOffset = Align(SharedDataOffset, MemoryLayout.SharedDataAlignment);
				InstanceDataOffset = Align(InstanceDataOffset, MemoryLayout.InstanceDataAlignment);

				MinAlignment32 = FMath::Max(MinAlignment32, MemoryLayout.SharedDataAlignment);
				MinAlignment32 = FMath::Max(MinAlignment32, MemoryLayout.InstanceDataAlignment);

				// Save our trait offsets
				TraitSharedDataOffset = SharedDataOffset;
				TraitInstanceDataOffset = InstanceDataOffset;

				// Include our trait
				SharedDataOffset += MemoryLayout.SharedDataSize;
				InstanceDataOffset += MemoryLayout.InstanceDataSize;

				// Base traits include the list of all latent property handles in its shared data
				// Latent property offsets will point into that list
				if (TraitTemplate.GetMode() == ETraitMode::Base)
				{
					// Align our handles
					SharedDataOffset = Align(SharedDataOffset, alignof(FLatentPropertiesHeader));

					// Save the offset where we start, we'll increment it as we consume it
					SharedLatentPropertyHandlesOffset = SharedDataOffset;

					// Include the handles in the shared data and their header
					SharedDataOffset += sizeof(FLatentPropertiesHeader) + (NumSubStackLatentProperties * sizeof(FLatentPropertyHandle));

					// Skip the header
					SharedLatentPropertyHandlesOffset += sizeof(FLatentPropertiesHeader);
				}

				// Save our latent pins offset (if we have any)
				NumLatentProperties = Trait->GetNumLatentTraitProperties();

				// The handle offset points to the first handle, if we are a base trait, our header precedes it
				TraitSharedLatentPropertyHandlesOffset = SharedLatentPropertyHandlesOffset;
				SharedLatentPropertyHandlesOffset += NumLatentProperties * sizeof(FLatentPropertyHandle);
			}

			check(NumSubStackLatentProperties <= MAX_uint16);
			check(NumLatentProperties <= MAX_uint16);

			TraitTemplate.NumLatentProperties = static_cast<uint16>(NumLatentProperties);
			TraitTemplate.NumSubStackLatentProperties = static_cast<uint16>(NumSubStackLatentProperties);

			check(TraitSharedDataOffset <= MAX_uint16);
			check(TraitSharedLatentPropertyHandlesOffset <= MAX_uint16);
			check(TraitInstanceDataOffset <= MAX_uint16);
			check(MinAlignment32 <= MAX_uint8);

			// Update our trait offsets
			TraitTemplate.NodeSharedOffset = static_cast<uint16>(TraitSharedDataOffset);
			TraitTemplate.NodeSharedLatentPropertyHandlesOffset = static_cast<uint16>(TraitSharedLatentPropertyHandlesOffset);
			TraitTemplate.NodeInstanceOffset = static_cast<uint16>(TraitInstanceDataOffset);
		}

		MinAlignment = static_cast<uint8>(MinAlignment32);

		// Make sure we respect our alignment constraints
		SharedDataOffset = Align(SharedDataOffset, alignof(FNodeDescription));

		// Our size is the offset of the trait that would follow afterwards
		// If the size is too large, we'll end up truncating the offsets/size
		// Set a value of 0 to be able to detect it later
		NodeSharedDataSize = SharedDataOffset > MAX_uint16 ? 0 : static_cast<uint16>(SharedDataOffset);
		NodeInstanceDataSize = InstanceDataOffset > MAX_uint16 ? 0 : static_cast<uint16>(InstanceDataOffset);
	}
}
