// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/NodeTemplateBuilder.h"

#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitInterfaces/IGarbageCollection.h"

namespace UE::UAF
{
	void FNodeTemplateBuilder::AddTrait(FTraitUID TraitUID)
	{
		TraitUIDs.Add(TraitUID);
	}

	FNodeTemplate* FNodeTemplateBuilder::BuildNodeTemplate(TArray<uint8>& NodeTemplateBuffer) const
	{
		return BuildNodeTemplate(TraitUIDs, NodeTemplateBuffer);
	}

	FNodeTemplate* FNodeTemplateBuilder::BuildNodeTemplate(TConstArrayView<FTraitUID> InTraitUIDs, TArray<uint8>& NodeTemplateBuffer)
	{
		NodeTemplateBuffer.Reset();

		const uint32 NodeTemplateUID = GetNodeTemplateUID(InTraitUIDs);

		NodeTemplateBuffer.AddUninitialized(sizeof(FNodeTemplate));
		new(NodeTemplateBuffer.GetData()) FNodeTemplate(NodeTemplateUID, InTraitUIDs.Num());

		for (int32 TraitIndex = 0; TraitIndex < InTraitUIDs.Num(); ++TraitIndex)
		{
			AppendTemplateTrait(InTraitUIDs, TraitIndex, NodeTemplateBuffer);
		}

		// Grab pointer after everything is setup, we could re-alloc
		FNodeTemplate* NodeTemplate = reinterpret_cast<FNodeTemplate*>(NodeTemplateBuffer.GetData());

		// Perform all our finalizing work
		NodeTemplate->Finalize();

		return NodeTemplate;
	}

	void FNodeTemplateBuilder::Reset()
	{
		TraitUIDs.Reset();
	}

	uint32 FNodeTemplateBuilder::GetNodeTemplateUID(TConstArrayView<FTraitUID> InTraitUIDs)
	{
		uint32 NodeTemplateUID = 0;

		for (const FTraitUID& TraitUID : InTraitUIDs)
		{
			NodeTemplateUID = HashCombineFast(NodeTemplateUID, TraitUID.GetUID());
		}

		return NodeTemplateUID;
	}

	void FNodeTemplateBuilder::AppendTemplateTrait(
		TConstArrayView<FTraitUID> InTraitUIDs, int32 TraitIndex,
		TArray<uint8>& NodeTemplateBuffer)
	{
		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

		const FTraitUID TraitUID = InTraitUIDs[TraitIndex];
		const FTraitRegistryHandle TraitHandle = TraitRegistry.FindHandle(TraitUID);
		const FTrait* Trait = TraitRegistry.Find(TraitHandle);
		const ETraitMode TraitMode = Trait->GetTraitMode();

		uint32 TraitIndexOrNumTraits;
		if (TraitMode == ETraitMode::Base)
		{
			// Find out how many traits we have
			TraitIndexOrNumTraits = 1;
			for (int32 Index = TraitIndex + 1; Index < InTraitUIDs.Num(); ++Index)	// Skip ourself
			{
				const FTrait* ChildTrait = TraitRegistry.Find(InTraitUIDs[Index]);
				if (ChildTrait->GetTraitMode() == ETraitMode::Base)
				{
					break;	// Found another base trait, we are done
				}

				// We are additive
				TraitIndexOrNumTraits++;
			}
		}
		else
		{
			// Find out our stack trait index (as opposed to the node trait index)
			// Iterate from this additive trait until we find our base trait
			TraitIndexOrNumTraits = 1;	// We are at least the second trait (first additive)
			for (int32 Index = TraitIndex - 1; Index >= 0; --Index)
			{
				const FTrait* ParentTrait = TraitRegistry.Find(InTraitUIDs[Index]);
				if (ParentTrait->GetTraitMode() == ETraitMode::Base)
				{
					break;	// Found our base trait, we are done
				}

				// We are additive
				TraitIndexOrNumTraits++;
			}
		}
		
		const bool bGCAware = Trait->GetTraitInterface(IGarbageCollection::InterfaceUID) != nullptr;

		// Append our trait template
		const int32 BufferIndex = NodeTemplateBuffer.AddUninitialized(sizeof(FTraitTemplate));
		new(&NodeTemplateBuffer[BufferIndex]) FTraitTemplate(TraitUID, TraitHandle, TraitMode, bGCAware, TraitIndexOrNumTraits);
	}
}
