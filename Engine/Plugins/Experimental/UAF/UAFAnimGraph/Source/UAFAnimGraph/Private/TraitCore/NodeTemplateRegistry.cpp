// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/NodeTemplateRegistry.h"

#include "TraitCore/TraitTemplate.h"
#include "TraitCore/NodeTemplate.h"
#include "Misc/ScopeRWLock.h"

namespace UE::UAF
{
	namespace Private
	{
		static FNodeTemplateRegistry* GNodeTemplateRegistry = nullptr;
	}

	FNodeTemplateRegistry& FNodeTemplateRegistry::Get()
	{
		checkf(Private::GNodeTemplateRegistry, TEXT("Node Template Registry is not instanced. It is only valid to access this while the engine module is loaded."));
		return *Private::GNodeTemplateRegistry;
	}

	void FNodeTemplateRegistry::Init()
	{
		if (ensure(Private::GNodeTemplateRegistry == nullptr))
		{
			Private::GNodeTemplateRegistry = new FNodeTemplateRegistry();
		}
	}

	void FNodeTemplateRegistry::Destroy()
	{
		if (ensure(Private::GNodeTemplateRegistry != nullptr))
		{
			delete Private::GNodeTemplateRegistry;
			Private::GNodeTemplateRegistry = nullptr;
		}
	}

#if WITH_DEV_AUTOMATION_TESTS
	FNodeTemplateRegistry* FNodeTemplateRegistry::Swap(FNodeTemplateRegistry* Other)
	{
		FNodeTemplateRegistry* Original = Private::GNodeTemplateRegistry;
		Private::GNodeTemplateRegistry = Other;
		return Original;
	}
#endif

	FNodeTemplateRegistryHandle FNodeTemplateRegistry::Find(uint32 NodeTemplateUID) const
	{
		FReadScopeLock Lock(TemplateLock);
		const FNodeTemplateRegistryHandle* TemplateHandle = TemplateUIDToHandleMap.Find(NodeTemplateUID);
		return TemplateHandle != nullptr ? *TemplateHandle : FNodeTemplateRegistryHandle();
	}

	FNodeTemplateRegistryHandle FNodeTemplateRegistry::FindOrAdd(const FNodeTemplate* NodeTemplate)
	{
		if (NodeTemplate == nullptr)
		{
			return FNodeTemplateRegistryHandle();
		}

		const uint32 TemplateUID = NodeTemplate->GetUID();

		{
			FReadScopeLock Lock(TemplateLock);
			if (FNodeTemplateRegistryHandle* TemplateHandle = TemplateUIDToHandleMap.Find(TemplateUID))
			{
				// We have already seen this template before, return its handle
				return *TemplateHandle;
			}
		}

		{
			FWriteScopeLock Lock(TemplateLock);

			// Re-check the map in case we added during lock upgrade
			if (FNodeTemplateRegistryHandle* TemplateHandle = TemplateUIDToHandleMap.Find(TemplateUID))
			{
				// We have already seen this template before, return its handle
				return *TemplateHandle;
			}

			// This is a new template
			const uint32 TemplateSize = NodeTemplate->GetNodeTemplateSize();

			// Copy the template into our global buffer
			TemplateBuffer.Append((uint8*)NodeTemplate, TemplateSize);
			const uint32 TemplateOffset = TemplateBuffer.Num() - TemplateSize;

			// Update our mapping
			FNodeTemplateRegistryHandle TemplateHandle = FNodeTemplateRegistryHandle::MakeHandle(TemplateOffset);
			TemplateUIDToHandleMap.Add(TemplateUID, TemplateHandle);

			return TemplateHandle;
		}
	}

	void FNodeTemplateRegistry::Unregister(const FNodeTemplate* NodeTemplate)
	{
		if (NodeTemplate == nullptr)
		{
			return;
		}

		const uint32 TemplateUID = NodeTemplate->GetUID();

		FWriteScopeLock Lock(TemplateLock);
		TemplateUIDToHandleMap.Remove(TemplateUID);
	}

	FNodeTemplate* FNodeTemplateRegistry::FindMutable(FNodeTemplateRegistryHandle TemplateHandle)
	{
		if (!TemplateHandle.IsValid())
		{
			return nullptr;
		}

		FReadScopeLock Lock(TemplateLock);
		return reinterpret_cast<FNodeTemplate*>(&TemplateBuffer[TemplateHandle.GetTemplateOffset()]);
	}

	const FNodeTemplate* FNodeTemplateRegistry::Find(FNodeTemplateRegistryHandle TemplateHandle) const
	{
		if (!TemplateHandle.IsValid())
		{
			return nullptr;
		}

		FReadScopeLock Lock(TemplateLock);
		return reinterpret_cast<const FNodeTemplate*>(&TemplateBuffer[TemplateHandle.GetTemplateOffset()]);
	}

	uint32 FNodeTemplateRegistry::GetNum() const
	{
		FReadScopeLock Lock(TemplateLock);
		return TemplateUIDToHandleMap.Num();
	}
}
