// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/Trait.h"

#include "TraitCore/TraitRegistry.h"

namespace UE::UAF
{
	void FTrait::SerializeTraitSharedData(FArchive& Ar, FAnimNextTraitSharedData& SharedData) const
	{
		UScriptStruct* SharedDataStruct = GetTraitSharedDataStruct();
		check(IsAligned(&SharedData, SharedDataStruct->GetMinAlignment()));
		SharedDataStruct->SerializeItem(Ar, &SharedData, nullptr);
	}

	FTraitLatentPropertyMemoryLayout FTrait::GetLatentPropertyMemoryLayoutImpl(
		const FAnimNextTraitSharedData& SharedData,
		FName PropertyName,
		uint32 PropertyIndex,
		TArray<FTraitLatentPropertyMemoryLayout>& LatentPropertyMemoryLayouts) const
	{
		// The property index here isn't important as long as it is stable, only used for caching purposes
		check(LatentPropertyMemoryLayouts.IsValidIndex(PropertyIndex));
		if (LatentPropertyMemoryLayouts[PropertyIndex].Size == 0)
		{
			// This is a new entry, initialize it
			// No need for locking, this is a deterministic write
			const UScriptStruct* SharedDataStruct = GetTraitSharedDataStruct();
			const FProperty* Property = SharedDataStruct->FindPropertyByName(PropertyName);
			check(Property != nullptr);

			LatentPropertyMemoryLayouts[PropertyIndex].Alignment = Property->GetMinAlignment();

			// Latent property handles are in the order defined by the enumerator macro within the shared data,
			// which means their index can differ from the one we serialize with which uses the UStruct order
			LatentPropertyMemoryLayouts[PropertyIndex].LatentPropertyIndex = GetLatentPropertyIndex(SharedData, PropertyName);

			// Ensure alignment is visible before we write the size to avoid torn reads
			FPlatformMisc::MemoryBarrier();

			LatentPropertyMemoryLayouts[PropertyIndex].Size = Property->GetSize();
		}
		
		return LatentPropertyMemoryLayouts[PropertyIndex];
	}

	TArray<FTraitInterfaceUID> FTrait::BuildTraitInterfaceList(
		const TConstArrayView<FTraitInterfaceUID>& SuperInterfaces,
		std::initializer_list<FTraitInterfaceUID> InterfaceList)
	{
		TArray<FTraitInterfaceUID> Result;
		Result.Reserve(SuperInterfaces.Num() + InterfaceList.size());

		Result.Append(SuperInterfaces);

		for (FTraitInterfaceUID InterfaceID : InterfaceList)
		{
			Result.AddUnique(InterfaceID);
		}

		Result.Shrink();
		Result.Sort();
		return Result;
	}

	TArray<FTraitEventUID> FTrait::BuildTraitEventList(
		const TConstArrayView<FTraitEventUID>& SuperEvents,
		std::initializer_list<FTraitEventUID> EventList)
	{
		TArray<FTraitEventUID> Result;
		Result.Reserve(SuperEvents.Num() + EventList.size());

		Result.Append(SuperEvents);

		for (FTraitEventUID InterfaceID : EventList)
		{
			Result.AddUnique(InterfaceID);
		}

		Result.Shrink();
		Result.Sort();
		return Result;
	}

#if WITH_EDITOR
	void FTrait::SaveTraitSharedData(const TFunction<FString(FName PropertyName)>& GetTraitProperty, FAnimNextTraitSharedData& OutSharedData) const
	{
		const UScriptStruct* SharedDataStruct = GetTraitSharedDataStruct();

		uint8* SharedData = reinterpret_cast<uint8*>(&OutSharedData);

		// Initialize our output struct with its default values
		SharedDataStruct->InitializeDefaultValue(SharedData);

		// Use UE reflection to iterate over every property
		// We convert every property from its string representation into its binary form
		for (const FProperty* Property = SharedDataStruct->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
		{
			// No need to skip editor only properties since serialization will take care of that afterwards

			const FString PropertyValue = GetTraitProperty(Property->GetFName());
			if (PropertyValue.Len() != 0)
			{
				const TCHAR* PropertyValuePtr = *PropertyValue;

				// C-style array properties aren't handled by ExportText, we need to handle it manually
				const bool bIsCArray = Property->ArrayDim > 1;
				if (bIsCArray)
				{
					ensure(PropertyValuePtr[0] == TEXT('('));
					PropertyValuePtr++;
				}

				for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
				{
					void* DataPtr = Property->ContainerPtrToValuePtr<void>(SharedData, Index);
					PropertyValuePtr = Property->ImportText_Direct(PropertyValuePtr, DataPtr, nullptr, PPF_SerializedAsImportText);

					if (Index + 1 < Property->ArrayDim)
					{
						ensure(PropertyValuePtr[0] == TEXT(','));
						PropertyValuePtr++;
					}
				}

				if (bIsCArray)
				{
					ensure(PropertyValuePtr[0] == TEXT(')'));
					PropertyValuePtr++;
				}
			}
		}
	}

	uint32 FTrait::GetLatentPropertyHandles(
		const FAnimNextTraitSharedData* InSharedData,
		TArray<FLatentPropertyMetadata>& OutLatentPropertyHandles,
		bool bFilterEditorOnly,
		const TFunction<uint16(FName PropertyName)>& GetTraitLatentPropertyIndex) const
	{
		uint32 NumHandlesAdded = 0;

		const UStruct* BaseStruct = GetTraitSharedDataStruct();

		// The property linked list on UScriptStruct iterates over the properties starting in the derived type
		// but with latent properties, the base type should be the first to be visited.
		// Gather our struct hierarchy from most derived to base
		TArray<const UStruct*> StructHierarchy;

		do
		{
			StructHierarchy.Add(BaseStruct);
			BaseStruct = BaseStruct->GetSuperStruct();
		}
		while (BaseStruct != nullptr);

		// Gather our latent properties from base to most derived
		for (auto It = StructHierarchy.rbegin(); It != StructHierarchy.rend(); ++It)
		{
			const UStruct* SharedDataStruct = *It;
			for (const FField* Field = SharedDataStruct->ChildProperties; Field != nullptr; Field = Field->Next)
			{
				const FProperty* Property = CastField<FProperty>(Field);

				if (bFilterEditorOnly && Property->IsEditorOnlyProperty())
				{
					continue;	// Skip editor only properties if we don't need them
				}

				// By default, properties are latent
				// However, there are exceptions:
				//     - Properties marked as hidden are not visible in the editor and cannot be hooked up manually
				//     - Properties marked as inline are only visible in the details panel and cannot be hooked up to another node
				//     - Properties of trait handle type are never lazy since they just encode graph connectivity
				const bool bIsPotentiallyLatent =
					!Property->HasMetaData(TEXT("Hidden")) &&
					!Property->HasMetaData(TEXT("Inline")) &&
					Property->GetCPPType() != TEXT("FAnimNextTraitHandle");

				if (!bIsPotentiallyLatent)
				{
					continue;	// Skip non-latent properties
				}

				if (!IsPropertyLatent(*InSharedData, Property->GetFName()))
				{
					// Skip properties not part of the latent macro enumerator
					// This can occur if the property is missing from the enumerator by mistake
					// in which case we'll warn during compilation
					continue;
				}

				FLatentPropertyMetadata Metadata;
				Metadata.Name = Property->GetFName();
				Metadata.RigVMIndex = GetTraitLatentPropertyIndex(Property->GetFName());

				// Always false for now, we don't support freezing yet
				Metadata.bCanFreeze = false;
				Metadata.bOnBecomeRelevant = Property->HasMetaData("OnBecomeRelevant");

				OutLatentPropertyHandles.Add(Metadata);
				NumHandlesAdded++;
			}
		}

		return NumHandlesAdded;
	}
#endif

	uint32 FTrait::GetVariableMappedLatentPropertyHandles(
		const FAnimNextTraitSharedData* InSharedData,
		TArray<FLatentPropertyMetadata>& OutLatentPropertyHandles,
		bool bFilterEditorOnly,
		const TFunction<uint16(FName PropertyName)>& GetTraitVariableMappingIndex) const
	{
		uint32 NumHandlesAdded = 0;

		const UStruct* BaseStruct = GetTraitSharedDataStruct();

		// The property linked list on UScriptStruct iterates over the properties starting in the derived type
		// but with latent properties, the base type should be the first to be visited.
		// Gather our struct hierarchy from most derived to base
		TArray<const UStruct*> StructHierarchy;

		do
		{
			StructHierarchy.Add(BaseStruct);
			BaseStruct = BaseStruct->GetSuperStruct();
		}
		while (BaseStruct != nullptr);

		// Gather our latent properties from base to most derived
		for (auto It = StructHierarchy.rbegin(); It != StructHierarchy.rend(); ++It)
		{
			const UStruct* SharedDataStruct = *It;
			for (const FField* Field = SharedDataStruct->ChildProperties; Field != nullptr; Field = Field->Next)
			{
				const FProperty* Property = CastField<FProperty>(Field);

				if (bFilterEditorOnly && Property->IsEditorOnlyProperty())
				{
					continue;	// Skip editor only properties if we don't need them
				}

				// By default, properties are latent
				// However, there are exceptions:
				//     - Properties of trait handle type are never lazy since they just encode graph connectivity
				
				bool bIsPotentiallyLatent = true;
				const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
				if (StructProperty != nullptr && StructProperty->Struct == FAnimNextTraitHandle::StaticStruct())
				{
					bIsPotentiallyLatent = false;
				}

				if (!bIsPotentiallyLatent)
				{
					continue;	// Skip non-latent properties
				}

				if (!IsPropertyLatent(*InSharedData, Property->GetFName()))
				{
					// Skip properties not part of the latent macro enumerator
					// This can occur if the property is missing from the enumerator by mistake
					// in which case we'll warn during compilation
					continue;
				}

				FLatentPropertyMetadata Metadata;
				Metadata.Name = Property->GetFName();
				Metadata.RigVMIndex = GetTraitVariableMappingIndex(Property->GetFName());

				// Always false for now, we don't support freezing yet
				Metadata.bCanFreeze = false;

				// Always true for now when calling this function
				Metadata.bUsesVariableCopy = true;

				OutLatentPropertyHandles.Add(Metadata);
				NumHandlesAdded++;
			}
		}

		return NumHandlesAdded;
	}

	FRigVMStructUpgradeInfo FTrait::GetUpgradeInfo() const
	{
		return FRigVMStructUpgradeInfo();
	}

	FTraitStaticInitHook::FTraitStaticInitHook(TraitConstructorFunc InTraitConstructor)
		: TraitConstructor(InTraitConstructor)
	{
		FTraitRegistry::StaticRegister(InTraitConstructor);
	}

	FTraitStaticInitHook::~FTraitStaticInitHook()
	{
		FTraitRegistry::StaticUnregister(TraitConstructor);
	}
}
