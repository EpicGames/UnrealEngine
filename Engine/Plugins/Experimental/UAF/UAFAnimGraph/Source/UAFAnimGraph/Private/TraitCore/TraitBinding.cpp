// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitBinding.h"

#include "TraitCore/ITraitInterface.h"

namespace UE::UAF
{
	FTraitInterfaceUID FTraitBinding::GetInterfaceUID() const
	{
		return InterfaceThisOffset != -1 ? GetInterfaceTyped<ITraitInterface>()->GetInterfaceUID() : FTraitInterfaceUID();
	}
}
