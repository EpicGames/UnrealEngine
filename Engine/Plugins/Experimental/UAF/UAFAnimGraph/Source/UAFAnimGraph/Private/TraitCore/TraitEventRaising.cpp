// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitEventRaising.h"
#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitEventList.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitStackBinding.h"
#include "Graph/UAFGraphInstanceComponent.h"
#include "EngineLogs.h"

namespace UE::UAF
{
	void RaiseTraitEvent(FExecutionContext& Context, const FTraitStackBinding& StackBinding, FAnimNextTraitEvent& Event)
	{
		if (!Event.IsValid())
		{
			return;	// Consumed events are invalid and do not propagate further
		}

		if (!StackBinding.IsValid())
		{
			UE_LOGF(LogAnimation, Warning, "RaiseTraitEvent: Could not raise trait event - The stack binding is invalid.");
			return;
		}

		FTraitBinding Binding;
		ensure(StackBinding.GetTopTrait(Binding));
		do
		{
			const ETraitStackPropagation Propagation = Binding.GetTrait()->OnTraitEvent(Context, Binding, Event);
			if (Propagation == ETraitStackPropagation::Stop || Event.IsConsumed())
			{
				// We don't wish to propagate the call further, stop here
				break;
			}
		} while (StackBinding.GetParentTrait(Binding, Binding));
	}

	void RaiseTraitEvents(FExecutionContext& Context, const FTraitStackBinding& StackBinding, const FTraitEventList& EventList)
	{
		if (!StackBinding.IsValid())
		{
			UE_LOGF(LogAnimation, Warning, "RaiseTraitEvents: Could not raise trait events - The stack binding is invalid.");
			return;
		}

		// Event handlers can raise events and as such the list may change while we iterate
		// However, if an event is added while we iterate, we will not visit it
		const int32 NumEvents = EventList.Num();
		for (int32 EventIndex = 0; EventIndex < NumEvents; ++EventIndex)
		{
			const FAnimNextTraitEventPtr Event = EventList[EventIndex];
			RaiseTraitEvent(Context, StackBinding, *Event);
		}
	}

	void RaiseTraitEvents(FExecutionContext& Context, FUAFGraphInstanceComponent& Component, const FTraitEventList& EventList)
	{
		// Event handlers can raise events and as such the list may change while we iterate
		// However, if an event is added while we iterate, we will not visit it
		const int32 NumEvents = EventList.Num();
		for (int32 EventIndex = 0; EventIndex < NumEvents; ++EventIndex)
		{
			const FAnimNextTraitEventPtr Event = EventList[EventIndex];
			if (Event->IsValid())
			{
				Component.OnTraitEvent(Context, *Event);
			}
		}
	}
}
