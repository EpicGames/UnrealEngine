// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitReader.h"
#include "Serialization/Archive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TraitHandle)

bool FAnimNextTraitHandle::Serialize(FArchive& Ar)
{
	Ar << PackedTraitIndexAndNodeHandle;

	if (Ar.IsLoading() && IsValid())
	{
		// On load, we hold a node ID that needs fix-up
		check(GetNodeHandle().IsNodeID());

		UE::UAF::FTraitReader& TraitReader = static_cast<UE::UAF::FTraitReader&>(Ar);
		*this = TraitReader.ResolveTraitHandle(*this);
		check(GetNodeHandle().IsSharedOffset());
	}

	return true;
}
