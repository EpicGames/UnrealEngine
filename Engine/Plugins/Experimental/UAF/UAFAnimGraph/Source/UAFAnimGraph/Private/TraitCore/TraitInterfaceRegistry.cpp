// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitInterfaceRegistry.h"

#include "TraitCore/ITraitInterface.h"
#include "EngineLogs.h"

namespace UE::UAF
{

namespace Private
{
	static FTraitInterfaceRegistry* GTraitInterfaceRegistry = nullptr;

	static TArray<TSharedPtr<ITraitInterface>> GPendingInterfaceRegistrationQueue;
}

FTraitInterfaceRegistry& FTraitInterfaceRegistry::Get()
{
	checkf(Private::GTraitInterfaceRegistry, TEXT("Trait Registry is not instanced. It is only valid to access this while the engine module is loaded."));
	return *Private::GTraitInterfaceRegistry;
}

void FTraitInterfaceRegistry::Init()
{
	if (ensure(Private::GTraitInterfaceRegistry == nullptr))
	{
		Private::GTraitInterfaceRegistry = new FTraitInterfaceRegistry();

		// Register all our pending static init traits
		for (TSharedPtr<ITraitInterface>& TraitInterface : Private::GPendingInterfaceRegistrationQueue)
		{
			Private::GTraitInterfaceRegistry->Register(TraitInterface);
		}

		// Reset the registration queue, it won't be used anymore now that the registry is up and ready
		Private::GPendingInterfaceRegistrationQueue.Empty(0);
	}
}

void FTraitInterfaceRegistry::Destroy()
{
	if (ensure(Private::GTraitInterfaceRegistry != nullptr))
	{
		TArray<FRegistryEntry> Entries;
		Private::GTraitInterfaceRegistry->TraitInterfaceUIDToEntryMap.GenerateValueArray(Entries);

		for (const FRegistryEntry& Entry : Entries)
		{
			Private::GTraitInterfaceRegistry->Unregister(Entry.TraitInterface);
		}

		delete Private::GTraitInterfaceRegistry;
		Private::GTraitInterfaceRegistry = nullptr;
	}
}

void FTraitInterfaceRegistry::StaticRegister(const TSharedPtr<ITraitInterface>& TraitInterface)
{
	if (Private::GTraitInterfaceRegistry != nullptr)
	{
		// Registry is already up and running, use it
		Private::GTraitInterfaceRegistry->Register(TraitInterface);
	}
	else
	{
		// Registry isn't ready yet, queue up our trait
		// Once Init() is called, our queue will be processed
		Private::GPendingInterfaceRegistrationQueue.Add(TraitInterface);
	}
}

void FTraitInterfaceRegistry::StaticUnregister(const TSharedPtr<ITraitInterface>& TraitInterface)
{
	if (Private::GTraitInterfaceRegistry != nullptr)
	{
		// Registry is already up and running, use it
		Private::GTraitInterfaceRegistry->Unregister(TraitInterface);
	}
	else
	{
		// Registry isn't ready yet or it got destroyed before the traits are unregistering
		const int32 TraitInterfaceIndex = Private::GPendingInterfaceRegistrationQueue.IndexOfByKey(TraitInterface);
		if (TraitInterfaceIndex != INDEX_NONE)
		{
			Private::GPendingInterfaceRegistrationQueue.RemoveAtSwap(TraitInterfaceIndex);
		}
	}
}

const ITraitInterface* FTraitInterfaceRegistry::Find(FTraitInterfaceUID TraitInterfaceUID) const
{
	if (!TraitInterfaceUID.IsValid())
	{
		UE_LOGF(LogAnimation, Warning, "FTraitInterfaceRegistry::Find: Provided trait interface UID is invalid.");
		return nullptr;
	}

	if (const FRegistryEntry* Entry = TraitInterfaceUIDToEntryMap.Find(TraitInterfaceUID.GetUID()))
	{
		return Entry->TraitInterface.Get();
	}

	// Trait UID not found
	return nullptr;
}

void FTraitInterfaceRegistry::Register(const TSharedPtr<ITraitInterface>& TraitInterface)
{
	if (TraitInterface == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FTraitInterfaceRegistry::Register: Could not register trait interface - Provided trait interface is invalid.");
		return;
	}

	const FTraitInterfaceUID TraitInterfaceUID = TraitInterface->GetInterfaceUID();

	if (ensure(!TraitInterfaceUIDToEntryMap.Contains(TraitInterfaceUID.GetUID())))
	{
		TraitInterfaceUIDToEntryMap.Add(TraitInterfaceUID.GetUID(), FRegistryEntry{ TraitInterface });
	}
}

void FTraitInterfaceRegistry::Unregister(const TSharedPtr<ITraitInterface>& TraitInterface)
{
	if (TraitInterface == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FTraitInterfaceRegistry::Unregister: Could not unregister trait interface - Provided trait interface is invalid.");
		return;
	}

	const FTraitInterfaceUID TraitInterfaceUID = TraitInterface->GetInterfaceUID();

	if (FRegistryEntry* Entry = TraitInterfaceUIDToEntryMap.Find(TraitInterfaceUID.GetUID()))
	{
		check(Entry->TraitInterface != nullptr);
		TraitInterfaceUIDToEntryMap.Remove(TraitInterfaceUID.GetUID());
	}
}

TArray<const ITraitInterface*> FTraitInterfaceRegistry::GetTraitInterfaces() const
{
	TArray<const ITraitInterface*> TraitInterfaces;
	TraitInterfaces.Reserve(TraitInterfaceUIDToEntryMap.Num());
	
	for (const auto& It : TraitInterfaceUIDToEntryMap)
	{
		TraitInterfaces.Add(It.Value.TraitInterface.Get());
	}

	return TraitInterfaces;
}

uint32 FTraitInterfaceRegistry::GetNum() const
{
	return TraitInterfaceUIDToEntryMap.Num();
}

// --- FTraitInterfaceStaticInitHook ---

FTraitInterfaceStaticInitHook::FTraitInterfaceStaticInitHook(const TSharedPtr<ITraitInterface>& InTraitInterface)
	: TraitInterface(InTraitInterface)
{
	FTraitInterfaceRegistry::StaticRegister(TraitInterface);
}

FTraitInterfaceStaticInitHook::~FTraitInterfaceStaticInitHook()
{
	FTraitInterfaceRegistry::StaticUnregister(TraitInterface);
}

} // end namespace UE::UAF
