// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitPtr.h"

#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"

namespace UE::UAF
{
	FTraitPtr::FTraitPtr(FNodeInstance* InNodeInstance, uint32 InTraitIndex)
		: PackedPointerAndFlags(InNodeInstance != nullptr ? reinterpret_cast<uintptr_t>(InNodeInstance) : 0)
		, TraitIndex(InTraitIndex)
	{
		check((reinterpret_cast<uintptr_t>(InNodeInstance) & FLAGS_MASK) == 0);	// Make sure we have enough alignment
		check(TraitIndex <= MAX_uint8);	// Make sure we don't truncate

		if (InNodeInstance != nullptr)
		{
			InNodeInstance->AddReference();
		}
	}

	FTraitPtr::FTraitPtr(const FTraitPtr& TraitPtr)
		: PackedPointerAndFlags(TraitPtr.PackedPointerAndFlags)
		, TraitIndex(TraitPtr.TraitIndex)
	{
		// Only increment the reference count if we aren't a weak handle
		// The new handle will remain weak
		if (!TraitPtr.IsWeak())
		{
			if (FNodeInstance* Node = TraitPtr.GetNodeInstance())
			{
				Node->AddReference();
			}
		}
	}

	FTraitPtr::FTraitPtr(const FWeakTraitPtr& TraitPtr)
		: PackedPointerAndFlags(reinterpret_cast<uintptr_t>(TraitPtr.GetNodeInstance()))
		, TraitIndex(TraitPtr.GetTraitIndex())
	{
		check((reinterpret_cast<uintptr_t>(TraitPtr.GetNodeInstance()) & FLAGS_MASK) == 0);	// Make sure we have enough alignment
		check(TraitPtr.GetTraitIndex() <= MAX_uint8);	// Make sure we don't truncate

		if (TraitPtr.IsValid())
		{
			PackedPointerAndFlags |= IS_WEAK_BIT;
		}
	}

	FTraitPtr::FTraitPtr(FTraitPtr&& TraitPtr) noexcept
		: PackedPointerAndFlags(TraitPtr.PackedPointerAndFlags)
		, TraitIndex(TraitPtr.TraitIndex)
	{
		TraitPtr.PackedPointerAndFlags = 0;
		TraitPtr.TraitIndex = 0;
	}

	FTraitPtr::FTraitPtr(FNodeInstance* InNodeInstance, EFlags InFlags, uint32 InTraitIndex)
		: PackedPointerAndFlags(InNodeInstance != nullptr ? (reinterpret_cast<uintptr_t>(InNodeInstance) | InFlags) : 0)
		, TraitIndex(InTraitIndex)
	{
		check((reinterpret_cast<uintptr_t>(InNodeInstance) & FLAGS_MASK) == 0);	// Make sure we have enough alignment
		check(TraitIndex <= MAX_uint8);	// Make sure we don't truncate

		// Only increment the reference count if we aren't a weak handle
		if (InNodeInstance != nullptr && (InFlags & IS_WEAK_BIT) == 0)
		{
			InNodeInstance->AddReference();
		}
	}

	FTraitPtr::~FTraitPtr()
	{
		Reset();
	}

	FTraitPtr& FTraitPtr::operator=(const FTraitPtr& TraitPtr)
	{
		// Add our new reference first in case this == TraitPtr

		// Only increment the reference count if we aren't a weak handle
		// The new handle will remain weak
		if (!TraitPtr.IsWeak())
		{
			if (FNodeInstance* NewNode = TraitPtr.GetNodeInstance())
			{
				NewNode->AddReference();
			}
		}

		Reset();

		PackedPointerAndFlags = TraitPtr.PackedPointerAndFlags;
		TraitIndex = TraitPtr.TraitIndex;

		return *this;
	}

	FTraitPtr& FTraitPtr::operator=(const FWeakTraitPtr& TraitPtr)
	{
		Reset();

		PackedPointerAndFlags = reinterpret_cast<uintptr_t>(TraitPtr.GetNodeInstance());
		TraitIndex = TraitPtr.GetTraitIndex();

		if (TraitPtr.IsValid())
		{
			PackedPointerAndFlags |= IS_WEAK_BIT;
		}

		return *this;
	}

	FTraitPtr& FTraitPtr::operator=(FTraitPtr&& TraitPtr) noexcept
	{
		Swap(PackedPointerAndFlags, TraitPtr.PackedPointerAndFlags);
		Swap(TraitIndex, TraitPtr.TraitIndex);

		return *this;
	}

	void FTraitPtr::Reset()
	{
		// Only decrement the reference count if we aren't a weak handle and if we are valid
		if (!IsWeak())
		{
			if (FNodeInstance* Node = GetNodeInstance())
			{
				FExecutionContext Context(Node->GetOwner());
				Context.ReleaseNodeInstance(*this);
			}
		}

		PackedPointerAndFlags = 0;
		TraitIndex = 0;
	}

	FWeakTraitPtr::FWeakTraitPtr(FNodeInstance* InNodeInstance, uint32 InTraitIndex)
		: NodeInstance(InNodeInstance)
		, TraitIndex(InTraitIndex)
	{
		check(TraitIndex <= MAX_uint8);	// Make sure we don't truncate
	}

	void FWeakTraitPtr::Reset()
	{
		NodeInstance = nullptr;
		TraitIndex = 0;
	}
}
