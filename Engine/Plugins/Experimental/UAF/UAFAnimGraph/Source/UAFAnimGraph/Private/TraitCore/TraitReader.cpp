// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitReader.h"

#include "TraitCore/TraitRegistry.h"
#include "TraitCore/NodeDescription.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/UObjectGlobals.h"

namespace UE::UAF
{
	FTraitReader::FTraitReader(const TArray<TObjectPtr<UObject>>& InGraphReferencedObjects, const TArray<FSoftObjectPath>& InGraphReferencedSoftObjects, FArchive& Ar)
		: FArchiveProxy(Ar)
		, GraphReferencedObjects(InGraphReferencedObjects)
		, GraphReferencedSoftObjects(InGraphReferencedSoftObjects)
	{
	}

	FTraitReader::EErrorState FTraitReader::ReadGraph(TArray<uint8>& GraphSharedData)
	{
		GraphSharedData.Empty(0);

		EErrorState ErrorState = ReadGraphSharedData(GraphSharedData);
		if (ErrorState != EErrorState::None)
		{
			return ErrorState;
		}

		return ErrorState;
	}

	FTraitReader::EErrorState FTraitReader::ReadGraphSharedData(TArray<uint8>& GraphSharedData)
	{
		// Read the node templates and register them as needed
		uint32 NumNodeTemplates = 0;
		*this << NumNodeTemplates;

		TArray<FNodeTemplateRegistryHandle> NodeTemplateHandles;
		NodeTemplateHandles.Reserve(NumNodeTemplates);

		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
		FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();

		{
			// We serialize each node template in the same buffer, they get copied into the registry and we don't need to retain them
			alignas(16) uint8 NodeTemplateBuffer[64 * 1024];
			FNodeTemplate* NodeTemplate = reinterpret_cast<FNodeTemplate*>(&NodeTemplateBuffer[0]);

			for (uint32 NodeTemplateIndex = 0; NodeTemplateIndex < NumNodeTemplates; ++NodeTemplateIndex)
			{
				NodeTemplate->Serialize(*this);

				if (!NodeTemplate->IsValid())
				{
					if (NodeTemplate->GetNodeSharedDataSize() == 0)
					{
						// This node shared data is too large
						return EErrorState::NodeSharedDataTooLarge;
					}

					if (NodeTemplate->GetNodeInstanceDataSize() == 0)
					{
						// This node instance data is too large
						return EErrorState::NodeInstanceDataTooLarge;
					}
				}

				// Register our node template
				NodeTemplateHandles.Add(NodeTemplateRegistry.FindOrAdd(NodeTemplate));
			}
		}

		// Read our graph shared data
		uint32 NumNodes = 0;
		*this << NumNodes;

		NodeHandles.Empty(0);
		NodeHandles.Reserve(NumNodes);

		// Calculate our shared data size and all node offsets
		uint32 SharedDataSize = 0;
		for (uint32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
		{
			uint32 NodeTemplateIndex = 0;
			*this << NodeTemplateIndex;

			if (NodeTemplateIndex >= NumNodeTemplates)
			{
				// Corrupted / malformed data
				return EErrorState::InvalidData;
			}

			const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(NodeTemplateHandles[NodeTemplateIndex]);
			check(NodeTemplate != nullptr);

			// Align this node
			check(NodeTemplate->GetMinAlignment() > 0);
			SharedDataSize = Align(SharedDataSize, NodeTemplate->GetMinAlignment());

			if ((size_t)SharedDataSize > MAXIMUM_GRAPH_SHARED_DATA_SIZE)
			{
				// This graph shared data is too large, we won't be able to create handles to this node
				return EErrorState::GraphTooLarge;
			}

			NodeHandles.Add(FNodeHandle::FromSharedOffset(SharedDataSize));	// This node starts here
			check(NodeHandles.Last().IsSharedOffset());

			uint32 NodeSharedDataSize = NodeTemplate->GetNodeSharedDataSize();

			if ((size_t)SharedDataSize + (size_t)NodeSharedDataSize > MAX_uint32)
			{
				// Detect overrun of SharedDataSize
				return EErrorState::GraphTooLarge;
			}

			SharedDataSize += NodeSharedDataSize;
		}

		// The shared data size might exceed MAXIMUM_GRAPH_SHARED_DATA_SIZE a little bit
		// The only requirement is that the node begins before that threshold so that we can
		// create handles to it

		GraphSharedData.Empty(0);
		GraphSharedData.AddZeroed(SharedDataSize);

		// Serialize our graph shared data
		for (uint32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
		{
			// Serialize our node shared data
			const uint32 SharedDataOffset = NodeHandles[NodeIndex].GetSharedOffset();
			if (SharedDataOffset >= SharedDataSize)
			{
				// Corrupted / malformed data
				return EErrorState::InvalidData;
			}

			FNodeDescription* NodeDesc = reinterpret_cast<FNodeDescription*>(&GraphSharedData[SharedDataOffset]);
			NodeDesc->Serialize(*this);

			const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(NodeDesc->GetTemplateHandle());
			check(NodeTemplate != nullptr);

			uint32 NodeInstanceDataSize = NodeTemplate->GetNodeInstanceDataSize();
			uint32 LatentPropertyOffset = NodeInstanceDataSize;

			// Read the latent properties and add them to our instance data (if any)
			const uint32 NumTraits = NodeTemplate->GetNumTraits();
			const FTraitTemplate* TraitTemplates = NodeTemplate->GetTraits();
			for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
			{
				const FTraitTemplate& TraitTemplate = TraitTemplates[TraitIndex];

				// This is how many latent properties are defined on the trait
				const uint32 NumTraitLatentProperties = TraitTemplate.GetNumLatentPropreties();

				// This is how many latent properties were hooked to something on that node
				uint32 NumSerializedLatentProperties = 0;
				*this << NumSerializedLatentProperties;

				// We allow traits to reserve more latent property handles than they might need
				check(NumSerializedLatentProperties <= NumTraitLatentProperties);

				const uint32 BaseTraitIndex = TraitIndex - TraitTemplate.GetTraitIndex();
				const FTraitTemplate& BaseTraitTemplate = TraitTemplates[BaseTraitIndex];

				// Latent header is looked up using the base trait
				FLatentPropertiesHeader& LatentHeader = BaseTraitTemplate.GetTraitLatentPropertiesHeader(*NodeDesc);
				FLatentPropertyHandle* LatentHandles = TraitTemplate.GetTraitLatentPropertyHandles(*NodeDesc);
				const FAnimNextTraitSharedData* SharedData = TraitTemplate.GetTraitDescription(*NodeDesc);

				// If we are a base trait, reset our header
				if (TraitTemplate.GetMode() == ETraitMode::Base)
				{
					LatentHeader.bHasValidLatentProperties = false;
					LatentHeader.bCanAllPropertiesFreeze = true;
					LatentHeader.bAreAllPropertiesVariableAccesses = true;
					LatentHeader.bAllLatentPropertiesOnBecomeRelevant = true;
				}

				// Default initialize our latent property handles
				for (uint32 LatentPropertyIndex = 0; LatentPropertyIndex < NumTraitLatentProperties; ++LatentPropertyIndex)
				{
					LatentHandles[LatentPropertyIndex] = FLatentPropertyHandle();
				}

				if (NumSerializedLatentProperties == 0)
				{
					continue;	// Nothing to do
				}

				bool bHasValidLatentProperties = !!LatentHeader.bHasValidLatentProperties;
				bool bCanAllPropertiesFreeze = !!LatentHeader.bCanAllPropertiesFreeze;
				bool bAreAllPropertiesVariableAccesses = !!LatentHeader.bAreAllPropertiesVariableAccesses;
				bool bAllLatentPropertiesOnBecomeRelevant = !!LatentHeader.bAllLatentPropertiesOnBecomeRelevant;

				const FTraitRegistryHandle TraitHandle = TraitTemplate.GetRegistryHandle();
				const FTrait* Trait = TraitRegistry.Find(TraitHandle);

				// Setup our latent property handles from the serialized data
				for (uint32 SerializedLatentPropertyIndex = 0; SerializedLatentPropertyIndex < NumSerializedLatentProperties; ++SerializedLatentPropertyIndex)
				{
					FLatentPropertyMetadata Metadata;
					*this << Metadata;

					if (Trait != nullptr)
					{
						const FTraitLatentPropertyMemoryLayout PropertyMemoryLayout = Trait->GetLatentPropertyMemoryLayout(*SharedData, Metadata.Name, SerializedLatentPropertyIndex);
						check(PropertyMemoryLayout.Size != 0);
						check(PropertyMemoryLayout.Alignment != 0 && FMath::IsPowerOfTwo(PropertyMemoryLayout.Alignment));
						check(PropertyMemoryLayout.LatentPropertyIndex != INDEX_NONE && (uint32)PropertyMemoryLayout.LatentPropertyIndex < NumTraitLatentProperties);

						uint16 RigVMIndex = MAX_uint16;
						uint32 CurrentLatentPropertyOffset = 0;
						bool bCanFreeze = true;
						bool bOnBecomeRelevant = true;

						// If this property is valid, setup our binding for it
						if (Metadata.RigVMIndex != MAX_uint16)
						{
							// Align our property
							LatentPropertyOffset = Align(LatentPropertyOffset, PropertyMemoryLayout.Alignment);

							RigVMIndex = Metadata.RigVMIndex;
							CurrentLatentPropertyOffset = LatentPropertyOffset;
							bCanFreeze = Metadata.bCanFreeze;
							bOnBecomeRelevant = Metadata.bOnBecomeRelevant;
							bAllLatentPropertiesOnBecomeRelevant &= Metadata.bOnBecomeRelevant;

							bHasValidLatentProperties = true;
							bCanAllPropertiesFreeze &= bCanFreeze;
							bAreAllPropertiesVariableAccesses &= Metadata.bUsesVariableCopy;

							// Consume the property size
							LatentPropertyOffset += PropertyMemoryLayout.Size;
						}

						// Latent property handles are in the order defined by the enumerator macro within the shared data,
						// which means their index can differ from the one we serialize with which uses the UStruct order
						LatentHandles[PropertyMemoryLayout.LatentPropertyIndex] = FLatentPropertyHandle(RigVMIndex, CurrentLatentPropertyOffset, bCanFreeze, bOnBecomeRelevant);
					}
				}

				LatentHeader.bHasValidLatentProperties = bHasValidLatentProperties;
				LatentHeader.bCanAllPropertiesFreeze = bCanAllPropertiesFreeze;
				LatentHeader.bAreAllPropertiesVariableAccesses = bAreAllPropertiesVariableAccesses;
				LatentHeader.bAllLatentPropertiesOnBecomeRelevant = bAllLatentPropertiesOnBecomeRelevant;
			}

			// Set our final node instance data size that factors in our latent properties
			NodeDesc->NodeInstanceDataSize = LatentPropertyOffset;
		}

		return EErrorState::None;
	}

	FArchive& FTraitReader::operator<<(UObject*& Obj)
	{
		// Load the object index
		int32 ObjectIndex = INDEX_NONE;
		*this << ObjectIndex;

		if (ensure(GraphReferencedObjects.IsValidIndex(ObjectIndex)))
		{
			Obj = GraphReferencedObjects[ObjectIndex];
		}
		else
		{
			// Something went wrong, the reference list must have gotten out of sync which shouldn't happen
			Obj = nullptr;
		}

		return *this;
	}

	FArchive& FTraitReader::operator<<(FObjectPtr& Obj)
	{
		return FArchiveUObject::SerializeObjectPtr(*this, Obj);
	}

	FArchive& FTraitReader::operator<<(FWeakObjectPtr& Value)
	{
		return FArchiveUObject::SerializeWeakObjectPtr(*this, Value);
	}

	FArchive& FTraitReader::operator<<(FSoftObjectPath& Value)
	{
		// Load the object index
		int32 SoftObjectIndex = INDEX_NONE;
		*this << SoftObjectIndex;

		if (ensure(GraphReferencedSoftObjects.IsValidIndex(SoftObjectIndex)))
		{
			Value = GraphReferencedSoftObjects[SoftObjectIndex];
		}
		else
		{
			// Something went wrong, the reference list must have gotten out of sync which shouldn't happen
			Value.Reset();
		}

		return *this;
	}

	FArchive& FTraitReader::operator<<(FSoftObjectPtr& Value)
	{
		// Load the object index
		int32 SoftObjectIndex = INDEX_NONE;
		*this << SoftObjectIndex;

		if (ensure(GraphReferencedSoftObjects.IsValidIndex(SoftObjectIndex)))
		{
			Value = FSoftObjectPtr(GraphReferencedSoftObjects[SoftObjectIndex]);
		}
		else
		{
			// Something went wrong, the reference list must have gotten out of sync which shouldn't happen
			Value.Reset();
		}

		return *this;
	}

	FNodeHandle FTraitReader::ResolveNodeHandle(FNodeHandle NodeHandle) const
	{
		if (!NodeHandle.IsValid())
		{
			// The node handle is invalid, return it unchanged
			return NodeHandle;
		}

		check(NodeHandle.IsNodeID());
		const FNodeID NodeID = NodeHandle.GetNodeID();
		check(NodeID.IsValid());

		return NodeHandles[NodeID.GetNodeIndex()];
	}

	FAnimNextTraitHandle FTraitReader::ResolveTraitHandle(FAnimNextTraitHandle TraitHandle) const
	{
		if (!TraitHandle.IsValid())
		{
			// The trait handle is invalid, return it unchanged
			return TraitHandle;
		}

		const FNodeHandle NodeHandle = ResolveNodeHandle(TraitHandle.GetNodeHandle());
		return FAnimNextTraitHandle(NodeHandle, TraitHandle.GetTraitIndex());
	}

	FAnimNextTraitHandle FTraitReader::ResolveEntryPointHandle(FAnimNextEntryPointHandle EntryPointHandle) const
	{
		if (!EntryPointHandle.IsValid())
		{
			// The trait handle is invalid, return an invalid handle
			return FAnimNextTraitHandle();
		}

		const FNodeHandle NodeHandle = ResolveNodeHandle(EntryPointHandle.GetNodeHandle());
		return FAnimNextTraitHandle(NodeHandle, EntryPointHandle.GetTraitIndex());
	}
}
