// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitRegistry.h"

#include "TraitCore/Trait.h"

namespace UE::UAF
{
	namespace Private
	{
		// The trait registry relies on static initialization to queue traits that attempt to register
		// before the module is ready. However, we need to store these pending requests in the meantime.
		// To do so safely, we return this through a function to ensure that it initializes before we
		// use it.
		struct FTraitRegistryStaticData
		{
			TArray<TraitConstructorFunc> PendingRegistrationQueue;
		};

		static FTraitRegistryStaticData& GetTraitRegistryStaticData()
		{
			static FTraitRegistryStaticData StaticData;
			return StaticData;
		}

		// Just a regular static to keep codegen clean as there is no race when accessing this during static
		// init. This pointer will be null until the module initializes (for the duration of static init) and
		// only returns to null when we shutdown the process and the module unloads.
		static FTraitRegistry* GTraitRegistry = nullptr;
	}

	FTraitRegistry& FTraitRegistry::Get()
	{
		checkf(Private::GTraitRegistry, TEXT("Trait Registry is not instanced. It is only valid to access this while the engine module is loaded."));
		return *Private::GTraitRegistry;
	}

	void FTraitRegistry::Init()
	{
		if (ensure(Private::GTraitRegistry == nullptr))
		{
			Private::GTraitRegistry = new FTraitRegistry();

			// Register all our pending static init traits
			Private::FTraitRegistryStaticData& StaticData = Private::GetTraitRegistryStaticData();
			for (TraitConstructorFunc TraitConstructor : StaticData.PendingRegistrationQueue)
			{
				Private::GTraitRegistry->AutoRegisterImpl(TraitConstructor);
			}

			// Reset the registration queue, it won't be used anymore now that the registry is up and ready
			StaticData.PendingRegistrationQueue.Empty(0);
		}
	}

	void FTraitRegistry::Destroy()
	{
		if (ensure(Private::GTraitRegistry != nullptr))
		{
			TArray<FRegistryEntry> Entries;
			Private::GTraitRegistry->TraitUIDToEntryMap.GenerateValueArray(Entries);

			for (const FRegistryEntry& Entry : Entries)
			{
				Private::GTraitRegistry->Unregister(Entry.Trait);
			}

			delete Private::GTraitRegistry;
			Private::GTraitRegistry = nullptr;
		}
	}

	void FTraitRegistry::StaticRegister(TraitConstructorFunc TraitConstructor)
	{
		if (Private::GTraitRegistry != nullptr)
		{
			// Registry is already up and running, use it
			Private::GTraitRegistry->AutoRegisterImpl(TraitConstructor);
		}
		else
		{
			// Registry isn't ready yet, queue up our trait
			// Once Init() is called, our queue will be processed
			Private::FTraitRegistryStaticData& StaticData = Private::GetTraitRegistryStaticData();
			StaticData.PendingRegistrationQueue.Add(TraitConstructor);
		}
	}

	void FTraitRegistry::StaticUnregister(TraitConstructorFunc TraitConstructor)
	{
		if (Private::GTraitRegistry != nullptr)
		{
			// Registry is already up and running, use it
			Private::GTraitRegistry->AutoUnregisterImpl(TraitConstructor);
		}
		else
		{
			// Registry isn't ready yet or it got destroyed before the traits are unregistering
			Private::FTraitRegistryStaticData& StaticData = Private::GetTraitRegistryStaticData();
			const int32 TraitIndex = StaticData.PendingRegistrationQueue.IndexOfByKey(TraitConstructor);
			if (TraitIndex != INDEX_NONE)
			{
				StaticData.PendingRegistrationQueue.RemoveAtSwap(TraitIndex);
			}
		}
	}

	void FTraitRegistry::AutoRegisterImpl(TraitConstructorFunc TraitConstructor)
	{
		// Grab the memory requirements of our trait
		FTraitMemoryLayout TraitMemoryRequirements;
		TraitConstructor(nullptr, TraitMemoryRequirements);

		// Align it and reserve space
		const uint32 OldBufferOffset = StaticTraitBufferOffset;

		uint8* TraitPtr = Align(&StaticTraitBuffer[OldBufferOffset], TraitMemoryRequirements.TraitAlignment);

		const uint32 NewBufferOffset = (TraitPtr + TraitMemoryRequirements.TraitSize) - &StaticTraitBuffer[0];
		const bool bFitsInStaticBuffer = NewBufferOffset <= STATIC_TRAIT_BUFFER_SIZE;

		FTraitRegistryHandle TraitHandle;
		if (bFitsInStaticBuffer)
		{
			// This trait fits in our buffer, add it
			StaticTraitBufferOffset = NewBufferOffset;
			TraitHandle = FTraitRegistryHandle::MakeStatic(TraitPtr - &StaticTraitBuffer[0]);
		}
		else
		{
			// We have too many static traits, we should consider increasing the static buffer
			// TODO: Warn
			// Allocate the trait on the heap instead
			TraitPtr = static_cast<uint8*>(FMemory::Malloc(TraitMemoryRequirements.TraitSize, TraitMemoryRequirements.TraitAlignment));
		}

		FTrait* Trait = TraitConstructor(TraitPtr, TraitMemoryRequirements);
		checkf((void*)TraitPtr == (void*)Trait, TEXT("FTrait 'this' should be where we specified"));

		const FTraitUID TraitUID = Trait->GetTraitUID();

		if (ensure(!TraitUIDToEntryMap.Contains(TraitUID.GetUID())) && ensure(!TraitNameToUIDMap.Contains(*Trait->GetTraitName())))
		{
			// This is a new trait, we'll keep it
			if (!bFitsInStaticBuffer)
			{
				// Find our dynamic trait index
				int32 TraitIndex;
				if (DynamicTraitFreeIndexHead != INDEX_NONE)
				{
					// We already had a free index, grab it
					TraitIndex = DynamicTraitFreeIndexHead;

					// Update the next free index
					DynamicTraitFreeIndexHead = DynamicTraits[DynamicTraitFreeIndexHead];

					// Set our trait ptr to our new index
					DynamicTraits[TraitIndex] = reinterpret_cast<intptr_t>(Trait);
				}
				else
				{
					// No free indices, allocate a new one
					TraitIndex = DynamicTraits.Add(reinterpret_cast<intptr_t>(Trait));
				}

				TraitHandle = FTraitRegistryHandle::MakeDynamic(TraitIndex);
			}

			TraitUIDToEntryMap.Add(TraitUID.GetUID(), FRegistryEntry{ Trait, TraitConstructor, TraitHandle });
			TraitNameToUIDMap.Add(*Trait->GetTraitName(), TraitUID.GetUID());
		}
		else
		{
			// We have already registered this trait, destroy our temporary instance
			Trait->~FTrait();

			if (bFitsInStaticBuffer)
			{
				// We were in the static buffer, clear our entry
				StaticTraitBufferOffset = OldBufferOffset;
				FMemory::Memzero(&StaticTraitBuffer[OldBufferOffset], NewBufferOffset - OldBufferOffset);
			}
			else
			{
				// It isn't in the static buffer, free it
				FMemory::Free(Trait);
				Trait = nullptr;
			}
		}
	}

	void FTraitRegistry::AutoUnregisterImpl(TraitConstructorFunc TraitConstructor)
	{
		for (auto It = TraitUIDToEntryMap.CreateIterator(); It; ++It)
		{
			FRegistryEntry& Entry = It.Value();

			if (Entry.TraitConstructor == TraitConstructor)
			{
				check(Entry.TraitHandle.IsValid());

				// Remove name from map before we destroy the Trait
				TraitNameToUIDMap.Remove(*Entry.Trait->GetTraitName());

				// Destroy and release our trait
				// We always own auto-registered trait instances
				Entry.Trait->~FTrait();

				if (Entry.TraitHandle.IsDynamic())
				{
					// It has been dynamically registered, free it
					const int32 TraitIndex = Entry.TraitHandle.GetDynamicIndex();

					DynamicTraits[TraitIndex] = DynamicTraitFreeIndexHead;
					DynamicTraitFreeIndexHead = TraitIndex;

					FMemory::Free(Entry.Trait);
				}
				else
				{
					// It was in the static buffer, we cannot reclaim the space easily, we'd have to add metadata to
					// track unused chunks so that we could coalesce
					if (TraitUIDToEntryMap.Num() == 1)
					{
						// Last static trait is being removed, reclaims all the space
						StaticTraitBufferOffset = 0;
					}
				}

				It.RemoveCurrent();

				break;
			}
		}
	}

	FTraitRegistryHandle FTraitRegistry::FindHandle(FTraitUID TraitUID) const
	{
		if (!TraitUID.IsValid())
		{
			return FTraitRegistryHandle();
		}

		if (const FRegistryEntry* Entry = TraitUIDToEntryMap.Find(TraitUID.GetUID()))
		{
			return Entry->TraitHandle;
		}

		// Trait not found
		return FTraitRegistryHandle();
	}

	const FTrait* FTraitRegistry::Find(FTraitRegistryHandle TraitHandle) const
	{
		if (!TraitHandle.IsValid())
		{
			return nullptr;
		}

		if (TraitHandle.IsStatic())
		{
			const int32 TraitOffset = TraitHandle.GetStaticOffset();
			return reinterpret_cast<const FTrait*>(&StaticTraitBuffer[TraitOffset]);
		}
		else
		{
			const int32 TraitIndex = TraitHandle.GetDynamicIndex();
			return reinterpret_cast<const FTrait*>(DynamicTraits[TraitIndex]);
		}
	}

	const FTrait* FTraitRegistry::Find(FTraitUID TraitUID) const
	{
		const FTraitRegistryHandle TraitHandle = FindHandle(TraitUID);
		return Find(TraitHandle);
	}

	const FTrait* FTraitRegistry::Find(const UScriptStruct* TraitSharedDataStruct) const
	{
		if (TraitSharedDataStruct == nullptr)
		{
			return nullptr;
		}

		for (const auto& KeyValuePair : TraitUIDToEntryMap)
		{
			const FTrait* Trait = KeyValuePair.Value.Trait;
			if (Trait->GetTraitSharedDataStruct() == TraitSharedDataStruct)
			{
				return Trait;
			}
		}

		return nullptr;
	}

	const FTrait* FTraitRegistry::Find(FName TraitTypeName) const
	{
		if (TraitTypeName == NAME_None)
		{
			return nullptr;
		}

		const FTraitUIDRaw* TraitGUID = TraitNameToUIDMap.Find(TraitTypeName);
		if (TraitGUID != nullptr && FTraitUID(*TraitGUID).IsValid())
		{
			if (const FRegistryEntry* Entry = TraitUIDToEntryMap.Find(*TraitGUID))
			{
				return Entry->Trait;
			}
		}

		return nullptr;
	}

	void FTraitRegistry::Register(FTrait* Trait)
	{
		if (Trait == nullptr)
		{
			return;
		}

		const FTraitUID TraitUID = Trait->GetTraitUID();

		if (ensure(!TraitUIDToEntryMap.Contains(TraitUID.GetUID())) && ensure(!TraitNameToUIDMap.Contains(*Trait->GetTraitName())))
		{
			// This is a new trait, we'll keep it
			// Find our dynamic trait index
			int32 TraitIndex;
			if (DynamicTraitFreeIndexHead != INDEX_NONE)
			{
				// We already had a free index, grab it
				TraitIndex = DynamicTraitFreeIndexHead;

				// Update the next free index
				DynamicTraitFreeIndexHead = DynamicTraits[DynamicTraitFreeIndexHead];

				// Set our trait ptr to our new index
				DynamicTraits[TraitIndex] = reinterpret_cast<intptr_t>(Trait);
			}
			else
			{
				// No free indices, allocate a new one
				TraitIndex = DynamicTraits.Add(reinterpret_cast<intptr_t>(Trait));
			}

			FTraitRegistryHandle TraitHandle = FTraitRegistryHandle::MakeDynamic(TraitIndex);

			TraitUIDToEntryMap.Add(TraitUID.GetUID(), FRegistryEntry{ Trait, nullptr, TraitHandle });
			TraitNameToUIDMap.Add(*Trait->GetTraitName(), TraitUID.GetUID());
		}
	}

	void FTraitRegistry::Unregister(FTrait* Trait)
	{
		if (Trait == nullptr)
		{
			return;
		}

		const FTraitUID TraitUID = Trait->GetTraitUID();

		if (FRegistryEntry* Entry = TraitUIDToEntryMap.Find(TraitUID.GetUID()))
		{
			check(Entry->TraitHandle.IsValid());

			// Remove name from map before we destroy the Trait
			TraitNameToUIDMap.Remove(*Entry->Trait->GetTraitName());

			if (Entry->TraitHandle.IsDynamic())
			{
				// It has been dynamically registered, free it
				const int32 TraitIndex = Entry->TraitHandle.GetDynamicIndex();

				DynamicTraits[TraitIndex] = DynamicTraitFreeIndexHead;
				DynamicTraitFreeIndexHead = TraitIndex;
			}

			if (Entry->TraitConstructor != nullptr)
			{
				// We own this trait instance, destroy it
				Entry->Trait->~FTrait();

				if (Entry->TraitHandle.IsDynamic())
				{
					FMemory::Free(Entry->Trait);
				}
				else
				{
					// It was in the static buffer, we cannot reclaim the space
				}
			}

			TraitUIDToEntryMap.Remove(TraitUID.GetUID());
		}
	}

	TArray<const FTrait*> FTraitRegistry::GetTraits() const
	{
		TArray<const FTrait*> Traits;
		Traits.Reserve(TraitUIDToEntryMap.Num());
		
		for (const auto& It : TraitUIDToEntryMap)
		{
			Traits.Add(It.Value.Trait);
		}

		return Traits;
	}

	uint32 FTraitRegistry::GetNum() const
	{
		return TraitUIDToEntryMap.Num();
	}
}
