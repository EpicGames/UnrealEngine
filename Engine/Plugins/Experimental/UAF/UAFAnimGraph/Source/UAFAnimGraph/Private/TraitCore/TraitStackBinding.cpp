// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitStackBinding.h"

#include "Graph/AnimNextGraphInstance.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"

namespace UE::UAF
{
	FTraitStackBinding::FTraitStackBinding(const FExecutionContext& InContext, const FWeakTraitPtr& TraitPtr)
		: Context(&InContext)
	{
		check(TraitPtr.IsValid());

		NodeInstance = TraitPtr.GetNodeInstance();

		check(InContext.IsBoundTo(NodeInstance->GetOwner()));

		NodeDescription = &InContext.GetNodeDescription(*NodeInstance);

		NodeTemplate = InContext.GetNodeTemplate(*NodeDescription);
		if (!ensure(NodeTemplate != nullptr))
		{
			// Node template wasn't found, node descriptor is perhaps corrupted
			Reset();
			return;
		}

		if (!ensure(TraitPtr.GetTraitIndex() < NodeTemplate->GetNumTraits()))
		{
			// The requested trait index doesn't exist on that node descriptor
			Reset();
			return;
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();

		// We only search within the partial stack of the provided trait
		const FTraitTemplate* CurrentTraitDesc = TraitDescs + TraitPtr.GetTraitIndex();
		const FTraitTemplate* BaseTraitDesc = CurrentTraitDesc - CurrentTraitDesc->GetTraitIndex();

		BaseTraitIndex = BaseTraitDesc - TraitDescs;
		TopTraitIndex = BaseTraitIndex + BaseTraitDesc->GetNumStackTraits() - 1;
	}

	void FTraitStackBinding::Reset()
	{
		new (this) FTraitStackBinding();
	}

	bool FTraitStackBinding::operator==(const FTraitStackBinding& Other) const
	{
		return NodeInstance == Other.NodeInstance && BaseTraitIndex == Other.BaseTraitIndex;
	}

	bool FTraitStackBinding::GetTopTrait(FTraitBinding& OutBinding) const
	{
		if (!IsValid())
		{
			OutBinding.Reset();
			return false;
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		uint32 TraitIndex = TopTraitIndex;

		// Skip invalid trait templates
		while (!TraitDescs[TraitIndex].IsValid() && TraitIndex != BaseTraitIndex)
		{
			TraitIndex--;
		}

		if (!ensure(TraitDescs[TraitIndex].IsValid()))
		{
			OutBinding.Reset();
			return false;	// No traits were valid on this stack but we should always at least have a valid base trait
		}

		OutBinding = FTraitBinding(this, Context->GetTrait(TraitDescs[TraitIndex]), TraitIndex);
		return true;
	}

	bool FTraitStackBinding::GetParentTrait(const FTraitBinding& ChildBinding, FTraitBinding& OutParentBinding) const
	{
		if (!IsValid())
		{
			OutParentBinding.Reset();
			return false;
		}

		if (ChildBinding.TraitIndex == BaseTraitIndex)
		{
			OutParentBinding.Reset();
			return false;	// No more parents, reached the base of the stack
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		uint32 ParentTraitIndex = ChildBinding.TraitIndex - 1;

		// Skip invalid trait templates
		while (!TraitDescs[ParentTraitIndex].IsValid() && ParentTraitIndex != BaseTraitIndex)
		{
			ParentTraitIndex--;
		}

		if (!ensure(TraitDescs[ParentTraitIndex].IsValid()))
		{
			OutParentBinding.Reset();
			return false;	// No parent trait was valid on this stack but we should always at least have a valid base trait
		}

		OutParentBinding = FTraitBinding(this, Context->GetTrait(TraitDescs[ParentTraitIndex]), ParentTraitIndex);
		return true;
	}

	bool FTraitStackBinding::GetBaseTrait(FTraitBinding& OutBinding) const
	{
		if (!IsValid())
		{
			OutBinding.Reset();
			return false;
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();

		if (!ensure(TraitDescs[BaseTraitIndex].IsValid()))
		{
			OutBinding.Reset();
			return false;	// We should always have a valid base trait
		}

		OutBinding = FTraitBinding(this, Context->GetTrait(TraitDescs[BaseTraitIndex]), BaseTraitIndex);
		return true;
	}

	bool FTraitStackBinding::GetChildTrait(const FTraitBinding& ParentBinding, FTraitBinding& OutChildBinding) const
	{
		if (!IsValid())
		{
			OutChildBinding.Reset();
			return false;
		}

		if (ParentBinding.TraitIndex == TopTraitIndex)
		{
			OutChildBinding.Reset();
			return false;	// No more children, reached the top of the stack
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		uint32 ChildTraitIndex = ParentBinding.TraitIndex + 1;

		// Skip invalid trait templates
		while (!TraitDescs[ChildTraitIndex].IsValid() && ChildTraitIndex != TopTraitIndex)
		{
			ChildTraitIndex++;
		}

		if (!TraitDescs[ChildTraitIndex].IsValid())
		{
			OutChildBinding.Reset();
			return false;	// We couldn't find a valid child
		}

		OutChildBinding = FTraitBinding(this, Context->GetTrait(TraitDescs[ChildTraitIndex]), ChildTraitIndex);
		return true;
	}

	bool FTraitStackBinding::GetTrait(uint32 TraitIndex, FTraitBinding& OutBinding) const
	{
		if (!IsValid())
		{
			OutBinding.Reset();
			return false;
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		if (TraitIndex >= TraitDescs[BaseTraitIndex].GetNumStackTraits())
		{
			OutBinding.Reset();
			return false;	// Invalid trait index
		}

		const uint32 StackTraitIndex = BaseTraitIndex + TraitIndex;
		if (!TraitDescs[StackTraitIndex].IsValid())
		{
			OutBinding.Reset();
			return false;
		}

		OutBinding = FTraitBinding(this, Context->GetTrait(TraitDescs[StackTraitIndex]), StackTraitIndex);
		return true;
	}

	uint32 FTraitStackBinding::GetNumTraits() const
	{
		if (!IsValid())
		{
			return 0;
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		return TraitDescs[BaseTraitIndex].GetNumStackTraits();
	}

	bool FTraitStackBinding::GetInterfaceImpl(FTraitInterfaceUID InterfaceUID, FTraitBinding& OutBinding) const
	{
		if (!IsValid())
		{
			OutBinding.Reset();
			return false;
		}

		// Start searching with the top trait towards our base trait
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs + TopTraitIndex;
		const FTraitTemplate* EndDesc = TraitDescs + BaseTraitIndex - 1;

		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; --TraitDesc)
		{
			const FTrait* Trait = Context->GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			if (const ITraitInterface* Interface = Trait->GetTraitInterface(InterfaceUID))
			{
				const uint32 TraitIndex = TraitDesc - TraitDescs;
				const uint32 InterfaceThisOffset = reinterpret_cast<const uint8*>(Interface) - reinterpret_cast<const uint8*>(Trait);

				OutBinding = FTraitBinding(this, Trait, TraitIndex, InterfaceThisOffset);
				return true;
			}
		}

		// We failed to find the specified interface on the trait stack
		OutBinding.Reset();
		return false;
	}

	bool FTraitStackBinding::GetInterfaceSuperImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding, FTraitBinding& OutSuperBinding) const
	{
		if (!IsValid())
		{
			OutSuperBinding.Reset();
			return false;
		}

		const uint32 CurrentTraitIndex = Binding.TraitIndex;
		if (CurrentTraitIndex == BaseTraitIndex)
		{
			OutSuperBinding.Reset();
			return false;	// We reached the base of the stack, we don't have a super
		}

		// Start searching with the current trait towards our base trait
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs + CurrentTraitIndex - 1;
		const FTraitTemplate* EndDesc = TraitDescs + BaseTraitIndex - 1;

		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; --TraitDesc)
		{
			const FTrait* Trait = Context->GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			if (const ITraitInterface* Interface = Trait->GetTraitInterface(InterfaceUID))
			{
				const uint32 SuperTraitIndex = TraitDesc - TraitDescs;
				const uint32 InterfaceThisOffset = reinterpret_cast<const uint8*>(Interface) - reinterpret_cast<const uint8*>(Trait);

				OutSuperBinding = FTraitBinding(this, Trait, SuperTraitIndex, InterfaceThisOffset);
				return true;
			}
		}

		// We failed to find the specified interface on the trait stack
		OutSuperBinding.Reset();
		return false;
	}

	bool FTraitStackBinding::HasInterfaceImpl(FTraitInterfaceUID InterfaceUID) const
	{
		if (!IsValid())
		{
			return false;
		}

		// Start searching with the top trait towards our base trait
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs + TopTraitIndex;
		const FTraitTemplate* EndDesc = TraitDescs + BaseTraitIndex - 1;

		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; --TraitDesc)
		{
			const FTrait* Trait = Context->GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			if (Trait->GetTraitInterface(InterfaceUID) != nullptr)
			{
				return true;
			}
		}

		// We failed to find the specified interface on the trait stack
		return false;
	}

	bool FTraitStackBinding::HasInterfaceSuperImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding) const
	{
		if (!IsValid())
		{
			return false;
		}

		const uint32 CurrentTraitIndex = Binding.TraitIndex;
		if (CurrentTraitIndex == BaseTraitIndex)
		{
			return false;	// We reached the base of the stack, we don't have a super
		}

		// Start searching with the current trait towards our base trait
		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* StartDesc = TraitDescs + CurrentTraitIndex - 1;
		const FTraitTemplate* EndDesc = TraitDescs + BaseTraitIndex - 1;

		for (const FTraitTemplate* TraitDesc = StartDesc; TraitDesc != EndDesc; --TraitDesc)
		{
			const FTrait* Trait = Context->GetTrait(*TraitDesc);
			if (Trait == nullptr)
			{
				continue;	// Trait hasn't been loaded or registered, skip it
			}

			if (Trait->GetTraitInterface(InterfaceUID) != nullptr)
			{
				return true;
			}
		}

		// We failed to find the specified interface on the trait stack
		return false;
	}

	void FTraitStackBinding::SnapshotLatentProperties(float DeltaTime, bool bIsFrozen, bool bJustBecameRelevant)
	{
		if (!IsValid())
		{
			return;	// Nothing to do
		}

		const FTraitTemplate* TraitDescs = NodeTemplate->GetTraits();
		const FTraitTemplate* BaseTraitDesc = TraitDescs + BaseTraitIndex;

		const FLatentPropertiesHeader& LatentHeader = BaseTraitDesc->GetTraitLatentPropertiesHeader(*NodeDescription);
		if (!LatentHeader.bHasValidLatentProperties)
		{
			return;	// All latent properties are inline, nothing to snapshot
		}
		else if ((bIsFrozen && LatentHeader.bCanAllPropertiesFreeze) || (!bJustBecameRelevant && LatentHeader.bAllLatentPropertiesOnBecomeRelevant))
		{
			return;	// We are frozen and all latent properties support freezing, nothing to snapshot
		}

		const FLatentPropertyHandle* LatentHandles = BaseTraitDesc->GetTraitLatentPropertyHandles(*NodeDescription);
		const uint32 NumLatentHandles = BaseTraitDesc->GetNumSubStackLatentPropreties();

		FAnimNextGraphInstance& GraphInstance = NodeInstance->GetOwner();

		if (LatentHeader.bAreAllPropertiesVariableAccesses)
		{
			GraphInstance.CopyVariablesToLatentPins(TConstArrayView<FLatentPropertyHandle>(LatentHandles, NumLatentHandles), NodeInstance, bIsFrozen, bJustBecameRelevant);
		}
		else
		{
			GraphInstance.ExecuteLatentPins(TConstArrayView<FLatentPropertyHandle>(LatentHandles, NumLatentHandles), NodeInstance, DeltaTime, bIsFrozen, bJustBecameRelevant);
		}
	}
}
