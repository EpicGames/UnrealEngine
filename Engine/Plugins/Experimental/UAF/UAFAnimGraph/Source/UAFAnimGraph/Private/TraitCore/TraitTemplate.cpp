// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitTemplate.h"

#include "Serialization/Archive.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"

namespace UE::UAF
{
	void FTraitTemplate::Serialize(FArchive& Ar)
	{
		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

		Ar << UID;

		if (Ar.IsSaving())
		{
			const FTrait* Trait = TraitRegistry.Find(RegistryHandle);

			uint32 TraitUID = Trait->GetTraitUID().GetUID();
			Ar << TraitUID;
		}
		else if (Ar.IsLoading())
		{
			uint32 TraitUID = 0;
			Ar << TraitUID;

			// It is possible that we fail to find the trait that we need
			// This can happen if the trait hasn't been loaded or registered
			// When this happens, the trait is a no-op and the runtime behavior
			// may not be what is expected
			RegistryHandle = TraitRegistry.FindHandle(FTraitUID(TraitUID));
		}
		else
		{
			// Counting, etc
			if (RegistryHandle.IsDynamic())
			{
				int32 DynamicIndex = RegistryHandle.GetDynamicIndex();
				Ar << DynamicIndex;
			}
			else if (RegistryHandle.IsStatic())
			{
				int32 StaticOffset = RegistryHandle.GetStaticOffset();
				Ar << StaticOffset;
			}
		}

		uint8 ModeByte = Mode;
		Ar << ModeByte;

		if (Ar.IsLoading())
		{
			Mode = ModeByte;
		}
		
		uint8 GCAwareByte = bGCAware;
		Ar << GCAwareByte;

		if (Ar.IsLoading())
		{
			bGCAware = GCAwareByte;
		}
		
		Ar << TraitIndexOrNumTraits;
	}
}
