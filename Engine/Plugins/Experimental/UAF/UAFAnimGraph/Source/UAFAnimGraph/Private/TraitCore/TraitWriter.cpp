// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitWriter.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitTemplate.h"
#include "TraitCore/NodeDescription.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "Serialization/ArchiveUObject.h"
#include "StructUtils/StructView.h"

#include "TraitCore/TraitReader.h"

namespace UE::UAF
{
	FTraitWriter::FTraitWriter()
		: FMemoryWriter(GraphSharedDataArchiveBuffer)
		, NextNodeID(FNodeID::GetFirstID())
		, NumNodesWritten(0)
		, bIsNodeWriting(false)
		, ErrorState(EErrorState::None)
	{
	}

	FNodeHandle FTraitWriter::RegisterNode(const FNodeTemplate& NodeTemplate)
	{
		ensure(!bIsNodeWriting);

		if (ErrorState != EErrorState::None)
		{
			// We encountered an error, do nothing
			return FNodeHandle();
		}

		if (NodeTemplate.GetNodeTemplateSize() > FNodeTemplate::MAXIMUM_SIZE)
		{
			// This node template is too large
			ErrorState = EErrorState::NodeTemplateTooLarge;
			return FNodeHandle();
		}

		if (!NextNodeID.IsValid())
		{
			// We have too many nodes in the graph, we need to be able to represent them with 16 bits
			// The node ID must have wrapped around
			ErrorState = EErrorState::TooManyNodes;
			return FNodeHandle();
		}

		const FNodeHandle NodeHandle = FNodeHandle::FromNodeID(NextNodeID);
		check(NodeHandle.IsValid() && NodeHandle.IsNodeID());
		check(NodeMappings.Num() == NodeHandle.GetNodeID().GetNodeIndex());

		NextNodeID = NextNodeID.GetNextID();

		FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();
		const FNodeTemplateRegistryHandle NodeTemplateHandle = NodeTemplateRegistry.FindOrAdd(&NodeTemplate);

		NodeMappings.Add({ NodeHandle, NodeTemplateHandle, 0 });

		return NodeHandle;
	}

	void FTraitWriter::BeginNodeWriting()
	{
		ensure(!bIsNodeWriting);
		ensure(NumNodesWritten == 0);

		if (ErrorState != EErrorState::None)
		{
			// We encountered an error, do nothing
			return;
		}

		bIsNodeWriting = true;
		GraphReferencedObjects.Reset();
		GraphReferencedSoftObjects.Reset();

		// Serialize the node templates
		TArray<FNodeTemplateRegistryHandle> NodeTemplateHandles;
		NodeTemplateHandles.Reserve(NodeMappings.Num());

		for (FNodeMapping& NodeMapping : NodeMappings)
		{
			NodeMapping.NodeTemplateIndex = NodeTemplateHandles.AddUnique(NodeMapping.NodeTemplateHandle);
		}

		uint32 NumNodeTemplates = NodeTemplateHandles.Num();
		*this << NumNodeTemplates;

		FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();
		for (FNodeTemplateRegistryHandle NodeTemplateHandle : NodeTemplateHandles)
		{
			FNodeTemplate* NodeTemplate = NodeTemplateRegistry.FindMutable(NodeTemplateHandle);
			NodeTemplate->Serialize(*this);
		}

		// Begin serializing the graph shared data
		uint32 NumNodes = NodeMappings.Num();
		*this << NumNodes;

		// Serialize the node template indices that we'll use for each node
		for (const FNodeMapping& NodeMapping : NodeMappings)
		{
			uint32 NodeTemplateIndex = NodeMapping.NodeTemplateIndex;
			*this << NodeTemplateIndex;
		}
	}

	void FTraitWriter::EndNodeWriting()
	{
		ensure(bIsNodeWriting);
		bIsNodeWriting = false;

		if (ErrorState != EErrorState::None)
		{
			// We encountered an error, do nothing
			return;
		}

		ensure(NumNodesWritten == NodeMappings.Num());
	}

#if WITH_EDITOR
	void FTraitWriter::WriteNode(
		const FNodeHandle NodeHandle,
		const TFunction<FString (uint32 TraitIndex, FName PropertyName)>& GetTraitProperty,
		const TFunction<uint16(uint32 TraitIndex, FName PropertyName)>& GetTraitLatentPropertyIndex)
	{
		ensure(bIsNodeWriting);

		if (ErrorState != EErrorState::None)
		{
			// We encountered an error, do nothing
			return;
		}

		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
		FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();

		const FNodeMapping* NodeMapping = NodeMappings.FindByPredicate([NodeHandle](const FNodeMapping& It) { return It.NodeHandle == NodeHandle; });
		if (NodeMapping == nullptr)
		{
			ErrorState = EErrorState::NodeHandleNotFound;
			return;
		}

		const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(NodeMapping->NodeTemplateHandle);
		if (NodeTemplate == nullptr)
		{
			ErrorState = EErrorState::NodeTemplateNotFound;
			return;
		}

		// Populate our node description into a temporary buffer
		alignas(16) uint8 Buffer[64 * 1024];	// Max node size

		FNodeDescription* NodeDesc = new(Buffer) FNodeDescription(NodeMapping->NodeHandle.GetNodeID(), NodeMapping->NodeTemplateHandle);

		// Populate our trait properties
		const uint32 NumTraits = NodeTemplate->GetNumTraits();
		const FTraitTemplate* TraitTemplates = NodeTemplate->GetTraits();
		for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FTraitRegistryHandle TraitHandle = TraitTemplates[TraitIndex].GetRegistryHandle();
			const FTrait* Trait = TraitRegistry.Find(TraitHandle);

			FAnimNextTraitSharedData* SharedData = TraitTemplates[TraitIndex].GetTraitDescription(*NodeDesc);

			// Curry our lambda with the trait index
			const auto GetTraitPropertyAt = [&GetTraitProperty, TraitIndex](FName PropertyName)
			{
				return GetTraitProperty(TraitIndex, PropertyName);
			};

			Trait->SaveTraitSharedData(GetTraitPropertyAt, *SharedData);
		}

		// Append our node and trait shared data to our archive
		NodeDesc->Serialize(*this);

		// Append our trait latent property handles to our archive
		// We only write out the properties that will be present at runtime
		// This takes into account editor only latent properties which can be stripped in cooked builds
		// Other forms of property stripping are not currently supported
		// The latent property offsets will be computed at runtime on load to support property sizes/alignment
		// changing between the editor and the runtime platform (e.g. 32 vs 64 bit pointers)
		// To that end, we serialize the following property metadata:
		//     * RigVM memory handle index
		//     * Whether the property supports freezing or not
		//     * The property name and index for us to look it up at runtime

		for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FTraitRegistryHandle TraitHandle = TraitTemplates[TraitIndex].GetRegistryHandle();
			const FTrait* Trait = TraitRegistry.Find(TraitHandle);

			// Curry our lambda with the trait index
			const auto GetTraitLatentPropertyIndexAt = [&GetTraitLatentPropertyIndex, TraitIndex](FName PropertyName)
			{
				return GetTraitLatentPropertyIndex(TraitIndex, PropertyName);
			};

			TArray<FLatentPropertyMetadata> LatentProperties;
			LatentProperties.Reserve(32);
			FAnimNextTraitSharedData* SharedData = TraitTemplates[TraitIndex].GetTraitDescription(*NodeDesc);
			uint32 NumLatentProperties = Trait->GetLatentPropertyHandles(SharedData, LatentProperties, IsFilterEditorOnly(), GetTraitLatentPropertyIndexAt);

			*this << NumLatentProperties;

			for (FLatentPropertyMetadata& Metadata : LatentProperties)
			{
				*this << Metadata;
			}
		}

		NumNodesWritten++;
	}
#endif

	void FTraitWriter::WriteNode(
		const FNodeHandle NodeHandle,
		TFunctionRef<uint16(uint32 TraitIndex, FName PropertyName)> GetTraitVariableMappingIndex,
		TFunctionRef<TConstStructView<FAnimNextTraitSharedData>(uint32 TraitIndex)> GetTraitData)
	{
		ensure(bIsNodeWriting);

		if (ErrorState != EErrorState::None)
		{
			// We encountered an error, do nothing
			return;
		}

		const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
		FNodeTemplateRegistry& NodeTemplateRegistry = FNodeTemplateRegistry::Get();

		const FNodeMapping* NodeMapping = NodeMappings.FindByPredicate([NodeHandle](const FNodeMapping& It) { return It.NodeHandle == NodeHandle; });
		if (NodeMapping == nullptr)
		{
			ErrorState = EErrorState::NodeHandleNotFound;
			return;
		}

		const FNodeTemplate* NodeTemplate = NodeTemplateRegistry.Find(NodeMapping->NodeTemplateHandle);
		if (NodeTemplate == nullptr)
		{
			ErrorState = EErrorState::NodeTemplateNotFound;
			return;
		}

		// Populate our node description into a temporary buffer
		alignas(16) uint8 Buffer[64 * 1024];	// Max node size

		FNodeDescription* NodeDesc = new(Buffer) FNodeDescription(NodeMapping->NodeHandle.GetNodeID(), NodeMapping->NodeTemplateHandle);

		// Populate our trait properties
		const uint32 NumTraits = NodeTemplate->GetNumTraits();
		const FTraitTemplate* TraitTemplates = NodeTemplate->GetTraits();
		for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FTraitRegistryHandle TraitHandle = TraitTemplates[TraitIndex].GetRegistryHandle();
			const FTrait* Trait = TraitRegistry.Find(TraitHandle);

			FAnimNextTraitSharedData* SharedData = TraitTemplates[TraitIndex].GetTraitDescription(*NodeDesc);
			const UScriptStruct* SharedDataStruct = Trait->GetTraitSharedDataStruct();

			// Initialize our output struct
			TConstStructView<FAnimNextTraitSharedData> SourceData = GetTraitData(TraitIndex);
			check(SourceData.GetScriptStruct() == SharedDataStruct);
			SharedDataStruct->InitializeDefaultValue((uint8*)SharedData);
			SharedDataStruct->CopyScriptStruct(SharedData, SourceData.GetMemory());
		}

		// Append our node and trait shared data to our archive
		NodeDesc->Serialize(*this);

		// Append our trait latent property handles to our archive
		// We only write out the properties that will be present at runtime
		// This takes into account editor only latent properties which can be stripped in cooked builds
		// Other forms of property stripping are not currently supported
		// The latent property offsets will be computed at runtime on load to support property sizes/alignment
		// changing between the editor and the runtime platform (e.g. 32 vs 64 bit pointers)
		// To that end, we serialize the following property metadata:
		//     * RigVM memory handle index
		//     * Whether the property supports freezing or not
		//     * The property name and index for us to look it up at runtime

		for (uint32 TraitIndex = 0; TraitIndex < NumTraits; ++TraitIndex)
		{
			const FTraitRegistryHandle TraitHandle = TraitTemplates[TraitIndex].GetRegistryHandle();
			const FTrait* Trait = TraitRegistry.Find(TraitHandle);

			// Curry our lambda with the trait index
			const auto GetTraitVariableMappingIndexAt = [&GetTraitVariableMappingIndex, TraitIndex](FName PropertyName)
			{
				return GetTraitVariableMappingIndex(TraitIndex, PropertyName);
			};

			TArray<FLatentPropertyMetadata> LatentProperties;
			LatentProperties.Reserve(32);
			FAnimNextTraitSharedData* SharedData = TraitTemplates[TraitIndex].GetTraitDescription(*NodeDesc);
			uint32 NumLatentProperties = Trait->GetVariableMappedLatentPropertyHandles(SharedData, LatentProperties, IsFilterEditorOnly(), GetTraitVariableMappingIndexAt);

			*this << NumLatentProperties;

			for (FLatentPropertyMetadata& Metadata : LatentProperties)
			{
				*this << Metadata;
			}
		}

		NumNodesWritten++;
	}

	FTraitWriter::EErrorState FTraitWriter::GetErrorState() const
	{
		return ErrorState;
	}

	const TArray<uint8>& FTraitWriter::GetGraphSharedData() const
	{
		return GraphSharedDataArchiveBuffer;
	}

	const TArray<UObject*>& FTraitWriter::GetGraphReferencedObjects() const
	{
		return GraphReferencedObjects;
	}

	const TArray<FSoftObjectPath>& FTraitWriter::GetGraphReferencedSoftObjects() const
	{
		return GraphReferencedSoftObjects;
	}

	FArchive& FTraitWriter::operator<<(UObject*& Obj)
	{
		// Add our object for tracking
		int32 ObjectIndex = GraphReferencedObjects.AddUnique(Obj);

		// Save our index, we'll use it to resolve the object on load
		*this << ObjectIndex;

		return *this;
	}

	FArchive& FTraitWriter::operator<<(FObjectPtr& Obj)
	{
		return FArchiveUObject::SerializeObjectPtr(*this, Obj);
	}

	FArchive& FTraitWriter::operator<<(FWeakObjectPtr& Value)
	{
		return FArchiveUObject::SerializeWeakObjectPtr(*this, Value);
	}

	FArchive& FTraitWriter::operator<<(FSoftObjectPath& Value)
	{
		// Add our object for tracking
		int32 SoftObjectIndex = GraphReferencedSoftObjects.AddUnique(Value);

		// Save our index, we'll use it to resolve the soft object on load
		*this << SoftObjectIndex;

		return *this;
	}

	FArchive& FTraitWriter::operator<<(FSoftObjectPtr& Value)
	{
		// Add our object for tracking
		int32 SoftObjectIndex = GraphReferencedSoftObjects.AddUnique(Value.ToSoftObjectPath());

		// Save our index, we'll use it to resolve the soft object on load
		*this << SoftObjectIndex;

		return *this;
	}
}

