// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "TraitCore/ExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAlphaInputArgs)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IAlphaInputArgs)

	FAlphaInputTraitArgs IAlphaInputArgs::Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
	{
		TTraitBinding<IAlphaInputArgs> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.Get(Context);
		}

		return FAlphaInputTraitArgs();
	}

	EAnimAlphaInputType IAlphaInputArgs::GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
	{
		TTraitBinding<IAlphaInputArgs> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetAlphaInputType(Context);
		}

		return EAnimAlphaInputType::Float;
	}

	FName IAlphaInputArgs::GetAlphaCurveName(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
	{
		TTraitBinding<IAlphaInputArgs> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetAlphaCurveName(Context);
		}

		return NAME_None;
	}

	float IAlphaInputArgs::GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
	{
		TTraitBinding<IAlphaInputArgs> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetCurrentAlphaValue(Context);
		}

		return 1.0f;
	}

	TFunction<float(float)> IAlphaInputArgs::GetInputScaleBiasClampCallback(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
	{
		TTraitBinding<IAlphaInputArgs> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetInputScaleBiasClampCallback(Context);
		}

		auto InputBiasClampCallback = [](float Alpha) -> float
		{
			return 1.0f;
		};

		return InputBiasClampCallback;
	}

#if WITH_EDITOR
	const FText& IAlphaInputArgs::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_AlphaInputArg_Name", "Alpha Input Args");
		return InterfaceName;
	}
	const FText& IAlphaInputArgs::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_AlphaInputArg_ShortName", "AIA");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR
}
