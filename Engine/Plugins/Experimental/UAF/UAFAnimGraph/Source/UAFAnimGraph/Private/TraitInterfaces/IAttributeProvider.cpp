// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IAttributeProvider.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IAttributeProvider)

#if WITH_EDITOR
	const FText& IAttributeProvider::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IAttributeProvider_Name", "AttributeProvider");
		return InterfaceName;
	}
	const FText& IAttributeProvider::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IAttributeProvider_ShortName", "ATP");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	FOnExtractRootMotionAttribute IAttributeProvider::GetOnExtractRootMotionAttribute(FExecutionContext& Context, const TTraitBinding<IAttributeProvider>& Binding) const
	{
		TTraitBinding<IAttributeProvider> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetOnExtractRootMotionAttribute(Context);
		}

		return FOnExtractRootMotionAttribute();
	}
}
