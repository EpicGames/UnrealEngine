// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IBlendStack.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IBlendStack)

#if WITH_EDITOR
	const FText& IBlendStack::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IBlendStack_Name", "Blend Stack");
		return InterfaceName;
	}
	const FText& IBlendStack::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IBlendStack_ShortName", "BST");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	int32 IBlendStack::PushGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequest&& GraphRequest) const
	{
		TTraitBinding<IBlendStack> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.PushGraph(Context, MoveTemp(GraphRequest));
		}
		return INDEX_NONE;
	}

	int32 IBlendStack::GetActiveGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequestPtr& OutGraphRequest) const
	{
		TTraitBinding<IBlendStack> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetActiveGraph(Context, OutGraphRequest);
		}

		OutGraphRequest = nullptr;
		return INDEX_NONE;
	}

	IBlendStack::FGraphRequestPtr IBlendStack::GetGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const
	{
		TTraitBinding<IBlendStack> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetGraph(Context, InChildIndex);
		}

		return nullptr;
	}

	FAnimNextGraphInstance* IBlendStack::GetGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const
	{
		TTraitBinding<IBlendStack> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetGraphInstance(Context, InChildIndex);
		}

		return nullptr;
	}

	int32 IBlendStack::GetActiveGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, FAnimNextGraphInstance*& OutGraphInstance) const
	{
		TTraitBinding<IBlendStack> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetActiveGraphInstance(Context, OutGraphInstance);
		}

		OutGraphInstance = nullptr;
		return INDEX_NONE;
	}
}
