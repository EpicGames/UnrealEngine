// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IContinuousBlend.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IContinuousBlend)

#if WITH_EDITOR
	const FText& IContinuousBlend::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ContinuousBlend_Name", "Continuous Blend");
		return InterfaceName;
	}  
	const FText& IContinuousBlend::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ContinuousBlend_ShortName", "CNB");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	float IContinuousBlend::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IContinuousBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendWeight(Context, ChildIndex);
		}

		return -1.0f;
	}
}
