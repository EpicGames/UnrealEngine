// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IDiscreteBlend.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IDiscreteBlend)

#if WITH_EDITOR
	const FText& IDiscreteBlend::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_DiscreteBlend_Name", "Discrete Blend");
		return InterfaceName;
	}
	const FText& IDiscreteBlend::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_DiscreteBlend_ShortName", "DSB");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	float IDiscreteBlend::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendWeight(Context, ChildIndex);
		}

		return -1.0f;
	}

	const FAlphaBlend* IDiscreteBlend::GetBlendState(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendState(Context, ChildIndex);
		}

		return nullptr;
	}

	int32 IDiscreteBlend::GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const
	{
		TTraitBinding<IDiscreteBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendDestinationChildIndex(Context);
		}

		return INDEX_NONE;
	}

	void IDiscreteBlend::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.OnBlendTransition(Context, OldChildIndex, NewChildIndex);
		}
	}

	void IDiscreteBlend::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.OnBlendInitiated(Context, ChildIndex);
		}
	}

	void IDiscreteBlend::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.OnBlendTerminated(Context, ChildIndex);
		}
	}
}
