// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IEvaluate.h"

#include "Animation/AnimCurveUtils.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitInterfaces/IHierarchy.h"
#include "AnimNextAnimGraphStats.h"
#include "EvaluationVM/EvaluationFlags.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/UAFAnimGraphTrace.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/AnimNextModuleInstance.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/ValueBundle.h"

namespace UE::UAF
{
	namespace Private
	{
		static bool GUseAbstractSkeletonRuntime = false;
		FAutoConsoleVariableRef CVar_UseAbstractSkeletonRuntime(TEXT("UAF.UseAbstractSkeletonRuntime"), GUseAbstractSkeletonRuntime, TEXT("Whether or not to use the abstract skeleton runtime"));
	}

	FEvaluateTraversalContext::FEvaluateTraversalContext(FEvaluationProgram& InEvaluationProgram)
		: EvaluationProgram(InEvaluationProgram)
	{
	}

	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IEvaluate)

#if WITH_EDITOR
	const FText& IEvaluate::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_Evaluate_Name", "Evaluate");
		return InterfaceName;
	}
	const FText& IEvaluate::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_Evaluate_ShortName", "EVA");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void IEvaluate::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		TTraitBinding<IEvaluate> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.PreEvaluate(Context);
		}
	}

	void IEvaluate::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		TTraitBinding<IEvaluate> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.PostEvaluate(Context);
		}
	}

	TWeakObjectPtr<const USkeletalMeshComponent> FEvaluateGraphContext::GetBindingObject() const
	{
		// Pull the component from the system's output component
		if (FAnimNextModuleInstance* SystemInstance = GraphInstance.GetModuleInstance())
		{
			return SystemInstance->GetOrAddComponent<FUAFSystemOutputComponent>().GetSkeletalMeshComponent();
		}
		return TWeakObjectPtr<const USkeletalMeshComponent>();
	}

	enum class EEvaluateStep
	{
		PreEvaluate,
		PostEvaluate,
	};

	// This structure is transient and lives either on the stack or the memstack and its destructor may not be called
	struct FEvaluateEntry
	{
		// The trait handle that points to our node to update
		FWeakTraitPtr		TraitPtr;

		// Which step we wish to perform when we next see this entry
		EEvaluateStep		DesiredStep = EEvaluateStep::PreEvaluate;

		// The trait stack binding for this evaluate entry
		FTraitStackBinding	TraitStack;

		// Once we've called PreUpdate, we cache the trait binding to avoid a redundant query to call PostUpdate
		TTraitBinding<IEvaluate> EvaluateTrait;

		// These pointers are mutually exclusive
		// An entry is either part of the pending stack, the free list, or neither
		union
		{
			// Next entry in the linked list of free entries
			FEvaluateEntry* NextFreeEntry = nullptr;

			// Previous entry below us in the nodes pending stack
			FEvaluateEntry* PrevStackEntry;
		};

		FEvaluateEntry(const FWeakTraitPtr& InTraitPtr, EEvaluateStep InDesiredStep, FEvaluateEntry* InPrevStackEntry = nullptr)
			: TraitPtr(InTraitPtr)
			, DesiredStep(InDesiredStep)
			, PrevStackEntry(InPrevStackEntry)
		{
		}
	};

	FEvaluationProgram EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext)
	{
		return EvaluateGraph(EvaluateGraphContext, EvaluateGraphContext.GetGraphInstance().GetGraphRootPtr());
	}

	FEvaluationProgram EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext, const FWeakTraitPtr& GraphRootPtr)
	{
		SCOPED_NAMED_EVENT(UAF_EvaluateGraph, FColor::Orange);
		
		FEvaluationProgram EvaluationProgram;

		if (!GraphRootPtr.IsValid())
		{
			return EvaluationProgram;	// Nothing to update
		}

		FMemStack& MemStack = FMemStack::Get();
		FMemMark Mark(MemStack);

		FChildrenArray Children;

		FEvaluateTraversalContext TraversalContext(EvaluationProgram);
		TraversalContext.BindTo(GraphRootPtr);

		// Add the graph root to kick start the evaluation process
		FEvaluateEntry GraphRootEntry(GraphRootPtr, EEvaluateStep::PreEvaluate);
		FEvaluateEntry* NodesPendingUpdateStackTop = &GraphRootEntry;

		// List of free entries we can recycle
		FEvaluateEntry* FreeEntryList = nullptr;

		TTraitBinding<IHierarchy> HierarchyTrait;

		// Process every node twice: pre-evaluate and post-evaluate
		while (NodesPendingUpdateStackTop != nullptr)
		{
			// Grab the top most entry
			FEvaluateEntry* Entry = NodesPendingUpdateStackTop;
			bool bIsEntryUsed = true;

			const FWeakTraitPtr& EntryTraitPtr = Entry->TraitPtr;

			if (Entry->DesiredStep == EEvaluateStep::PreEvaluate)
			{
				// Bind and cache our trait stack
				ensure(TraversalContext.GetStack(EntryTraitPtr, Entry->TraitStack));

				if (Entry->TraitStack.GetInterface(Entry->EvaluateTrait))
				{
					// This is the first time we visit this node, time to pre-evaluate
					Entry->EvaluateTrait.PreEvaluate(TraversalContext);

					// Leave our entry on top of the stack, we'll need to call PostEvaluate once the children
					// we'll push on top finish
					Entry->DesiredStep = EEvaluateStep::PostEvaluate;
				}
				else
				{
					// This node doesn't implement IUpdate, we can pop it from the stack
					NodesPendingUpdateStackTop = Entry->PrevStackEntry;
					bIsEntryUsed = false;
				}

				if (Entry->TraitStack.GetInterface(HierarchyTrait))
				{
					IHierarchy::GetStackChildren(TraversalContext, Entry->TraitStack, Children);

					// Append our children in reserve order so that they are visited in the same order they were added
					for (int32 ChildIndex = Children.Num() - 1; ChildIndex >= 0; --ChildIndex)
					{
						const FWeakTraitPtr& ChildPtr = Children[ChildIndex];
						if (!ChildPtr.IsValid())
						{
							continue;
						}

						// Insert our new child on top of the stack

						FEvaluateEntry* ChildEntry;
						if (FreeEntryList != nullptr)
						{
							// Grab an entry from the free list
							ChildEntry = FreeEntryList;
							FreeEntryList = ChildEntry->NextFreeEntry;

							ChildEntry->TraitPtr = ChildPtr;
							ChildEntry->DesiredStep = EEvaluateStep::PreEvaluate;
							ChildEntry->PrevStackEntry = NodesPendingUpdateStackTop;
						}
						else
						{
							// Allocate a new entry
							ChildEntry = new(MemStack) FEvaluateEntry(ChildPtr, EEvaluateStep::PreEvaluate, NodesPendingUpdateStackTop);
						}

						NodesPendingUpdateStackTop = ChildEntry;
					}

					// Reset our container for the next time we need it
					Children.Reset();
				}

				// Break and continue to the next top-most entry
				// It is either a child ready for its pre-evaluate or our current entry ready for its post-evaluate (leaf)
			}
			else
			{
				// We've already visited this node once, time to post-update
				check(Entry->EvaluateTrait.IsValid());
				Entry->EvaluateTrait.PostEvaluate(TraversalContext);

				// Now that we are done processing this entry, we can pop it
				NodesPendingUpdateStackTop = Entry->PrevStackEntry;
				bIsEntryUsed = false;

				// Break and continue to the next top-most entry
				// It is either a sibling read for its post-evaluate or our parent entry ready for its post-evaluate
			}

			if (!bIsEntryUsed)
			{
				// This entry is no longer used, add it to the free list
				Entry->NextFreeEntry = FreeEntryList;
				FreeEntryList = Entry;
			}
		}

		return EvaluationProgram;
	}

	void EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext, FAnimNextGraphLODPose& OutputPose)
	{
		FAnimNextGraphInstance& GraphInstance = EvaluateGraphContext.GetGraphInstance();
		const FReferencePose& RefPose = EvaluateGraphContext.GetRefPose();
		const int32 GraphLODLevel = EvaluateGraphContext.GetGraphLODLevel();

		const FEvaluationProgram EvaluationProgram = EvaluateGraph(EvaluateGraphContext);

		TRACE_UAF_EVALUATIONPROGRAM(EvaluationProgram, GraphInstance);

		{
			SCOPED_NAMED_EVENT(UAF_EvaluatePose, FColor::Orange);

			UAbstractSkeletonSetBinding* Binding = EvaluateGraphContext.GetSkeletonSetBinding();
			FName SetName = EvaluateGraphContext.GetSkeletonSetName();
			USkeletalMesh* SkeletalMesh = EvaluateGraphContext.GetSkeletalMesh();

			FEvaluationVM EvaluationVM(EEvaluationFlags::All, RefPose, Binding, SkeletalMesh, SetName, GraphLODLevel);
			bool bHasValidOutput = false;

			if (!EvaluationProgram.IsEmpty())
			{
				EvaluationProgram.Execute(EvaluationVM);

				TUniquePtr<FValueBundle> Collection;
				if (Private::GUseAbstractSkeletonRuntime && EvaluationVM.PopValue(ATTRIBUTE_STACK_NAME, Collection))
				{
					const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute>();
					if (!Collection->IsEmpty())
					{
						// Attributes should no longer be attached to bones in UAF
						// Attributes should have unique names instead
						// Attach to root for now for backwards compatibility
						const FCompactPoseBoneIndex RootCompactPoseBoneIndex(0);

						// Bound value maps
						for (auto It = Collection->GetBoundValueMaps().CreateIterator(); It; ++It)
						{
							const FBoundValueMap* Map = It.GetMap();
							const FAttributeTypedSetPtr& TypedSet = Map->GetTypedSet();

							if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Cast<FBoneTransformAnimationAttribute>(Map))
							{
								const int32 NumPoseLODBones = OutputPose.LODPose.GetNumBones();

								for (int32 LODBoneIndex = 0; LODBoneIndex < NumPoseLODBones; ++LODBoneIndex)
								{
									const int32 SkeletonBoneIndex = RefPose.GetSkeletonBoneIndexFromLODBoneIndex(LODBoneIndex);
									const FAttributeBindingIndex BoneBindingIndex = FAttributeBindingIndex(FSkeletonPoseBoneIndex(SkeletonBoneIndex));
									check(BoneBindingIndex.IsValid());

									const FAttributeSetIndex BoneSetIndex = TypedSet->GetIndex(BoneBindingIndex);
									if (BoneSetIndex.IsValid())
									{
										OutputPose.LODPose.LocalTransformsView[LODBoneIndex] = (*BoneTransforms)[BoneSetIndex].Value;
									}
									else
									{
										// This bone isn't present with the current named set, use the bind pose value instead
										OutputPose.LODPose.LocalTransformsView[LODBoneIndex] = RefPose.GetRefPoseTransform(LODBoneIndex);
									}
								}
							}
							else if (const TBoundValueMap<FFloatAnimationAttribute>* Floats = Cast<FFloatAnimationAttribute>(Map))
							{
								const auto GetNameFromIndex = [&TypedSet](int32 CurveIndex)
									{
										return TypedSet->GetName(FAttributeSetIndex(CurveIndex));
									};

								const auto GetValueFromIndex = [Floats](int32 CurveIndex)
									{
										return (*Floats)[FAttributeSetIndex(CurveIndex)].Value;
									};

								// Convert float set attributes into curves
								UE::Anim::FCurveUtils::BuildSorted(OutputPose.Curves, Floats->Num(), GetNameFromIndex, GetValueFromIndex);
							}
							else
							{
								// Everything else
								const UScriptStruct* ValueType = Map->GetValueType();
								const int32 NumAttributes = Map->Num();

								for (FAttributeSetIndex SetIndex(0); SetIndex < NumAttributes; ++SetIndex)
								{
									const UE::Anim::FAttributeId AttributeId(TypedSet->GetName(SetIndex), RootCompactPoseBoneIndex);

									uint8* AttributePtr = OutputPose.Attributes.FindOrAdd(ValueType, AttributeId);

									Map->GetValueWithGetter(SetIndex, [AttributePtr](UScriptStruct* ValueType_, const uint8* ValuePtr)
										{
											ValueType_->CopyScriptStruct(AttributePtr, ValuePtr);
										});
								}
							}
						}

						// Unbound value maps
						for (auto It = Collection->GetUnboundValueMaps().CreateConstIterator(); It; ++It)
						{
							const FUnboundValueMap* Map = It.GetMap();
							const UScriptStruct* ValueType = Map->GetValueType();
							const int32 NumValues = Map->Num();

							for (int32 ValueIndex = 0; ValueIndex < NumValues; ++ValueIndex)
							{
								const UE::Anim::FAttributeId AttributeId(Map->GetName(ValueIndex), RootCompactPoseBoneIndex);

								uint8* AttributePtr = OutputPose.Attributes.FindOrAdd(ValueType, AttributeId);

								Map->GetValueWithGetter(ValueIndex, [AttributePtr](UScriptStruct* ValueType_, const uint8* ValuePtr)
									{
										ValueType_->CopyScriptStruct(AttributePtr, ValuePtr);
									});
							}
						}
					}
					else
					{
						// Nothing was produced, populate with bind pose
						FKeyframeState ReferenceKeyframe = EvaluationVM.MakeReferenceKeyframe(false);
						OutputPose.LODPose.CopyFrom(ReferenceKeyframe.Pose);
						OutputPose.Curves.CopyFrom(ReferenceKeyframe.Curves);
						OutputPose.Attributes.CopyFrom(ReferenceKeyframe.Attributes);
					}

					bHasValidOutput = true;
				}

				TUniquePtr<FKeyframeState> EvaluatedKeyframe;
				if (!bHasValidOutput && EvaluationVM.PopValue(KEYFRAME_STACK_NAME, EvaluatedKeyframe))
				{
					OutputPose.LODPose.CopyFrom(EvaluatedKeyframe->Pose);
					OutputPose.Curves.CopyFrom(EvaluatedKeyframe->Curves);
					OutputPose.Attributes.CopyFrom(EvaluatedKeyframe->Attributes);
					bHasValidOutput = true;
				}
			}

			if (!bHasValidOutput)
			{
				// We need to output a valid pose, generate one
				FKeyframeState ReferenceKeyframe = EvaluationVM.MakeReferenceKeyframe(false);
				OutputPose.LODPose.CopyFrom(ReferenceKeyframe.Pose);
				OutputPose.Curves.CopyFrom(ReferenceKeyframe.Curves);
				OutputPose.Attributes.CopyFrom(ReferenceKeyframe.Attributes);
			}
		}
	}

	bool UseAbstractSkeletonRuntime()
	{
		return Private::GUseAbstractSkeletonRuntime;
	}
}
