// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IGarbageCollection.h"

#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IGarbageCollection)

#if WITH_EDITOR
	const FText& IGarbageCollection::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IGarbageCollection_Name", "Garbage Collection");
		return InterfaceName;
	}
	const FText& IGarbageCollection::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IGarbageCollection_ShortName", "GC");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void IGarbageCollection::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		TTraitBinding<IGarbageCollection> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.AddReferencedObjects(Context, Collector);
		}
	}
}
