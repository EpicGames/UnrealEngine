// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IGraphFactory.h"

#include "Asset/UAFAnimGraphAssetData.h"
#include "Factory/AnimGraphFactory.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "TraitCore/ExecutionContext.h"
#include "UAF/UAFAssetFactory.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IGraphFactory)

#if WITH_EDITOR
	const FText& IGraphFactory::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IGraphFactory_Name", "Graph Factory");
		return InterfaceName;
	}
	const FText& IGraphFactory::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IGraphFactory_ShortName", "GF");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void IGraphFactory::GetFactoryParams(FExecutionContext& Context, const TTraitBinding<IGraphFactory>& Binding, FGraphAssetHandleConstView AssetHandle, FAnimNextFactoryParams& InOutParams) const
	{
		TTraitBinding<IGraphFactory> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.GetFactoryParams(Context, AssetHandle, InOutParams);
		}
	}

	const UUAFAnimGraph* IGraphFactory::GetOrBuildGraph(FExecutionContext& Context, const FTraitBinding& Binding, FGraphAssetHandleConstView AssetHandle, FAnimNextFactoryParams& InOutParams)
	{
		if (AssetHandle.IsValid() == false || AssetHandle.Get().Validate() == false)
		{
			return nullptr;
		}
		// If a graph was provided directly, just use it
		if (const FUAFGraphFactoryAsset_Graph* AnimationGraphAssetData = AssetHandle.GetPtr<FUAFGraphFactoryAsset_Graph>())
		{
			return AnimationGraphAssetData->AnimationGraph;
		}

		// Otherwise attempt to build a graph procedurally from the supplied params
		
		if (!InOutParams.IsValid())
		{
			// If no params provided, initialize with defaults from the asset handle
			InOutParams = FAnimGraphFactory::GetDefaultParamsForAsset(AssetHandle);
			FAnimGraphFactory::InitializeDefaultParamsForAsset(AssetHandle, InOutParams);
		}

		// Defer to the trait stack to gather any additional traits to instantiate
		TTraitBinding<IGraphFactory> GraphBuilderBinding;
		if (Binding.GetStackInterface(GraphBuilderBinding))
		{
			GraphBuilderBinding.GetFactoryParams(Context, AssetHandle, InOutParams);
		}

		return FAnimGraphFactory::BuildGraph(InOutParams.GetBuilder());

	}

	const UUAFAnimGraph* IGraphFactory::GetOrBuildGraph(FExecutionContext& Context, const FTraitBinding& Binding, const UObject* Asset,
		FAnimNextFactoryParams& InOutParams)
	{
		FGraphAssetHandle AssetHandle = FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(Asset);
		if (AssetHandle.IsValid() == false)
		{
			return nullptr;
		}

		return GetOrBuildGraph(Context, Binding, AssetHandle, InOutParams);
	}
}
