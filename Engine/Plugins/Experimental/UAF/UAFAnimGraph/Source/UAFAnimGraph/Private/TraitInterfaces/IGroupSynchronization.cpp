// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IGroupSynchronization.h"

#include "TraitCore/ExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IGroupSynchronization)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IGroupSynchronization)

#if WITH_EDITOR
	const FText& IGroupSynchronization::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IGroupSynchronization_Name", "Group Synchronization");
		return InterfaceName;
	}
	const FText& IGroupSynchronization::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IGroupSynchronization_ShortName", "GRS");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	FSyncGroupParameters IGroupSynchronization::GetGroupParameters(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding) const
	{
		TTraitBinding<IGroupSynchronization> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetGroupParameters(Context);
		}

		return FSyncGroupParameters();
	}

	void IGroupSynchronization::AdvanceBy(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		TTraitBinding<IGroupSynchronization> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.AdvanceBy(Context, DeltaTime, bDispatchEvents);
		}
	}
}
