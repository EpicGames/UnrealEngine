// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IHierarchy.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IHierarchy)

#if WITH_EDITOR
	const FText& IHierarchy::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IHierarchy_Name", "Hierarchy");
		return InterfaceName;
	}
	const FText& IHierarchy::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IHierarchy_ShortName", "HIE");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	uint32 IHierarchy::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		// We only wish to count children of the queried trait
		// No need to forward to our super
		// To count all children of a trait stack, use GetNumStackChildren instead
		return 0;
	}

	void IHierarchy::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		// We only wish to append children of the queried trait
		// No need to forward to our super
		// To get all children of a trait stack, use GetStackChildren instead
	}

	void IHierarchy::GetStackChildren(const FExecutionContext& Context, const FTraitStackBinding& Binding, FChildrenArray& Children)
	{
		Children.Reset();

		if (!Binding.IsValid())
		{
			return;
		}

		TTraitBinding<IHierarchy> HierarchyTrait;

		// Visit the trait stack and queue our children
		Binding.GetInterface(HierarchyTrait);
		while (HierarchyTrait.IsValid())
		{
			HierarchyTrait.GetChildren(Context, Children);

			Binding.GetInterfaceSuper(HierarchyTrait, HierarchyTrait);
		}
	}

	uint32 IHierarchy::GetNumStackChildren(const FExecutionContext& Context, const FTraitStackBinding& Binding)
	{
		if (!Binding.IsValid())
		{
			return 0;
		}

		TTraitBinding<IHierarchy> HierarchyTrait;
		uint32 NumChildren = 0;

		// Visit the trait stack and append the children count
		Binding.GetInterface(HierarchyTrait);
		while (HierarchyTrait.IsValid())
		{
			NumChildren += HierarchyTrait.GetNumChildren(Context);

			Binding.GetInterfaceSuper(HierarchyTrait, HierarchyTrait);
		}

		return NumChildren;
	}

	FWeakTraitPtr IHierarchy::GetStackForwardingChild(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		// By default, the 'forwarding child' is a singular child for the whole stack, and multi child stacks must override this function if they want to forward
		FChildrenArray Children;
		GetChildren(Context, Binding, Children);

		if (Children.Num() == 1 && Children[0].IsValid())
		{
			return Children[0];
		}
		return FWeakTraitPtr();
	}
}
