// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IInertializerBlend.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IInertializerBlend)

#if WITH_EDITOR
	const FText& IInertializerBlend::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IInertializerBlend_Name", "Inertializer Blend");
		return InterfaceName;
	}
	const FText& IInertializerBlend::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IInertializerBlend_ShortName", "IB");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	float IInertializerBlend::GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IInertializerBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendTime(Context, ChildIndex);
		}

		return 0.0f;
	}
}
