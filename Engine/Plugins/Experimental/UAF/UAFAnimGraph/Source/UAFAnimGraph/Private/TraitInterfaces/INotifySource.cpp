// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/INotifySource.h"
#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{

AUTO_REGISTER_ANIM_TRAIT_INTERFACE(INotifySource)

#if WITH_EDITOR
const FText& INotifySource::GetDisplayName() const
{
	static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_INotifySource_Name", "Notify Source");
	return InterfaceName;
}
const FText& INotifySource::GetDisplayShortName() const
{
	static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_INotifySource_ShortName", "NOT");
	return InterfaceShortName;
}
#endif // WITH_EDITOR

void INotifySource::GetNotifies(FExecutionContext& Context, const TTraitBinding<INotifySource>& Binding, float StartPosition, float Duration, bool bLooping, TArray<FAnimNotifyEventReference>& OutNotifies) const
{
	TTraitBinding<INotifySource> SuperBinding;
	if (Binding.GetStackInterfaceSuper(SuperBinding))
	{
		return SuperBinding.GetNotifies(Context, StartPosition, Duration, bLooping, OutNotifies);
	}
}

}
