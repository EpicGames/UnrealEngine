// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/ISmoothBlend.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(ISmoothBlend)

#if WITH_EDITOR
	const FText& ISmoothBlend::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ISmoothBlend_Name", "Smooth Blend");
		return InterfaceName;
	}
	const FText& ISmoothBlend::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ISmoothBlend_ShortName", "SMB");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	float ISmoothBlend::GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<ISmoothBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendTime(Context, ChildIndex);
		}

		return 0.0f;
	}

	EAlphaBlendOption ISmoothBlend::GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<ISmoothBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendType(Context, ChildIndex);
		}

		return EAlphaBlendOption::Linear;
	}

	UCurveFloat* ISmoothBlend::GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<ISmoothBlend> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetCustomBlendCurve(Context, ChildIndex);
		}

		return nullptr;
	}
}
