// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/ISmoothBlendPerBone.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(ISmoothBlendPerBone)

#if WITH_EDITOR
	const FText& ISmoothBlendPerBone::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ISmoothBlendPerBone_Name", "Smooth Blend Per Bone");
		return InterfaceName;
	}
	const FText& ISmoothBlendPerBone::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ISmoothBlendPerBone_ShortName", "SMBPB");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	const UUAFBlendProfile* ISmoothBlendPerBone::GetBlendProfile(FExecutionContext& Context, const TTraitBinding<ISmoothBlendPerBone>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<ISmoothBlendPerBone> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetBlendProfile(Context, ChildIndex);
		}

		return nullptr;
	}
}
