// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/ITimeline.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(ITimeline)

#if WITH_EDITOR
	const FText& ITimeline::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ITimeline_Name", "Timeline");
		return InterfaceName;
	}
	const FText& ITimeline::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ITimeline_ShortName", "TIM");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void ITimeline::GetSyncMarkers(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineSyncMarkerArray& OutSyncMarkers) const
	{
		TTraitBinding<ITimeline> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetSyncMarkers(Context, OutSyncMarkers);
		}
	}

	bool ITimeline::GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const
	{
		TTraitBinding<ITimeline> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetState(Context, OutTimelineState);
		}

		return false;
	}

	bool ITimeline::GetDelta(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineDelta& OutTimelineDelta) const
	{
		TTraitBinding<ITimeline> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetDelta(Context, OutTimelineDelta);
		}

		return false;
	}
}
