// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/ITimelinePlayer.h"

#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(ITimelinePlayer)

#if WITH_EDITOR
	const FText& ITimelinePlayer::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ITimelinePlayer_Name", "Timeline Player");
		return InterfaceName;
	}
	const FText& ITimelinePlayer::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_ITimelinePlayer_ShortName", "TIM");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void ITimelinePlayer::AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		TTraitBinding<ITimelinePlayer> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.AdvanceBy(Context, DeltaTime, bDispatchEvents);
		}
	}
}
