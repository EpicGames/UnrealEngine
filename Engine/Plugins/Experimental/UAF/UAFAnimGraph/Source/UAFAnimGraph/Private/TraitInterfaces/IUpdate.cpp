// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitInterfaces/IUpdate.h"

#include "TraitInterfaces/IHierarchy.h"
#include "Graph/UAFGraphInstanceComponent.h"
#include "AnimNextAnimGraphStats.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "TraitCore/TraitEventRaising.h"

#ifndef DO_UAF_GRAPH_LOOPGUARD
	#if (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)
		#define DO_UAF_GRAPH_LOOPGUARD 1
	#else
		#define DO_UAF_GRAPH_LOOPGUARD 0
	#endif
#endif

static int32 GMaxUAFGraphUpdateLoopCount = 1000;
static FAutoConsoleVariableRef CVarMaxUAFGraphUpdateLoopCount(
	TEXT("UAF.MaxGraphUpdateLoopCount"),
	GMaxUAFGraphUpdateLoopCount,
	TEXT("Sets the number of update loops before graph is considered in an infinite loop and will break out of update.\n"),
	ECVF_Default
);

namespace UE::UAF
{
	namespace Private
	{
		struct FUpdateEventBookkeepingEntry;

		struct FUpdateEventBookkeepingList
		{
			// We maintain a double linked list of event bookkeeping entries we execute before post-update
			FUpdateEventBookkeepingEntry* EventBookkeepingHead = nullptr;
			FUpdateEventBookkeepingEntry* EventBookkeepingTail = nullptr;
		};

		// This structure is transient and lives either on the stack or the memstack and its destructor may not be called
		struct FUpdateEntry
		{
			// The trait state for this entry
			FTraitUpdateState			TraitState;

			// The trait handle that points to our node to update
			FWeakTraitPtr				TraitPtr;

			// Whether or not PreUpdate had been called already
			// TODO: Store bHasPreUpdated in the LSB of the entry pointer to save padding?
			bool						bHasPreUpdated = false;

			// The trait stack binding for this update entry
			FTraitStackBinding			TraitStack;

			// Once we've called PreUpdate, we cache the trait binding to avoid a redundant query to call PostUpdate
			TTraitBinding<IUpdate>		UpdateTrait;

			// A pointer to our parent entry or nullptr if we are the root
			FUpdateEntry*				ParentEntry = nullptr;

			// We maintain a double linked list of event bookkeeping entries we execute before post-update
			FUpdateEventBookkeepingList EventBookkeepingList;

			// These pointers are mutually exclusive
			// An entry is either part of the queued update stack, the update stack, the free list, or none of the above
			union
			{
				// Next entry in the stack of free entries
				FUpdateEntry* NextFreeEntry = nullptr;

				// Previous entry on the update stack
				FUpdateEntry* PrevUpdateStackEntry;

				// Previous entry on the queued update stack
				FUpdateEntry* PrevQueuedUpdateStackEntry;
			};

			FUpdateEntry(const FWeakTraitPtr& InTraitPtr, const FTraitUpdateState InTraitState)
				: TraitState(InTraitState)
				, TraitPtr(InTraitPtr)
			{
			}
		};

		// The update traversal performs various event bookkeeping actions
		enum class FUpdateEventBookkeepingAction : uint8
		{
			// Pushes an output trait event
			PushOutput,

			// Consumes a trait event
			Consume,
		};

		// Encapsulates a bookkeeping entry that we'll execute before post-update
		// These may be allocated on the memstack and their destructor might not run
		struct FUpdateEventBookkeepingEntry
		{
			// The event the action manipulates
			FAnimNextTraitEventPtr Event;

			// The action to perform
			// TODO: Could back a single bit in the LSB of one of the linked list pointers
			FUpdateEventBookkeepingAction Action = FUpdateEventBookkeepingAction::Consume;

			// Once allocated and bound to an update entry, bookkeeping entries form a double linked list
			// where we append at the tail (next) to maintain queue ordering when we execute them
			// When the bookkeeping entry isn't bound to an update entry, it lives in a free list where
			// the next entry is the top of the free list
			FUpdateEventBookkeepingEntry* NextEntry = nullptr;
			FUpdateEventBookkeepingEntry* PrevEntry = nullptr;

			// Creates a fresh entry
			FUpdateEventBookkeepingEntry(FUpdateEventBookkeepingAction InAction, FAnimNextTraitEventPtr InEvent)
				: Event(InEvent)
				, Action(InAction)
			{
			}
		};

		// Queues the specified bookkeeping entry in the provided update entry
		static void QueueBookkeepingEntry(FUpdateEventBookkeepingList& BookkeepingList, FUpdateEventBookkeepingEntry* BookkeepingEntry)
		{
			// Previous entry is the current tail (if any)
			BookkeepingEntry->PrevEntry = BookkeepingList.EventBookkeepingTail;

			if (BookkeepingList.EventBookkeepingHead == nullptr)
			{
				// This is the first bookkeeping entry, start our list
				BookkeepingList.EventBookkeepingHead = BookkeepingEntry;
			}
			else
			{
				// Stitch the current tail with our new entry before we update it
				BookkeepingList.EventBookkeepingTail->NextEntry = BookkeepingEntry;
			}

			// Append our entry at the tail
			BookkeepingList.EventBookkeepingTail = BookkeepingEntry;
		}

		// Raises every queued event from the list on the specified entry
		static void RaiseTraitEvents(FUpdateTraversalContext& Context, Private::FUpdateEntry* UpdateEntry, const UE::UAF::FTraitEventList& EventList)
		{
			// TODO: Performance note
			// 
			// Event lists are typically very small or empty and similarly most nodes handle few or no events
			// They are thus a great fit to leverage bloom filters
			// A node can pre-compute and cache one in its node template. This bloom filter contains all the event types it handles
			// An event list can build a bloom filter of the event types it contains
			// 
			// Here (in this function), we could test if the event list bloom filter overlaps the node bloom filter: (node filter AND list filter) != 0
			// If any bits intersect, then perhaps the node handles a type contained in the list (if the node handles nothing or if the list is empty, the result is always 0)
			// With most nodes handling few events and the event list containing few events, we are likely to be able to skip many nodes with a very cheap test
			// 
			// Next, when we iterate over the event list, we can perform a similar test again for every event: (event filter AND node filter) == event filter
			// If the event filter is contained in the node filter, the node might be handling the event type
			// If not, then for sure it doesn't handle that event and we can avoid the virtual call and the event branching
			// With most nodes handling few events and with an inclusion test, the rate of false positives is likely very low and we can skip most events
			// avoiding the dispatch cost
			// 
			// To efficiently support this, we need to be able to build and cache bloom filters for each event in our list
			// Caching the filter in the event itself is tricky as there is no good place to perform initialization work
			// if we allow inheritance of events. Instead, the event list could store a struct with the event ptr and the bloom filter
			// that we build when the event is inserted into the list. This would avoid the need to call a virtual function on the event
			// to return a static constexpr filter. Similarly, we could store the event type UID alongside. We could pass this to the FTrait::OnTraitEvent
			// function to avoid the virtual call we have for GetTypeUID.
			// 

			UE::UAF::RaiseTraitEvents(Context, UpdateEntry->TraitStack, EventList);
		}
	}

	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IUpdate)

#if WITH_EDITOR
	const FText& IUpdate::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IUpdate_Name", "Update");
		return InterfaceName;
	}
	const FText& IUpdate::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IUpdate_ShortName", "UPD");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void IUpdate::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		TTraitBinding<IUpdate> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.OnBecomeRelevant(Context, TraitState);
		}
	}

	void IUpdate::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		TTraitBinding<IUpdate> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.PreUpdate(Context, TraitState);
		}
	}

	void IUpdate::PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		TTraitBinding<IUpdate> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.PostUpdate(Context, TraitState);
		}
	}

	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IUpdateTraversal)

#if WITH_EDITOR
	const FText& IUpdateTraversal::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IUpdateTraversal_Name", "Update Traversal");
		return InterfaceName;
	}
	const FText& IUpdateTraversal::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IUpdateTraversal_ShortName", "TRA");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	void IUpdateTraversal::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		// Nothing to do
		// This function is called for each trait on the stack, one by one
		// No need to forward to our super
	}

	TWeakObjectPtr<const USkeletalMeshComponent> FUpdateGraphContext::GetBindingObject() const
	{
		// Pull the component from the system's output component
		if (FAnimNextModuleInstance* SystemInstance = GraphInstance.GetModuleInstance())
		{
			return SystemInstance->GetOrAddComponent<FUAFSystemOutputComponent>().GetSkeletalMeshComponent();
		}
		return TWeakObjectPtr<const USkeletalMeshComponent>();
	}

	//////////////////////////////////////////////////////////////////////////
	// Traversal implementation

	void FUpdateTraversalContext::RaiseInputTraitEvent(FAnimNextTraitEventPtr Event)
	{
		if (!Event || !Event->IsValid())
		{
			return;
		}

		if (!ensureMsgf(ExecutingEntry == nullptr || !ExecutingEntry->bHasPreUpdated, TEXT("Input trait events can only be raised before a trait stack pre-updates")))
		{
			return;
		}

		if (ExecutingEntry != nullptr)
		{
			// If we are currently executing a node, we don't want the input event to be seen by our parent/siblings
			// Add a bookkeeping entry to consume the event when we post-update
			Private::FUpdateEventBookkeepingEntry* BookkeepingEntry = GetNewBookkeepingEntry(Private::FUpdateEventBookkeepingAction::Consume, Event);

			Private::QueueBookkeepingEntry(ExecutingEntry->EventBookkeepingList, BookkeepingEntry);
		}

		InputEventList->Push(MoveTemp(Event));
	}

	void FUpdateTraversalContext::RaiseOutputTraitEvent(FAnimNextTraitEventPtr Event)
	{
		if (!Event || !Event->IsValid())
		{
			return;
		}

		ensureMsgf(Event->IsTransient(), TEXT("Output trait events must have transient duration"));

		if (ExecutingEntry != nullptr && ExecutingEntry->ParentEntry != nullptr)
		{
			// If we are currently executing a node, we don't want the output event to be visible on this node or its siblings
			// Only its parent should see it
			// Add a bookkeeping entry to push the event when our parent post-updates
			// If we don't have a parent (e.g. root node), then we append to a fake parent list
			Private::FUpdateEventBookkeepingEntry* BookkeepingEntry = GetNewBookkeepingEntry(Private::FUpdateEventBookkeepingAction::PushOutput, Event);

			Private::FUpdateEventBookkeepingList* BookkeepingList = ExecutingEntry->ParentEntry != nullptr ? &ExecutingEntry->ParentEntry->EventBookkeepingList : RootParentBookkeepingEntryList;
			Private::QueueBookkeepingEntry(*BookkeepingList, BookkeepingEntry);
		}
		else
		{
			// We aren't executing a trait stack or we are the root stack, just queue the output
			// We might be in a component pre/post-update
			OutputEventList->Push(MoveTemp(Event));
		}
	}

	void FUpdateTraversalContext::ExecuteBookkeepingActions(Private::FUpdateEventBookkeepingList& BookkeepingList)
	{
		if (BookkeepingList.EventBookkeepingHead == nullptr)
		{
			return;	// Nothing to do
		}

		// Iterate over our action list
		Private::FUpdateEventBookkeepingEntry* BookkeepingEntry = BookkeepingList.EventBookkeepingHead;
		while (BookkeepingEntry != nullptr)
		{
			switch (BookkeepingEntry->Action)
			{
			case Private::FUpdateEventBookkeepingAction::PushOutput:
				OutputEventList->Push(MoveTemp(BookkeepingEntry->Event));
				break;
			case Private::FUpdateEventBookkeepingAction::Consume:
				BookkeepingEntry->Event->MarkConsumed();
				break;
			}

			// Reset our pointer manually since the destructor won't run
			BookkeepingEntry->Event.Reset();

			Private::FUpdateEventBookkeepingEntry* NextEntry = BookkeepingEntry->NextEntry;

			// Return our entry to the free list
			PushFreeBookkeepingEntry(BookkeepingEntry);

			// Continue iterating
			BookkeepingEntry = NextEntry;
		}

		// Clear the list
		BookkeepingList.EventBookkeepingHead = BookkeepingList.EventBookkeepingTail = nullptr;
	}

	void FUpdateTraversalContext::PushQueuedUpdateEntries(FUpdateTraversalQueue& TraversalQueue, Private::FUpdateEntry* ParentEntry)
	{
		// Pop every entry from the queued update stack and push them onto the update stack
		// reversing their order
		while (Private::FUpdateEntry* Entry = TraversalQueue.QueuedUpdateStackHead)
		{
			// Update our queued stack head
			TraversalQueue.QueuedUpdateStackHead = Entry->PrevQueuedUpdateStackEntry;

			Entry->ParentEntry = ParentEntry;

			// Push our new entry onto the update stack
			PushUpdateEntry(Entry);
		}
	}

	void FUpdateTraversalContext::PushUpdateEntry(Private::FUpdateEntry* Entry)
	{
		Entry->PrevUpdateStackEntry = UpdateStackHead;
		UpdateStackHead = Entry;
	}

	Private::FUpdateEntry* FUpdateTraversalContext::PopUpdateEntry()
	{
		Private::FUpdateEntry* ChildEntry = UpdateStackHead;
		if (ChildEntry != nullptr)
		{
			// We have a child, set our new head
			UpdateStackHead = ChildEntry->PrevUpdateStackEntry;
		}

		return ChildEntry;
	}

	void FUpdateTraversalContext::PushFreeEntry(Private::FUpdateEntry* Entry)
	{
		Entry->NextFreeEntry = FreeEntryStackHead;
		FreeEntryStackHead = Entry;
	}

	Private::FUpdateEntry* FUpdateTraversalContext::GetNewEntry(const FWeakTraitPtr& TraitPtr, const FTraitUpdateState& TraitState)
	{
		Private::FUpdateEntry* FreeEntry = FreeEntryStackHead;
		if (FreeEntry != nullptr)
		{
			// We have a free entry, set our new head
			FreeEntryStackHead = FreeEntry->NextFreeEntry;

			// Update our entry
			FreeEntry->TraitState = TraitState;
			FreeEntry->TraitPtr = TraitPtr;
			FreeEntry->bHasPreUpdated = false;
			FreeEntry->ParentEntry = nullptr;
			FreeEntry->EventBookkeepingList = Private::FUpdateEventBookkeepingList();
			FreeEntry->NextFreeEntry = nullptr;		// Mark it as not being a member of any list
		}
		else
		{
			// Allocate a new entry
			FreeEntry = new(MemStack) Private::FUpdateEntry(TraitPtr, TraitState);
		}

		return FreeEntry;
	}

	Private::FUpdateEventBookkeepingEntry* FUpdateTraversalContext::GetNewBookkeepingEntry(Private::FUpdateEventBookkeepingAction Action, FAnimNextTraitEventPtr Event)
	{
		Private::FUpdateEventBookkeepingEntry* FreeEntry = FreeBookkeepingEntryStackHead;
		if (FreeEntry != nullptr)
		{
			// We have a free entry, set our new head
			FreeBookkeepingEntryStackHead = FreeEntry->NextEntry;

			// Update our entry
			FreeEntry->Event = Event;
			FreeEntry->Action = Action;
			FreeEntry->NextEntry = nullptr;		// Mark it as not being a member of any list
		}
		else
		{
			// Allocate a new entry
			FreeEntry = new(MemStack) Private::FUpdateEventBookkeepingEntry(Action, Event);
		}

		return FreeEntry;
	}

	void FUpdateTraversalContext::PushFreeBookkeepingEntry(Private::FUpdateEventBookkeepingEntry* Entry)
	{
		Entry->NextEntry = FreeBookkeepingEntryStackHead;
		Entry->PrevEntry = nullptr;

		FreeBookkeepingEntryStackHead = Entry;
	}

	FUpdateTraversalQueue::FUpdateTraversalQueue(FUpdateTraversalContext& InTraversalContext)
		: TraversalContext(InTraversalContext)
	{
	}

	void FUpdateTraversalQueue::Push(const FWeakTraitPtr& ChildPtr, const FTraitUpdateState& ChildTraitState)
	{
		if (!ChildPtr.IsValid())
		{
			return;	// Don't queue invalid pointers
		}

		Private::FUpdateEntry* ChildEntry = TraversalContext.GetNewEntry(ChildPtr, ChildTraitState);

		// We push children that are queued onto a stack
		// Once pre-update is done, we'll pop queued entries one by one and push them
		// onto the update stack
		// This has the effect of reversing the entries so that they are traversed in
		// the same order they are queued in:
		//    - First queued will be at the bottom of the queued stack and it ends up at the
		//      at the top of the update stack (last entry pushed)
		ChildEntry->PrevQueuedUpdateStackEntry = QueuedUpdateStackHead;
		QueuedUpdateStackHead = ChildEntry;
	}

	// Performance note
	// When we process an animation graph for a frame, typically we'll update first before we evaluate
	// As a result of this, when we query for the update interface here, we will likely hit cold memory
	// which will cache miss (by touching the graph instance for the first time).
	// 
	// The processor will cache miss and continue to process as many instructions as it can before the
	// out-of-order execution window fills up. This is problematic here because a lot of the subsequent
	// instructions depend on the node instance and the interface it returns. The processor will be unable
	// to execute any of the instructions that follow in the current loop iteration. However, it might
	// be able to get started on the next node entry which is likely to cache miss as well. Should the
	// processor make it that far and it turns out that we have to push a child onto the stack, all
	// of the work it tried to do ahead of time will have to be thrown away.
	// 
	// There are two things that we can do here to try and help performance: prefetch ahead and bulk
	// query.
	// 
	// If we prefetch, we have to be careful because we do not know what the node will do in its PreUpdate.
	// If it turns out that it does a lot of work, our prefetch might end up getting thrown out. This is
	// because prefetched cache lines typically end up being the first evicted unless they are touched first.
	// It is thus dangerous to use manual prefetching when the memory access pattern isn't fully known.
	// In practice, it is likely viable as most nodes won't do too much work.
	// 
	// A better approach could be to instead bulk query for our interfaces. We could cache in the FUpdateEntry
	// the trait bindings for IUpdate and IHierarchy (and re-use the binding for IUpdate for PostUpdate).
	// Every iteration we could check how many children are queued up on the stack. We could then grab N
	// entries (2 to 4) and query their interfaces in bulk. The idea is to clump the cache miss instructions
	// together and to interleave the interface queries. This will queue up as much work as possible in the
	// out-of-order execution window that will not be thrown away because of a branch. Eventually the first
	// interface query will complete and execution will resume here to call PreUpdate etc. This will be able
	// to happen while the processor still waits on the cache misses and finishes the interface query of the
	// other bulked children. The same effect could be achieved by querying the interfaces after the call
	// to GetChildren by bulk querying all of them right then. This way, as soon as the execution window
	// can clear the end of the loop, it can start working on the next entry which will be warm in the L1
	// cache allowing the CPU to carry ahead before all child interfaces are fully resolved.
	// 
	// The above may seem like a stretch and an insignificant over optimization but it could very well be the
	// key to unlocking large performance gains during traversal. The above optimization would allow us to
	// perform as much useful work as possible while waiting for memory, hiding its slow latency by fully
	// leveraging out-of-order CPU execution.

	void UpdateGraph(FUpdateGraphContext& UpdateGraphContext)
	{
		SCOPED_NAMED_EVENT(UAF_UpdateGraph, FColor::Orange);
		
		FAnimNextGraphInstance& GraphInstance = UpdateGraphContext.GetGraphInstance(); 

		if (!GraphInstance.IsValid())
		{
			return;	// Nothing to update
		}

		if (!ensure(GraphInstance.IsRoot()))
		{
			return;	// We can only update starting at the root
		}

		FTraitEventList& InputEventList = UpdateGraphContext.GetInputEventList();
		FTraitEventList& OutputEventList = UpdateGraphContext.GetOutputEventList();
		const float DeltaTime = UpdateGraphContext.GetDeltaTime();

		FUpdateTraversalContext TraversalContext;
		TraversalContext.InputEventList = &InputEventList;
		TraversalContext.OutputEventList = &OutputEventList;

		FMemStack& MemStack = TraversalContext.GetMemStack();
		FMemMark Mark(MemStack);

		FChildrenArray Children;
		TTraitBinding<IHierarchy> HierarchyTrait;
		TTraitBinding<IUpdateTraversal> UpdateTraversalTrait;
		FTraitBinding TraitBinding;

		FUpdateTraversalQueue TraversalQueue(TraversalContext);

		Private::FUpdateEventBookkeepingList RootParentBookkeepingEntryList;
		TraversalContext.RootParentBookkeepingEntryList = &RootParentBookkeepingEntryList;

		const FTraitUpdateState RootState =
			FTraitUpdateState(DeltaTime)
			.AsNewlyRelevant(!GraphInstance.HasUpdated());

		// Mark the graph instance itself as updated
		GraphInstance.MarkAsUpdated();

		// Before we start the traversal, we give the graph instance components the chance to do some work
		TraversalContext.BindTo(GraphInstance);
		TraversalContext.ForEachComponent([&TraversalContext](FUAFGraphInstanceComponent& InComponent)
		{
			RaiseTraitEvents(TraversalContext, InComponent, *TraversalContext.InputEventList);

			InComponent.PreUpdate(TraversalContext);
			return true;
		});

		// Add the graph root to start the update process
		Private::FUpdateEntry RootEntry(GraphInstance.GetGraphRootPtr(), RootState);
		TraversalContext.PushUpdateEntry(&RootEntry);

#if DO_UAF_GRAPH_LOOPGUARD
		int32 ActiveUpdateLoopCount = 0;
#endif		
		while (Private::FUpdateEntry* Entry = TraversalContext.PopUpdateEntry())
		{
			TraversalContext.ExecutingEntry = Entry;

			const FWeakTraitPtr& EntryTraitPtr = Entry->TraitPtr;

			if (!Entry->bHasPreUpdated)
			{
				// This is the first time we visit this node, time to pre-update

				// Bind and cache our trait stack
				ensure(TraversalContext.GetStack(EntryTraitPtr, Entry->TraitStack));

				// First, if it has latent pins, we must execute and cache their results
				// This will ensure that other calls into this node will have a consistent view of
				// what the node saw when it started to update. We thus take a snapshot.
				// When a trait stack is blending out, its properties are frozen by default
				// unless a property opts to always update regardless.
				const bool bIsFrozen = Entry->TraitState.IsBlendingOut();
				const bool bJustBecameRelevant = Entry->TraitState.IsNewlyRelevant();
				Entry->TraitStack.SnapshotLatentProperties(DeltaTime, bIsFrozen, bJustBecameRelevant);

				const bool bImplementsIUpdate = Entry->TraitStack.GetInterface(Entry->UpdateTrait);

				// Before we PreUpdate, signal that we became newly relevant
				if (bImplementsIUpdate && Entry->TraitState.IsNewlyRelevant())
				{
					Entry->UpdateTrait.OnBecomeRelevant(TraversalContext, Entry->TraitState);
				}

				// Raise our input events
				Private::RaiseTraitEvents(TraversalContext, Entry, *TraversalContext.InputEventList);

				// Main update before our children
				if (bImplementsIUpdate)
				{
					Entry->UpdateTrait.PreUpdate(TraversalContext, Entry->TraitState);
				}

				// Make sure that next time we visit this entry, we'll post-update
				Entry->bHasPreUpdated = true;

				// Push this entry onto the update stack, we'll call it once all our children have finished executing
				TraversalContext.PushUpdateEntry(Entry);

				// Now visit the trait stack and queue our children
				ensure(Entry->TraitStack.GetTopTrait(TraitBinding));
				do
				{
					if (TraitBinding.AsInterface(UpdateTraversalTrait))
					{
						// Request that the trait queues the children it wants to visit
						// This is a separate function from PreUpdate to simplify traversal management. It is often the case that
						// the base trait is the one best placed to figure out how to optimally queue children since it
						// owns the handles to them. However, if an additive trait wishes to override PreUpdate, it might want
						// to perform logic after the base PreUpdate but before children are queued. Without a separate function,
						// we would have to rewrite the base PreUpdate entirely and use IHierarchy to query the handles of our children.
						UpdateTraversalTrait.QueueChildrenForTraversal(TraversalContext, Entry->TraitState, TraversalQueue);


						// Iterate over our queued children and push them onto the update stack
						// We do this to allow children to be queued in traversal order which is intuitive
						// but to traverse them in that order, they must be pushed in reverse order onto the update stack
						TraversalContext.PushQueuedUpdateEntries(TraversalQueue, Entry);
					}
					else if (TraitBinding.AsInterface(HierarchyTrait))
					{
						HierarchyTrait.GetChildren(TraversalContext, Children);

						// Append our children in reserve order so that they are visited in the same order they were added
						for (int32 ChildIndex = Children.Num() - 1; ChildIndex >= 0; --ChildIndex)
						{
							const FWeakTraitPtr& ChildPtr = Children[ChildIndex];
							if (ChildPtr.IsValid())
							{
								Private::FUpdateEntry* ChildEntry = TraversalContext.GetNewEntry(ChildPtr, Entry->TraitState);
								ChildEntry->ParentEntry = Entry;
								TraversalContext.PushUpdateEntry(ChildEntry);
							}
						}

						// Reset our container for the next time we need it
						Children.Reset();
					}
					else
					{
						// We don't have any children since we don't implement any of the relevant interfaces
					}
				} while (Entry->TraitStack.GetParentTrait(TraitBinding, TraitBinding));
			}
			else
			{
				// Execute event bookkeeping actions
				TraversalContext.ExecuteBookkeepingActions(Entry->EventBookkeepingList);

				// Raise our output events
				Private::RaiseTraitEvents(TraversalContext, Entry, *TraversalContext.OutputEventList);

				// We've already visited this node once, time to PostUpdate
				if (Entry->UpdateTrait.IsValid())
				{
					Entry->UpdateTrait.PostUpdate(TraversalContext, Entry->TraitState);
				}

				// Now that it finished updating, we can pop any scoped interfaces this node might have pushed
				TraversalContext.PopStackScopedInterfaces(Entry->TraitStack);

				// We don't need this entry anymore
				TraversalContext.PushFreeEntry(Entry);
			}
			
#if DO_UAF_GRAPH_LOOPGUARD
			// Check for loop count and break out if a potential endless loop is detected
			++ActiveUpdateLoopCount;
			if (!ensureMsgf(ActiveUpdateLoopCount < GMaxUAFGraphUpdateLoopCount, 
				TEXT("Potential endless loop detected in Graph [%s] of System [%s] - will break out of the update loop"),
				 *GraphInstance.GetAssetName().ToString(), GraphInstance.GetModuleInstance() ? *GraphInstance.GetModuleInstance()->GetAssetName().ToString() : TEXT("NONE")))
			{
				break;
			}
#endif	
		}

		// Clear our executing entry
		TraversalContext.ExecutingEntry = nullptr;

		// Executing any bookkeeping our root node might need
		TraversalContext.ExecuteBookkeepingActions(*TraversalContext.RootParentBookkeepingEntryList);

		// After we finish the traversal, we give the graph instance components the chance to do some work
		TraversalContext.ForEachComponent([&TraversalContext](FUAFGraphInstanceComponent& InComponent)
		{
			RaiseTraitEvents(TraversalContext, InComponent, *TraversalContext.OutputEventList);

			InComponent.PostUpdate(TraversalContext);
			return true;
		});

		// At this point, we shouldn't have any remaining scoped interfaces
		// If this fails, it means we failed to pop them due to a push/pop mismatch
		ensure(!TraversalContext.HasScopedInterfaces());
	}
}
