// Copyright Epic Games, Inc. All Rights Reserved.


#include "Traits/ApplyAdditivePerBoneTrait.h"

#include "EvaluationVM/Tasks/ApplyAdditiveKeyframePerBone.h"
#include "EvaluationVM/Tasks/ConvertRotationsToLocalSpace.h"
#include "EvaluationVM/Tasks/ConvertRotationsToMeshSpace.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"
#include "UAF/BlendMask/UAFBlendMask.h"

namespace UE::UAF
{
	
AUTO_REGISTER_ANIM_TRAIT(FApplyAdditivePerBoneTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
GeneratorMacro(IContinuousBlend) \
GeneratorMacro(IEvaluate) \
GeneratorMacro(IHierarchy) \
GeneratorMacro(IUpdate) \
GeneratorMacro(IUpdateTraversal) \
GeneratorMacro(IGarbageCollection) \

GENERATE_ANIM_TRAIT_IMPLEMENTATION(FApplyAdditivePerBoneTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR
	
void FApplyAdditivePerBoneTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding,	const FTraitUpdateState& TraitState) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	
	TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
	Binding.GetStackInterface(ContinuousBlendTrait);
	
	const TObjectPtr<UUAFBlendMask> BlendMask = SharedData->GetBlendMask(Binding);
	if (!InstanceData->Base.IsValid())
	{
		InstanceData->Base = Context.AllocateNodeInstance(Binding, SharedData->Base);
	}
	else
	{
		InstanceData->bWasBaseRelevant = true;
	}
	
	const float AdditiveWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
	if (FAnimWeight::IsRelevant(AdditiveWeight) && BlendMask)
	{
		if (!InstanceData->Additive.IsValid())
		{
			// We need to blend a child that isn't instanced yet, allocate it
			InstanceData->Additive = Context.AllocateNodeInstance(Binding, SharedData->Additive);
		}
		else
		{
			InstanceData->bWasAdditiveRelevant = true;
		}
	}
	else
	{
		InstanceData->bWasAdditiveRelevant = false;
		InstanceData->Additive.Reset();
	}
}


void FApplyAdditivePerBoneTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	
	const UUAFBlendMask* BlendMask = SharedData->GetBlendMask(Binding);

	if (InstanceData->Base.IsValid() && InstanceData->Additive.IsValid() && BlendMask)
	{
		TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
		Binding.GetStackInterface(ContinuousBlendTrait);
		
		const float AdditiveWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
		const bool bBlendInMeshSpace = SharedData->GetbMeshSpaceBlend(Binding);

		if (bBlendInMeshSpace)
		{
			Context.AppendTask(FAnimNextConvertRotationsToMeshSpaceTask::Make(2));
		}

		Context.AppendTask(FUAFApplyAdditiveKeyframeWithBlendMaskTask::Make(AdditiveWeight, BlendMask));
		Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());

		if (bBlendInMeshSpace)
		{
			Context.AppendTask(FAnimNextConvertRotationsToLocalSpaceTask::Make(1));
		}
	}
	else
	{
		// We have only one child that is active, do nothing
	}
}
	
void FApplyAdditivePerBoneTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding,
	const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	
	const UUAFBlendMask* BlendMask = SharedData->GetBlendMask(Binding);
	
	TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
	Binding.GetStackInterface(ContinuousBlendTrait);

	const float BaseWeight = ContinuousBlendTrait.GetBlendWeight(Context, 0);
	const float AdditiveWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
	if (InstanceData->Base.IsValid())
	{
		TraversalQueue.Push(InstanceData->Base, TraitState.WithWeight(BaseWeight).AsNewlyRelevant(!InstanceData->bWasBaseRelevant));
	}

	if (InstanceData->Additive.IsValid() && FAnimWeight::IsRelevant(AdditiveWeight) && BlendMask)
	{
		TraversalQueue.Push(InstanceData->Additive, TraitState.WithWeight(AdditiveWeight).AsNewlyRelevant(!InstanceData->bWasAdditiveRelevant));
	}
}


uint32 FApplyAdditivePerBoneTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
{
	return 2;
}

void FApplyAdditivePerBoneTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
{
	const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	// Add the two children, even if the handles are empty
	Children.Add(InstanceData->Base);
	Children.Add(InstanceData->Additive);
}

float FApplyAdditivePerBoneTrait::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	float Alpha = FMath::Clamp(SharedData->GetAlpha(Binding), 0.0f, 1.0f);
	if (SharedData->GetBlendMask(Binding) == nullptr)
	{
		Alpha = 0.0f;
	}

	if (ChildIndex == 0)
	{
		return 1.0f - Alpha;
	}
	
	if (ChildIndex == 1)
	{
		return Alpha;
	}

	// Invalid child index
	return -1.0f;
}

void FApplyAdditivePerBoneTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
{
	IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);
	
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	TObjectPtr<UUAFBlendMask> BlendMask = SharedData->GetBlendMask(Binding);

	if (BlendMask)
	{
		Collector.AddReferencedObject(BlendMask);
	}
}
	
}
