// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HierarchyTableBlendProfile.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/ApplyAdditivePerBoneTraitData.h"


namespace UE::UAF
{
	/**
	 * FApplyAdditivePerBone Trait
	 * 
	 * A trait that can apply an additive animation with a hierarchy table based mask
	 */
	struct FApplyAdditivePerBoneTrait : FBaseTrait, IEvaluate, IUpdate, IUpdateTraversal, IHierarchy, IContinuousBlend, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FApplyAdditivePerBoneTrait, FBaseTrait)

		using FSharedData = FApplyAdditivePerBoneTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Base;
			FTraitPtr Additive;

			bool bWasBaseRelevant = false;
			bool bWasAdditiveRelevant = false;
		};


		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IContinuousBlend impl
		virtual float GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const override;
		
		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};
}
