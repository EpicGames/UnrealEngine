// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/ApplyAdditiveTrait.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/ApplyAdditiveKeyframe.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FApplyAdditiveTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \
		GeneratorMacro(IContinuousBlend) \

	// Note: We don't require IAlphaInputArgs, as an IContinuousBlend can override our alpha
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FApplyAdditiveTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FApplyAdditiveTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		check(!Base.IsValid());
		Base = Context.AllocateNodeInstance(Binding, SharedData->Base);
	}

	void FApplyAdditiveTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		if (InstanceData->Additive.IsValid())
		{
			TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
			Binding.GetStackInterface(ContinuousBlendTrait);

			const float Alpha = ContinuousBlendTrait.GetBlendWeight(Context, 1);

			// @TODO: Note in review, we could make this dissapear with some templating, But that approach doesn't scale well.
			// Whenver the task may be very custom and have multiple considerations other than alpha. Expect consumers to perform this logic.
			TTraitBinding<IAlphaInputArgs> AlphaInputArgsTrait;
			if (Binding.GetStackInterface(AlphaInputArgsTrait) && AlphaInputArgsTrait.GetAlphaInputType(Context) == EAnimAlphaInputType::Curve)
			{
				// Assume weight for additive is on the base, not additive itself
				static constexpr int32 AlphaSourceInputKeyframeIndex = 0;
				Context.AppendTask(FAnimNextApplyAdditiveKeyframeTask::Make(AlphaInputArgsTrait.GetAlphaCurveName(Context), AlphaSourceInputKeyframeIndex, AlphaInputArgsTrait.GetInputScaleBiasClampCallback(Context)));
			}
			else
			{
				Context.AppendTask(FAnimNextApplyAdditiveKeyframeTask::Make(Alpha));
			}
		}
		else
		{
			// Additive is not is active, do nothing
		}
	}

	void FApplyAdditiveTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
		Binding.GetStackInterface(ContinuousBlendTrait);

		const float Alpha = ContinuousBlendTrait.GetBlendWeight(Context, 1);
		if (FAnimWeight::IsRelevant(Alpha))
		{
			if (!InstanceData->Additive.IsValid())
			{
				// We need an additive child that isn't instanced yet, allocate it
				InstanceData->Additive = Context.AllocateNodeInstance(Binding, SharedData->Additive);
			}
			else
			{
				InstanceData->bWasAdditiveRelevant = true;
			}
		}
		else
		{
			// We no longer need this child, release it
			InstanceData->Additive.Reset();
			InstanceData->bWasAdditiveRelevant = false;
		}
	}

	void FApplyAdditiveTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		// Note: No need to manage newly relevant for incoming TraitState, will be set as need by caller. See IUpdate.cpp.
		TraversalQueue.Push(InstanceData->Base, TraitState);

		if (InstanceData->Additive.IsValid())
		{
			TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
			Binding.GetStackInterface(ContinuousBlendTrait);

			const float Alpha = ContinuousBlendTrait.GetBlendWeight(Context, 1);
			TraversalQueue.Push(InstanceData->Additive, TraitState.WithWeight(Alpha).AsNewlyRelevant(!InstanceData->bWasAdditiveRelevant));
		}
	}

	uint32 FApplyAdditiveTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 2;
	}

	void FApplyAdditiveTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		Children.Add(InstanceData->Base);
		Children.Add(InstanceData->Additive);
	}

	float FApplyAdditiveTrait::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		float Alpha = 1.0f;
		TTraitBinding<IAlphaInputArgs> AlphaInputArgsTrait;
		if (Binding.GetStackInterface(AlphaInputArgsTrait))
		{
			Alpha = AlphaInputArgsTrait.GetCurrentAlphaValue(Context);
		}
		else
		{
			// @TODO: Fallback path, to be removed
			Alpha = SharedData->GetAlpha(Binding);
		}

		if (ChildIndex == 0)
		{
			// Base is always full weight
			return 1.0f;
		}
		else if (ChildIndex == 1)
		{
			return Alpha;
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}

}  // UE::UAF
