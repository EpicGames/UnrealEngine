// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/ApplyAdditiveTraitData.h"

namespace UE::UAF
{
	/**
	 * A trait that can apply a mesh or local space additive to this trait stack.
	 * 
	 * Ex: LookAt's, minor hit reactions, up-down floating, etc.
	 */
	struct FApplyAdditiveTrait : FBaseTrait, IEvaluate, IUpdate, IUpdateTraversal, IHierarchy, IContinuousBlend
	{
		DECLARE_ANIM_TRAIT(FApplyAdditiveTrait, FBaseTrait)

		using FSharedData = FAnimNextApplyAdditiveTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			/** Reference to base for additive we are applying */
			FTraitPtr Base;

			/** Reference to additive we are applying */
			FTraitPtr Additive;

			/** True if Additive branch has any contribution (Alpha non-zero)  */
			bool bWasAdditiveRelevant = false;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IContinuousBlend impl
		virtual float GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const override;
	};
}
