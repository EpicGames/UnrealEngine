// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/ApplyNamedSetTrait.h"

#include "EvaluationVM/Tasks/ApplyNamedSetTask.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FApplyNamedSetTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FApplyNamedSetTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FApplyNamedSetTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (!InstanceData->Input.IsValid())
		{
			InstanceData->Input = Context.AllocateNodeInstance(Binding, SharedData->Input);
		}
	}

	void FApplyNamedSetTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FName SetName = SharedData->GetSetName(Binding);

		Context.AppendTask(FPushNamedSetTask::Make(SetName));

		IEvaluate::PreEvaluate(Context, Binding);
	}

	void FApplyNamedSetTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FName SetName = SharedData->GetSetName(Binding);

		Context.AppendTask(FPopNamedSetTask::Make(SetName));
	}

	uint32 FApplyNamedSetTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 1;
	}

	void FApplyNamedSetTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the child, even if the handle is empty
		Children.Add(InstanceData->Input);
	}
}
