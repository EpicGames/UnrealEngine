// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "Traits/ApplyNamedSetTraitData.h"

namespace UE::UAF
{
	/**
	 * FApplyNamedSetTrait
	 *
	 * A trait that applies a new named set to evaluate with.
	 * Traits upstream will use the new named set to evaluate.
	 * The trait that consumes its output must handle mismatched named sets (e.g. layering).
	 */
	struct FApplyNamedSetTrait : FBaseTrait, IEvaluate, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FApplyNamedSetTrait, FBaseTrait)

		using FSharedData = FApplyNamedSetSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Input;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
	};
}
