// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/Args/AlphaInputArgCoreTrait.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

namespace UE::UAF
{

namespace AlphaInput
{

float ComputeAlphaValueForFloat(FInputScaleBiasClamp& InAlphaScaleBiasClamp
	, const FInputScaleBias& InAlphaScaleBias
	, const float InBaseAlpha
	, const float InDeltaTime)
{
	float Result = InAlphaScaleBias.ApplyTo(InAlphaScaleBiasClamp.ApplyTo(InBaseAlpha, InDeltaTime));
	return FMath::Clamp(Result, 0.0f, 1.0f);
}

float ComputeAlphaValueForBool(FInputAlphaBoolBlend& InAlphaBoolBlend
	, const bool bInAlphaBoolEnabled
	, const float InDeltaTime)
{
	float Result = InAlphaBoolBlend.ApplyTo(bInAlphaBoolEnabled, InDeltaTime);
	return FMath::Clamp(Result, 0.0f, 1.0f);
}

float ComputeAlphaValueForType(const EAnimAlphaInputType InAlphaInputType
	, FInputScaleBiasClamp& InAlphaScaleBiasClamp
	, const FInputScaleBias& InAlphaScaleBias
	, const float InBaseAlpha
	, FInputAlphaBoolBlend& InAlphaBoolBlend
	, const bool bInAlphaBoolEnabled
	, const float InDeltaTime)
{
	float Result = 0.f;

	switch (InAlphaInputType)
	{
		case EAnimAlphaInputType::Float:
		{
			Result = AlphaInput::ComputeAlphaValueForFloat(IN OUT InAlphaScaleBiasClamp, InAlphaScaleBias, InBaseAlpha, InDeltaTime);
			break;
		}
		case EAnimAlphaInputType::Bool:
		{
			Result = AlphaInput::ComputeAlphaValueForBool(IN OUT InAlphaBoolBlend, bInAlphaBoolEnabled, InDeltaTime);
			break;
		}
		case EAnimAlphaInputType::Curve:
		{
			Result = 1.0; // This will be calculated on the task when reading the curve, but we need full weight for branches
			break;
		}
	};

	return Result;
}

} // namespace AlphaInput


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAlphaInputArgCoreTrait

AUTO_REGISTER_ANIM_TRAIT(FAlphaInputArgCoreTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IAlphaInputArgs) \
	GeneratorMacro(IUpdate) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FAlphaInputArgCoreTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

void FAlphaInputArgCoreTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	ComputedAlphaValue = SharedData->GetAlpha(Binding);
	AlphaBoolBlend = SharedData->GetAlphaBoolBlend(Binding);
	AlphaScaleBiasClamp = SharedData->AlphaScaleBiasClamp;
}

void FAlphaInputArgCoreTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->ComputedAlphaValue = AlphaInput::ComputeAlphaValueForType(SharedData->GetAlphaInputType(Binding)
		, IN OUT InstanceData->AlphaScaleBiasClamp
		, SharedData->AlphaScaleBias
		, SharedData->GetAlpha(Binding)
		, IN OUT InstanceData->AlphaBoolBlend
		, SharedData->GetbAlphaBoolEnabled(Binding)
		, TraitState.GetDeltaTime());

	InstanceData->AlphaBoolBlend.Reinitialize();
	InstanceData->AlphaScaleBiasClamp.Reinitialize();
}

void FAlphaInputArgCoreTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	// Store DeltaTime for deferred curve sampling
	InstanceData->DeltaTime = TraitState.GetDeltaTime();
	InstanceData->ComputedAlphaValue = AlphaInput::ComputeAlphaValueForType(SharedData->GetAlphaInputType(Binding)
		, IN OUT InstanceData->AlphaScaleBiasClamp
		, SharedData->AlphaScaleBias
		, SharedData->GetAlpha(Binding)
		, IN OUT InstanceData->AlphaBoolBlend
		, SharedData->GetbAlphaBoolEnabled(Binding)
		, TraitState.GetDeltaTime());

	IUpdate::PreUpdate(Context, Binding, TraitState);
}

FAlphaInputTraitArgs FAlphaInputArgCoreTrait::Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	FAlphaInputTraitArgs Result = FAlphaInputTraitArgs();
	Result.Alpha = SharedData->GetAlpha(Binding);
	Result.AlphaScaleBias = SharedData->AlphaScaleBias;
	Result.AlphaScaleBiasClamp = InstanceData->AlphaScaleBiasClamp;
	Result.bAlphaBoolEnabled = SharedData->GetbAlphaBoolEnabled(Binding);
	Result.AlphaBoolBlend = InstanceData->AlphaBoolBlend;
	Result.AlphaCurveName = SharedData->GetAlphaCurveName(Binding);
	Result.AlphaInputType = SharedData->GetAlphaInputType(Binding);

	return Result;
}

EAnimAlphaInputType FAlphaInputArgCoreTrait::GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	return SharedData->GetAlphaInputType(Binding);
}

FName FAlphaInputArgCoreTrait::GetAlphaCurveName(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	return SharedData->GetAlphaCurveName(Binding);
}

float FAlphaInputArgCoreTrait::GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	return InstanceData->ComputedAlphaValue;
}

TFunction<float(float)> FAlphaInputArgCoreTrait::GetInputScaleBiasClampCallback(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	auto InputBiasClampCallback = [DeltaTime = InstanceData->DeltaTime, InputScaleBiasClamp = &InstanceData->AlphaScaleBiasClamp](float Alpha) -> float
	{
		return InputScaleBiasClamp->ApplyTo(Alpha, DeltaTime);
	};

	return InputBiasClampCallback;
}


} // namespace UE::UAF

