// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

namespace UE::UAF
{
	namespace AlphaInput
	{
		extern UAFANIMGRAPH_API float ComputeAlphaValueForFloat(FInputScaleBiasClamp& InAlphaScaleBiasClamp
			, const FInputScaleBias& InAlphaScaleBias
			, const float InBaseAlpha
			, const float InDeltaTime);

		extern UAFANIMGRAPH_API float ComputeAlphaValueForBool(FInputAlphaBoolBlend& InAlphaBoolBlend
			, const bool bInAlphaBoolEnabled
			, const float InDeltaTime);

		extern UAFANIMGRAPH_API float ComputeAlphaValueForType(const EAnimAlphaInputType InAlphaInputType
			, FInputScaleBiasClamp& InAlphaScaleBiasClamp
			, const FInputScaleBias& InAlphaScaleBias
			, const float InBaseAlpha
			, FInputAlphaBoolBlend& InAlphaBoolBlend
			, const bool bInAlphaBoolEnabled
			, const float InDeltaTime);

	} // namespace AlphaInput

	/**
	 * FAlphaInputArgCoreTrait
	 *
	 * Additive trait that provides configurable alpha input args for another trait to use. Ex: as a weight.
	 * This trait's API calls mutate internal state, so it should not be used by multiple traits concurrently.
	 * 
	 * In particular, curve evaluation will not make sense as curves may differ per trait keyframe & DeltaTime
	 * will be updated multiple per consumer trait.
	 * 
	 * Note: Does not implement IContinousBlend as cannot resolve trait-specific child index handling.
	 */
	struct FAlphaInputArgCoreTrait : FAdditiveTrait, IUpdate, IAlphaInputArgs
	{
		DECLARE_ANIM_TRAIT(FAlphaInputArgCoreTrait, FAdditiveTrait)

		using FSharedData = FAlphaInputTraitArgs;
		
		struct FInstanceData : FTrait::FInstanceData
		{
			FInputAlphaBoolBlend AlphaBoolBlend;
			FInputScaleBiasClamp AlphaScaleBiasClamp;

			float DeltaTime = 0.f;
			float ComputedAlphaValue = 0.f;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IUpdate
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IAlphaInputArgs
		virtual FAlphaInputTraitArgs Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual FName GetAlphaCurveName(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual float GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual TFunction<float(float)> GetInputScaleBiasClampCallback(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
	};

} // namespace UE::UAF
