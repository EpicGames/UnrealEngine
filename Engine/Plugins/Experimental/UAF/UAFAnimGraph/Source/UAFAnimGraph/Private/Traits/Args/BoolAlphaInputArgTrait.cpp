// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/Args/BoolAlphaInputArgTrait.h"
#include "Traits/Args/AlphaInputArgCoreTrait.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoolAlphaInputArgTrait)

namespace UE::UAF
{

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FBoolAlphaInputArgTrait

AUTO_REGISTER_ANIM_TRAIT(FBoolAlphaInputArgTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IAlphaInputArgs) \
	GeneratorMacro(IUpdate) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBoolAlphaInputArgTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

void FBoolAlphaInputArgTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	AlphaBoolBlend = SharedData->GetAlphaBoolBlend(Binding);
}

void FBoolAlphaInputArgTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->ComputedAlphaValue = AlphaInput::ComputeAlphaValueForBool(IN OUT InstanceData->AlphaBoolBlend, SharedData->GetbAlphaBoolEnabled(Binding), TraitState.GetDeltaTime());
	
	InstanceData->AlphaBoolBlend.Reinitialize();
}

void FBoolAlphaInputArgTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->ComputedAlphaValue = AlphaInput::ComputeAlphaValueForBool(IN OUT InstanceData->AlphaBoolBlend, SharedData->GetbAlphaBoolEnabled(Binding), TraitState.GetDeltaTime());

	IUpdate::PreUpdate(Context, Binding, TraitState);
}

FAlphaInputTraitArgs FBoolAlphaInputArgTrait::Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	FAlphaInputTraitArgs Result = FAlphaInputTraitArgs();
	// Result.Alpha = InstanceData->InternalAlpha;
	// Result.AlphaScaleBias = SharedData->AlphaScaleBias;
	// Result.AlphaScaleBiasClamp = InstanceData->AlphaScaleBiasClamp;
	Result.bAlphaBoolEnabled = SharedData->GetbAlphaBoolEnabled(Binding);
	Result.AlphaBoolBlend = InstanceData->AlphaBoolBlend;
	// Result.AlphaCurveName = SharedData->GetAlphaCurveName(Binding);
	Result.AlphaInputType = EAnimAlphaInputType::Bool;

	return Result;
}

EAnimAlphaInputType FBoolAlphaInputArgTrait::GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	return EAnimAlphaInputType::Bool;
}

float FBoolAlphaInputArgTrait::GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	return InstanceData->ComputedAlphaValue;
}

} // namespace UE::UAF

