// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "BoolAlphaInputArgTrait.generated.h"

USTRUCT(meta = (DisplayName = "Bool Alpha Input Args"))
struct FBoolAlphaInputTraitArgs : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "bEnabled"))
	bool bAlphaBoolEnabled = true;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, DisplayName = "Blend Settings"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(bAlphaBoolEnabled) \
		GeneratorMacro(AlphaBoolBlend) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FBoolAlphaInputTraitArgs, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	/**
	 * FBoolAlphaInputArgTrait
	 *
	 * See 'FAlphaInputArgCoreTrait'. Bool input only version.
	 */
	struct FBoolAlphaInputArgTrait : FAdditiveTrait, IUpdate, IAlphaInputArgs
	{
		DECLARE_ANIM_TRAIT(FBoolAlphaInputArgTrait, FAdditiveTrait)

		using FSharedData = FBoolAlphaInputTraitArgs;

		struct FInstanceData : FTrait::FInstanceData
		{
			FInputAlphaBoolBlend AlphaBoolBlend;

			float ComputedAlphaValue = 0.f;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IUpdate
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IAlphaInputArgs
		virtual FAlphaInputTraitArgs Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual float GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
	};

} // namespace UE::UAF
