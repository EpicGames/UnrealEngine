// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/Args/CurveAlphaInputArgTrait.h"
#include "Traits/Args/AlphaInputArgCoreTrait.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveAlphaInputArgTrait)

namespace UE::UAF
{

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FCurveAlphaInputArgTrait

AUTO_REGISTER_ANIM_TRAIT(FCurveAlphaInputArgTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IAlphaInputArgs) \
	GeneratorMacro(IUpdate) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FCurveAlphaInputArgTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

void FCurveAlphaInputArgTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	AlphaScaleBiasClamp = SharedData->AlphaScaleBiasClamp;
}

void FCurveAlphaInputArgTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->AlphaScaleBiasClamp.Reinitialize();
}

void FCurveAlphaInputArgTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->DeltaTime = TraitState.GetDeltaTime();

	IUpdate::PreUpdate(Context, Binding, TraitState);
}

FAlphaInputTraitArgs FCurveAlphaInputArgTrait::Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	FAlphaInputTraitArgs Result = FAlphaInputTraitArgs();
	// Alpha = InstanceData->InternalAlpha;
	// Result.AlphaScaleBias = SharedData->AlphaScaleBias;
	Result.AlphaScaleBiasClamp = InstanceData->AlphaScaleBiasClamp;
	// Result.bAlphaBoolEnabled = SharedData->GetbAlphaBoolEnabled(Binding);
	// Result.AlphaBoolBlend = InstanceData->AlphaBoolBlend;
	Result.AlphaCurveName = SharedData->GetAlphaCurveName(Binding);
	Result.AlphaInputType = EAnimAlphaInputType::Curve;

	return Result;
}

EAnimAlphaInputType FCurveAlphaInputArgTrait::GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	return EAnimAlphaInputType::Curve;
}

FName FCurveAlphaInputArgTrait::GetAlphaCurveName(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	return SharedData->GetAlphaCurveName(Binding);
}

TFunction<float(float)> FCurveAlphaInputArgTrait::GetInputScaleBiasClampCallback(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	
	auto InputBiasClampCallback = [DeltaTime = InstanceData->DeltaTime, InputScaleBiasClamp = &InstanceData->AlphaScaleBiasClamp](float Alpha) -> float
	{
		return InputScaleBiasClamp->ApplyTo(Alpha, DeltaTime);
	};

	return InputBiasClampCallback;
}

} // namespace UE::UAF

