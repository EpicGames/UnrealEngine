// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "CurveAlphaInputArgTrait.generated.h"

USTRUCT(meta = (DisplayName = "Curve Alpha Input Args"))
struct FCurveAlphaInputTraitArgs : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, DisplayAfter = "AlphaCurveName"))
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	/** Note: For curve types, the additive branch will always be evaluated. Curve weight blending will occur at task evaluation time. */
	UPROPERTY(EditAnywhere, Category = Settings)
	FName AlphaCurveName = NAME_None;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(AlphaCurveName) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FCurveAlphaInputTraitArgs, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

	/**
	 * FCurveAlphaInputArgTrait
	 *
	 * See 'FAlphaInputArgCoreTrait'. Curve input only version.
	 */
	struct FCurveAlphaInputArgTrait : FAdditiveTrait, IUpdate, IAlphaInputArgs
	{
		DECLARE_ANIM_TRAIT(FCurveAlphaInputArgTrait, FAdditiveTrait)

		using FSharedData = FCurveAlphaInputTraitArgs;
		
		struct FInstanceData : FTrait::FInstanceData
		{
			FInputScaleBiasClamp AlphaScaleBiasClamp;

			float DeltaTime = 0.f;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IUpdate
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IAlphaInputArgs
		virtual FAlphaInputTraitArgs Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual FName GetAlphaCurveName(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual TFunction<float(float)> GetInputScaleBiasClampCallback(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
	};

} // namespace UE::UAF
