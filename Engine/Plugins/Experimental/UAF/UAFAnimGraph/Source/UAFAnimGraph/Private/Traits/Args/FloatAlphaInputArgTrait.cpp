// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/Args/FloatAlphaInputArgTrait.h"
#include "Traits/Args/AlphaInputArgCoreTrait.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatAlphaInputArgTrait)

namespace UE::UAF
{

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FFloatAlphaInputArgTrait

AUTO_REGISTER_ANIM_TRAIT(FFloatAlphaInputArgTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IAlphaInputArgs) \
	GeneratorMacro(IUpdate) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FFloatAlphaInputArgTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

void FFloatAlphaInputArgTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	ComputedAlphaValue = SharedData->GetAlpha(Binding);
	AlphaScaleBiasClamp = SharedData->AlphaScaleBiasClamp;
}

void FFloatAlphaInputArgTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->ComputedAlphaValue = AlphaInput::ComputeAlphaValueForFloat(IN OUT InstanceData->AlphaScaleBiasClamp, SharedData->AlphaScaleBias, SharedData->GetAlpha(Binding), TraitState.GetDeltaTime());

	InstanceData->AlphaScaleBiasClamp.Reinitialize();
}

void FFloatAlphaInputArgTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->ComputedAlphaValue = AlphaInput::ComputeAlphaValueForFloat(IN OUT InstanceData->AlphaScaleBiasClamp, SharedData->AlphaScaleBias, SharedData->GetAlpha(Binding), TraitState.GetDeltaTime());
	
	IUpdate::PreUpdate(Context, Binding, TraitState);
}

FAlphaInputTraitArgs FFloatAlphaInputArgTrait::Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	FAlphaInputTraitArgs Result = FAlphaInputTraitArgs();
	Result.Alpha = SharedData->GetAlpha(Binding);
	Result.AlphaScaleBias = SharedData->AlphaScaleBias;
	Result.AlphaScaleBiasClamp = InstanceData->AlphaScaleBiasClamp;
	// Result.bAlphaBoolEnabled = SharedData->GetbAlphaBoolEnabled(Binding);
	// Result.AlphaBoolBlend = InstanceData->AlphaBoolBlend;
	// Result.AlphaCurveName = SharedData->GetAlphaCurveName(Binding);
	Result.AlphaInputType = EAnimAlphaInputType::Float;

	return Result;
}

EAnimAlphaInputType FFloatAlphaInputArgTrait::GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	return EAnimAlphaInputType::Float;
}

float FFloatAlphaInputArgTrait::GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	return InstanceData->ComputedAlphaValue;
}

} // namespace UE::UAF

