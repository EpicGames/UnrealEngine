// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "FloatAlphaInputArgTrait.generated.h"

USTRUCT(meta = (DisplayName = "Float Alpha Input Args"))
struct FFloatAlphaInputTraitArgs : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	float Alpha = 1.f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden))
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden))
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FFloatAlphaInputTraitArgs, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

	/**
	 * FFloatAlphaInputArgTrait
	 *
	 * See 'FAlphaInputArgCoreTrait'. Float input only version.
	 */
	struct FFloatAlphaInputArgTrait : FAdditiveTrait, IUpdate, IAlphaInputArgs
	{
		DECLARE_ANIM_TRAIT(FFloatAlphaInputArgTrait, FAdditiveTrait)

		using FSharedData = FFloatAlphaInputTraitArgs;
		
		struct FInstanceData : FTrait::FInstanceData
		{
			FInputScaleBiasClamp AlphaScaleBiasClamp;

			float ComputedAlphaValue = 0.f;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IUpdate
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IAlphaInputArgs
		virtual FAlphaInputTraitArgs Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual float GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
	};

} // namespace UE::UAF
