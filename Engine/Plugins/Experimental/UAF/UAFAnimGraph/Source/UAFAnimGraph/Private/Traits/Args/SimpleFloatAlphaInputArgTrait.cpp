// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/Args/SimpleFloatAlphaInputArgTrait.h"
#include "Traits/Args/AlphaInputArgCoreTrait.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleFloatAlphaInputArgTrait)

namespace UE::UAF
{

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FSimpleFloatAlphaInputArgTrait

AUTO_REGISTER_ANIM_TRAIT(FSimpleFloatAlphaInputArgTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IAlphaInputArgs) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FSimpleFloatAlphaInputArgTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

FAlphaInputTraitArgs FSimpleFloatAlphaInputArgTrait::Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	FAlphaInputTraitArgs Result = FAlphaInputTraitArgs();
	Result.Alpha = SharedData->GetAlpha(Binding);
	// Result.AlphaScaleBias = SharedData->AlphaScaleBias;
	// Result.AlphaScaleBiasClamp = InstanceData->AlphaScaleBiasClamp;
	// Result.bAlphaBoolEnabled = SharedData->GetbAlphaBoolEnabled(Binding);
	// Result.AlphaBoolBlend = InstanceData->AlphaBoolBlend;
	// Result.AlphaCurveName = SharedData->GetAlphaCurveName(Binding);
	Result.AlphaInputType = EAnimAlphaInputType::Float;

	return Result;
}

EAnimAlphaInputType FSimpleFloatAlphaInputArgTrait::GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	return EAnimAlphaInputType::Float;
}

float FSimpleFloatAlphaInputArgTrait::GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	return SharedData->GetAlpha(Binding);
}

} // namespace UE::UAF

