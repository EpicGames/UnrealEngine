// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "SimpleFloatAlphaInputArgTrait.generated.h"

USTRUCT(meta = (DisplayName = "Simple Float Alpha Input Args"))
struct FSimpleFloatAlphaInputTraitArgs : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	float Alpha = 1.f;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FSimpleFloatAlphaInputTraitArgs, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

	/**
	 * FSimpleFloatAlphaInputArgTrait
	 *
	 * See 'FAlphaInputArgCoreTrait'. Simple Float input only version. Alpha is always updated.
	 */
	struct FSimpleFloatAlphaInputArgTrait : FAdditiveTrait, IAlphaInputArgs
	{
		DECLARE_ANIM_TRAIT(FSimpleFloatAlphaInputArgTrait, FAdditiveTrait)

		using FSharedData = FSimpleFloatAlphaInputTraitArgs;
		
		struct FInstanceData : FTrait::FInstanceData
		{
		};

		// IAlphaInputArgs
		virtual FAlphaInputTraitArgs Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
		virtual float GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const override;
	};

} // namespace UE::UAF
