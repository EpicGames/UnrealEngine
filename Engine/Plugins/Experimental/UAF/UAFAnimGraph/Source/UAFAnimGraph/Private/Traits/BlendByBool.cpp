// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendByBool.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendByBool)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendByBoolTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendByBoolTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	static constexpr int32 TRUE_CHILD_INDEX = 0;
	static constexpr int32 FALSE_CHILD_INDEX = 1;

	void FBlendByBoolTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// If we were previously relevant, update our status
		if (InstanceData->PreviousChildIndex != INDEX_NONE)
		{
			if (InstanceData->PreviousChildIndex == TRUE_CHILD_INDEX)
			{
				InstanceData->bWasTrueChildRelevant = true;
			}
			else
			{
				InstanceData->bWasFalseChildRelevant = true;
			}
		}

		const int32 DestinationChildIndex = DiscreteBlendTrait.GetBlendDestinationChildIndex(Context);
		if (InstanceData->PreviousChildIndex != DestinationChildIndex)
		{
			DiscreteBlendTrait.OnBlendTransition(Context, InstanceData->PreviousChildIndex, DestinationChildIndex);

			InstanceData->PreviousChildIndex = DestinationChildIndex;
		}
	}

	void FBlendByBoolTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// The destination child index has been updated in PreUpdate, we can use the cached version
		const int32 DestinationChildIndex = InstanceData->PreviousChildIndex;

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		const float BlendWeightTrue = DiscreteBlendTrait.GetBlendWeight(Context, TRUE_CHILD_INDEX);
		if (InstanceData->TrueChild.IsValid() && (SharedData->bAlwaysUpdateTrueChild || FAnimWeight::IsRelevant(BlendWeightTrue)))
		{
			FTraitUpdateState TraitStateTrue = TraitState
				.WithWeight(BlendWeightTrue)
				.AsBlendingOut(DestinationChildIndex != TRUE_CHILD_INDEX)
				.AsNewlyRelevant(!InstanceData->bWasTrueChildRelevant);

			TraversalQueue.Push(InstanceData->TrueChild, TraitStateTrue);
		}

		const float BlendWeightFalse = 1.0f - BlendWeightTrue;
		if (InstanceData->FalseChild.IsValid() && FAnimWeight::IsRelevant(BlendWeightFalse))
		{
			FTraitUpdateState TraitStateFalse = TraitState
				.WithWeight(BlendWeightFalse)
				.AsBlendingOut(DestinationChildIndex != FALSE_CHILD_INDEX)
				.AsNewlyRelevant(!InstanceData->bWasFalseChildRelevant);

			TraversalQueue.Push(InstanceData->FalseChild, TraitStateFalse);
		}
	}

	uint32 FBlendByBoolTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 2;
	}

	void FBlendByBoolTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the two children, even if the handles are empty
		Children.Add(InstanceData->TrueChild);
		Children.Add(InstanceData->FalseChild);
	}

	float FBlendByBoolTrait::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		const float DestinationChildIndex = DiscreteBlendTrait.GetBlendDestinationChildIndex(Context);

		if (ChildIndex == TRUE_CHILD_INDEX)
		{
			return (DestinationChildIndex == TRUE_CHILD_INDEX) ? 1.0f : 0.0f;
		}
		else if (ChildIndex == FALSE_CHILD_INDEX)
		{
			return (DestinationChildIndex == FALSE_CHILD_INDEX) ? 1.0f : 0.0f;
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}

	int32 FBlendByBoolTrait::GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		const bool bCondition = SharedData->GetbCondition(Binding);
		return bCondition ? TRUE_CHILD_INDEX : FALSE_CHILD_INDEX;
	}

	void FBlendByBoolTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// We initiate immediately when we transition
		DiscreteBlendTrait.OnBlendInitiated(Context, NewChildIndex);

		// We terminate immediately when we transition
		DiscreteBlendTrait.OnBlendTerminated(Context, OldChildIndex);
	}

	void FBlendByBoolTrait::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Allocate our new child instance
		if (!InstanceData->TrueChild.IsValid() && (ChildIndex == TRUE_CHILD_INDEX || SharedData->bAlwaysUpdateTrueChild))
		{
			InstanceData->TrueChild = Context.AllocateNodeInstance(Binding, SharedData->TrueChild);
		}
		
		if (!InstanceData->FalseChild.IsValid() && (ChildIndex == FALSE_CHILD_INDEX))
		{
			InstanceData->FalseChild = Context.AllocateNodeInstance(Binding, SharedData->FalseChild);
		}
	}

	void FBlendByBoolTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Deallocate our child instance
		if (ChildIndex == TRUE_CHILD_INDEX)
		{
			if (!SharedData->bAlwaysUpdateTrueChild)
			{
				InstanceData->TrueChild.Reset();
				InstanceData->bWasTrueChildRelevant = false;
			}
		}
		else if (ChildIndex == FALSE_CHILD_INDEX)
		{
			InstanceData->FalseChild.Reset();
			InstanceData->bWasFalseChildRelevant = false;
		}
	}
}
