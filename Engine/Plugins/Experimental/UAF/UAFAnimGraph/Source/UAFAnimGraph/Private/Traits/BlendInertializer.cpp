// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendInertializer.h"

#include "TraitCore/ExecutionContext.h"

#include "Traits/Inertialization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendInertializer)


namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendInertializerCoreTrait)
	AUTO_REGISTER_ANIM_TRAIT(FBlendInertializerTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(ISmoothBlend) \
	
	// Trait required interfaces implementation boilerplate
	#define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInertializerBlend) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendInertializerCoreTrait, TRAIT_INTERFACE_ENUMERATOR, TRAIT_REQUIRED_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR


	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInertializerBlend) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendInertializerTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FBlendInertializerCoreTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		// Trigger the new transition
		IDiscreteBlend::OnBlendTransition(Context, Binding, OldChildIndex, NewChildIndex);

		TTraitBinding<IInertializerBlend> InertializerBlendTrait;
		Binding.GetStackInterface(InertializerBlendTrait);

		const float BlendTime = InertializerBlendTrait.GetBlendTime(Context, NewChildIndex);
		if (BlendTime <= 0.0f)
		{
			return;	// No blend time means we are disabled
		}

		// Make Request Event
		TSharedPtr<FAnimNextInertializationRequestEvent> Event = MakeTraitEvent<FAnimNextInertializationRequestEvent>();
		Event->Request.BlendTime = BlendTime;
		Context.RaiseOutputTraitEvent(Event);
	}

	float FBlendInertializerCoreTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		TTraitBinding<IInertializerBlend> InertializerBlendTrait;
		Binding.GetStackInterface(InertializerBlendTrait);

		const float BlendTime = InertializerBlendTrait.GetBlendTime(Context, ChildIndex);
		if (BlendTime > 0.0f)
		{
			// We hijack the blend time and always transition instantaneously
			return 0.0f;
		}

		// We are disabled
		return ISmoothBlend::GetBlendTime(Context, Binding, ChildIndex);
	}

	float FBlendInertializerTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		return SharedData->BlendTime;
	}
}
