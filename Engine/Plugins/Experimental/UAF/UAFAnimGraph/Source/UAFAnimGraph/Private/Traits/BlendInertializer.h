// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IInertializerBlend.h"
#include "TraitInterfaces/ISmoothBlend.h"

#include "BlendInertializer.generated.h"

/**
 * A trait that converts a normal smooth blend into an inertializing blend.
 * It only implements the inertializing logic, it queries its required arguments using the IInertializerBlend interface.
 */
USTRUCT(meta = (DisplayName = "Blend Inertializer Core", ShowTooltip=true))
struct FAnimNextBlendInertializerCoreTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// This struct is empty but required so that we can give a nice display name to the trait
};

/**
 * A trait that converts a normal smooth blend into an inertializing blend.
 * This trait implements both the logic and contains the arguments necessary.
 */
USTRUCT(meta = (DisplayName = "Blend Inertializer", ShowTooltip=true))
struct FAnimNextBlendInertializerTraitSharedData : public FAnimNextBlendInertializerCoreTraitSharedData
{
	GENERATED_BODY()

	// Inertialization Blend Time
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	float BlendTime = 0.2f;
};

namespace UE::UAF
{
	/**
	 * FBlendInertializerCoreTrait
	 *
	 * A trait that converts a normal smooth blend into an inertializing blend.
	 * It only implements the inertializing logic, it queries its required arguments using the IInertializerBlend interface.
	 */
	struct FBlendInertializerCoreTrait : FAdditiveTrait, IDiscreteBlend, ISmoothBlend
	{
		DECLARE_ANIM_TRAIT(FBlendInertializerCoreTrait, FAdditiveTrait)

		using FSharedData = FAnimNextBlendInertializerCoreTraitSharedData;

		// IDiscreteBlend impl
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;

		// ISmoothBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return true; }
#endif
	};

	/**
	 * FBlendInertializerTrait
	 * 
	 * A trait that converts a normal smooth blend into an inertializing blend.
	 * This trait implements both the logic and contains the arguments necessary.
	 */
	struct FBlendInertializerTrait : FBlendInertializerCoreTrait, IInertializerBlend
	{
		DECLARE_ANIM_TRAIT(FBlendInertializerTrait, FBlendInertializerCoreTrait)

		using FSharedData = FAnimNextBlendInertializerTraitSharedData;

		// Unhide base function to silence override hiding warning with clang
		using FBlendInertializerCoreTrait::GetBlendTime;

		// IInertializerBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const override;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return false; } // overrdie the base, as it is hidden
#endif
	};
}
