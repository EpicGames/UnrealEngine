// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendLayer.h"

#include "AnimationRuntime.h"
#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"
#include "EvaluationVM/Tasks/ConvertRotationsToLocalSpace.h"
#include "EvaluationVM/Tasks/ConvertRotationsToMeshSpace.h"
#include "HierarchyTable.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "SkeletonHierarchyTableType.h"
#include "UAF/BlendMask/UAFBlendMask.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendLayerTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IContinuousBlend) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendLayerTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FBlendLayerTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const UUAFBlendMask* BlendMask = SharedData->GetBlendMask(Binding);

		if (InstanceData->ChildBase.IsValid() && InstanceData->ChildBlend.IsValid() && BlendMask)
		{
			// We have two children, interpolate them

			TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
			Binding.GetStackInterface(ContinuousBlendTrait);

			const USkeleton* Skeleton = BlendMask->GetSkeleton();

			const float BlendWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
			const bool bBlendInMeshSpace = SharedData->GetbBlendInMeshSpace(Binding);

			if (bBlendInMeshSpace)
			{
				Context.AppendTask(FAnimNextConvertRotationsToMeshSpaceTask::Make(2));
			}

			Context.AppendTask(FAnimNextBlendKeyframePerBoneWithScaleTask::Make(BlendMask, BlendWeight));
			Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());

			if (bBlendInMeshSpace)
			{
				Context.AppendTask(FAnimNextConvertRotationsToLocalSpaceTask::Make(1));
			}
		}
		else
		{
			// We have only one child that is active, do nothing
		}
	}

	void FBlendLayerTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
		Binding.GetStackInterface(ContinuousBlendTrait);

		const UUAFBlendMask* BlendMask = SharedData->GetBlendMask(Binding);

		if (!InstanceData->ChildBase.IsValid())
		{
			InstanceData->ChildBase = Context.AllocateNodeInstance(Binding, SharedData->ChildBase);
		}
		else
		{
			InstanceData->bWasChildBaseRelevant = true;
		}
		
		const float ChildWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
		if ((FAnimWeight::IsRelevant(ChildWeight) || SharedData->GetbAlwaysUpdateChildBlend(Binding)) && BlendMask)
		{
			if (!InstanceData->ChildBlend.IsValid())
			{
				// We need to blend a child that isn't instanced yet, allocate it
				InstanceData->ChildBlend = Context.AllocateNodeInstance(Binding, SharedData->ChildBlend);
			}
			else
			{
				InstanceData->bWasChildBlendRelevant = true;
			}
		}
		else
		{
			InstanceData->ChildBlend.Reset();
		}
	}

	void FBlendLayerTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
		Binding.GetStackInterface(ContinuousBlendTrait);

		const float BlendWeightBlend = ContinuousBlendTrait.GetBlendWeight(Context, 1);
		if (InstanceData->ChildBase.IsValid())
		{
			TraversalQueue.Push(InstanceData->ChildBase, TraitState.WithWeight(1.0f).AsNewlyRelevant(!InstanceData->bWasChildBaseRelevant));
		}

		if (InstanceData->ChildBlend.IsValid() && SharedData->GetBlendMask(Binding))
		{
			TraversalQueue.Push(InstanceData->ChildBlend, TraitState.WithWeight(BlendWeightBlend).AsNewlyRelevant(!InstanceData->bWasChildBlendRelevant));
		}
	}

	uint32 FBlendLayerTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 2;
	}

	void FBlendLayerTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the two children, even if the handles are empty
		Children.Add(InstanceData->ChildBase);
		Children.Add(InstanceData->ChildBlend);
	}

	float FBlendLayerTrait::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		const float BlendWeight = SharedData->GetBlendWeight(Binding);
		float ClampedWeight = FMath::Clamp(BlendWeight, 0.0f, 1.0f);
		if (!SharedData->GetBlendMask(Binding))
		{
			ClampedWeight = 0.0f;
		}

		if (ChildIndex == 0)
		{
			return 1.0f - ClampedWeight;
		}
		else if (ChildIndex == 1)
		{
			return ClampedWeight;
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}
}
