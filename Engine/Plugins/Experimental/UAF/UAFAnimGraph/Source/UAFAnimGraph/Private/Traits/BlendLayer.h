// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EvaluationVM/Tasks/BlendKeyframesPerBone.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/BlendLayerTraitData.h"

namespace UE::UAF
{
	/**
	 * FBlendLayerTrait
	 * 
	 * A trait that can blend a layer into an input.
	 */
	 struct FBlendLayerTrait : FBaseTrait, IEvaluate, IUpdate, IUpdateTraversal, IHierarchy, IContinuousBlend
	 {
		DECLARE_ANIM_TRAIT(FBlendLayerTrait, FBaseTrait)


		using FSharedData = FUAFBlendLayerTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr ChildBase;
			FTraitPtr ChildBlend;

			bool bWasChildBaseRelevant = false;
			bool bWasChildBlendRelevant = false;
		};


		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IContinuousBlend impl
		virtual float GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const override;
	 };
}