// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendSmoother.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitInterfaces/IHierarchy.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSmoother)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendSmootherCoreTrait)
	AUTO_REGISTER_ANIM_TRAIT(FBlendSmootherTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IUpdate) \
	
	// Trait required interfaces implementation boilerplate
	#define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacroRerquired) \
		GeneratorMacroRerquired(ISmoothBlend) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendSmootherCoreTrait, TRAIT_INTERFACE_ENUMERATOR, TRAIT_REQUIRED_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(ISmoothBlend) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendSmootherTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FBlendSmootherCoreTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// We override the default behavior since we need to blend over time

		int32 NumBlending = 0;
		for (const FBlendData& ChildBlendData : InstanceData->PerChildBlendData)
		{
			NumBlending += ChildBlendData.bIsBlending ? 1 : 0;
		}

		if (NumBlending < 2)
		{
			return;	// If we don't have at least 2 children blending, there is nothing to do
		}

		// Children are visited depth first, in the order returned
		// As such, when we evaluate the task program, the keyframe of the last child will be
		// on top of the keyframe stack
		// We thus process children in reverse order

		// The last child override the top keyframe and scales it
		int32 ChildIndex = InstanceData->PerChildBlendData.Num() - 1;
		for (; ChildIndex >= 0; --ChildIndex)
		{
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			if (!ChildBlendData.bIsBlending)
			{
				continue;	// Skip this inactive child
			}

			// This trait controls the blend weight and owns it
			Context.AppendTask(FAnimNextBlendOverwriteKeyframeWithScaleTask::Make(ChildBlendData.Weight));

			// We found the last child to blend
			break;
		}

		// Other children accumulate with scale
		ChildIndex--;
		for (; ChildIndex >= 0; --ChildIndex)
		{
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			if (!ChildBlendData.bIsBlending)
			{
				continue;	// Skip this inactive child
			}

			// This trait controls the blend weight and owns it
			Context.AppendTask(FAnimNextBlendAddKeyframeWithScaleTask::Make(ChildBlendData.Weight));
		}

		// Once we are done, we normalize rotations
		Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
	}

	void FBlendSmootherCoreTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// If this is our first update, allocate our blend data
		if (InstanceData->PerChildBlendData.IsEmpty())
		{
			InitializeInstanceData(Context, Binding, SharedData, InstanceData);
		}

		// Update the traits below us, they might trigger a transition
		IUpdate::PreUpdate(Context, Binding, TraitState);

		const float DeltaTime = TraitState.GetDeltaTime();

		// Advance the weights
		float SumWeight = 0.0f;
		uint32 NumBlending = 0;

		for (FBlendData& ChildBlendData : InstanceData->PerChildBlendData)
		{
			if (!ChildBlendData.bIsBlending)
			{
				continue;	// Skip children that aren't blending
			}

			ChildBlendData.Blend.Update(DeltaTime);

			const float NewBlendWeight = ChildBlendData.Blend.GetBlendedValue();

			ChildBlendData.Weight = NewBlendWeight;
			SumWeight += NewBlendWeight;
			NumBlending++;
		}

		if (NumBlending <= 1)
		{
			return;	// Nothing to do if we don't blend at least 2 children together
		}

		// Renormalize the weights if the sum isn't near 0.0 or near 1.0
		if (SumWeight > ZERO_ANIMWEIGHT_THRESH &&
			FMath::Abs(SumWeight - 1.0f) > ZERO_ANIMWEIGHT_THRESH)
		{
			const float ReciprocalSum = 1.0f / SumWeight;

			for (FBlendData& ChildBlendData : InstanceData->PerChildBlendData)
			{
				ChildBlendData.Weight *= ReciprocalSum;
			}
		}
		
		// Do not attempt to terminate blends in case of being ticked with zero delta-time
		if (DeltaTime == 0.f)
		{
			return;
		}

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// Free any newly inactive children
		const int32 NumChildren = InstanceData->PerChildBlendData.Num();
		for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];

			if (ChildBlendData.bIsBlending && ChildBlendData.Weight <= 0.0f)
			{
				// This child has finished blending out, terminate it
				DiscreteBlendTrait.OnBlendTerminated(Context, ChildIndex);

				ChildBlendData.bIsBlending = false;
			}
		}
	}

	float FBlendSmootherCoreTrait::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->PerChildBlendData.IsValidIndex(ChildIndex) ? InstanceData->PerChildBlendData[ChildIndex].Weight : -1.0f;
	}

	const FAlphaBlend* FBlendSmootherCoreTrait::GetBlendState(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->PerChildBlendData.IsValidIndex(ChildIndex) ? &InstanceData->PerChildBlendData[ChildIndex].Blend : nullptr;
	}

	void FBlendSmootherCoreTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<ISmoothBlend> SmoothBlendTrait;
		Binding.GetStackInterface(SmoothBlendTrait);

		const int32 NumChildren = InstanceData->PerChildBlendData.Num();
		if (NewChildIndex >= NumChildren)
		{
			// We have a new child
			check(NewChildIndex == NumChildren);

			const EAlphaBlendOption BlendType = SmoothBlendTrait.GetBlendType(Context, NewChildIndex);
			UCurveFloat* CustomBlendCurve = SmoothBlendTrait.GetCustomBlendCurve(Context, NewChildIndex);

			FBlendData& ChildBlendData = InstanceData->PerChildBlendData.AddDefaulted_GetRef();

			ChildBlendData.Blend.SetBlendOption(BlendType);
			ChildBlendData.Blend.SetCustomCurve(CustomBlendCurve);
		}

		// scale by the weight difference since we want consistency:
		// - if you're moving from 0 to full weight 1, it will use the normal blend time
		// - if you're moving from 0.5 to full weight 1, it will get there in half the time
		const float NewChildCurrentWeight = InstanceData->PerChildBlendData[NewChildIndex].Weight;
		const float NewChildDesiredWeight = 1.0f;
		const float WeightDifference = FMath::Clamp(FMath::Abs(NewChildDesiredWeight - NewChildCurrentWeight), 0.0f, 1.0f);

		const float BlendTime = SmoothBlendTrait.GetBlendTime(Context, NewChildIndex);
		const float RemainingBlendTime = OldChildIndex != INDEX_NONE ? (BlendTime * WeightDifference) : 0.0f;

		if (OldChildIndex != INDEX_NONE)
		{
			// Make sure the old child starts blending out
			FBlendData& OldChildBlendData = InstanceData->PerChildBlendData[OldChildIndex];
			OldChildBlendData.Blend.SetValueRange(OldChildBlendData.Weight, 0.0f);
			check(OldChildBlendData.bIsBlending);
		}

		{
			// Setup the new child to blend in
			FBlendData& NewChildBlendData = InstanceData->PerChildBlendData[NewChildIndex];
			NewChildBlendData.Blend.SetValueRange(NewChildBlendData.Weight, 1.0f);
			NewChildBlendData.Blend.ResetAlpha();	// Reset the alpha right away in case another trait needs it
			NewChildBlendData.bIsBlending = true;
		}

		// We set the new blend time on all children
		for (FBlendData& ChildBlendData : InstanceData->PerChildBlendData)
		{
			ChildBlendData.Blend.SetBlendTime(RemainingBlendTime);
		}

		// Don't call the super since we hijack the transition to smooth it out over time
		// We just initiate the new blend manually

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		DiscreteBlendTrait.OnBlendInitiated(Context, NewChildIndex);
	}

	void FBlendSmootherCoreTrait::InitializeInstanceData(FExecutionContext& Context, const FTraitBinding& Binding, const FSharedData* SharedData, FInstanceData* InstanceData)
	{
		check(InstanceData->PerChildBlendData.IsEmpty());

		TTraitBinding<ISmoothBlend> SmoothBlendTrait;
		Binding.GetStackInterface(SmoothBlendTrait);

		const uint32 NumChildren = IHierarchy::GetNumStackChildren(Context, Binding);

		InstanceData->PerChildBlendData.SetNum(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			const EAlphaBlendOption BlendType = SmoothBlendTrait.GetBlendType(Context, ChildIndex);
			UCurveFloat* CustomBlendCurve = SmoothBlendTrait.GetCustomBlendCurve(Context, ChildIndex);

			FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			ChildBlendData.Blend.SetBlendOption(BlendType);
			ChildBlendData.Blend.SetCustomCurve(CustomBlendCurve);
		}
	}

	float FBlendSmootherTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		if (SharedData->BlendTimes.IsValidIndex(ChildIndex))
		{
			return SharedData->BlendTimes[ChildIndex];
		}
		else if (!SharedData->BlendTimes.IsEmpty())
		{
			// If we index outside the array of values we have, use the last value
			// Allows a user to specify a single blend time to be used with all children
			return SharedData->BlendTimes.Last();
		}
		else
		{
			// No blend time has been specified, forward below us on the stack, maybe someone can provide one
			return ISmoothBlend::GetBlendTime(Context, Binding, ChildIndex);
		}
	}

	EAlphaBlendOption FBlendSmootherTrait::GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		return SharedData->BlendType;
	}

	UCurveFloat* FBlendSmootherTrait::GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		return SharedData->CustomBlendCurve;
	}
}
