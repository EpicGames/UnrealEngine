// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendSmootherPerBone.h"

#include "AlphaBlend.h"
#include "Animation/BlendProfile.h"
#include "Animation/Skeleton.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitInterfaces/IHierarchy.h"
#include "EvaluationVM/Tasks/BlendKeyframesPerBone.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSmootherPerBone)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendSmootherPerBoneCoreTrait)
	AUTO_REGISTER_ANIM_TRAIT(FBlendSmootherPerBoneTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IUpdate) \
	
	// Trait required interfaces implementation boilerplate
	#define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacroRequired) \
		GeneratorMacroRequired(ISmoothBlend) \
		GeneratorMacroRequired(ISmoothBlendPerBone) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendSmootherPerBoneCoreTrait, TRAIT_INTERFACE_ENUMERATOR, TRAIT_REQUIRED_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(ISmoothBlendPerBone) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendSmootherPerBoneTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR

	void FBlendSmootherPerBoneCoreTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		int32 NumBlending = 0;

		for (const FBlendData& ChildBlendData : InstanceData->PerChildBlendData)
		{
			NumBlending += ChildBlendData.bIsBlending ? 1 : 0;
		}

		if (NumBlending < 2)
		{
			return;	// If we don't have at least 2 children blending, there is nothing to do
		}

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// Children are visited depth first, in the order returned
		// As such, when we evaluate the task program, the keyframe of the last child will be
		// on top of the keyframe stack
		// We thus process children in reverse order

		// The last child override the top keyframe and scales it
		int32 ChildIndex = InstanceData->PerChildBlendData.Num() - 1;
		for (; ChildIndex >= 0; --ChildIndex)
		{
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			if (!ChildBlendData.bIsBlending)
			{
				continue;	// Skip this inactive child
			}

			const float ChildWeight = DiscreteBlendTrait.GetBlendWeight(Context, ChildIndex);

			// This trait controls the blend weight and owns it
			if (InstanceData->BlendProfileSkeleton)
			{
				const FBlendSampleData& SampleData = InstanceData->PerBoneSampleData[ChildIndex];
				Context.AppendTask(FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask::Make(InstanceData->BlendProfileSkeleton, SampleData, 1.0f));
			}
			else
			{
				Context.AppendTask(FAnimNextBlendOverwriteKeyframeWithScaleTask::Make(ChildWeight));
			}

			// We found the last child to blend
			break;
		}

		// Other children accumulate with scale
		ChildIndex--;
		for (; ChildIndex >= 0; --ChildIndex)
		{
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			if (!ChildBlendData.bIsBlending)
			{
				continue;	// Skip this inactive child
			}
			
			const float ChildWeight = DiscreteBlendTrait.GetBlendWeight(Context, ChildIndex);

			// This trait controls the blend weight and owns it
			if (InstanceData->BlendProfileSkeleton)
			{
				const FBlendSampleData& PoseSampleDataA = InstanceData->PerBoneSampleData[ChildIndex];
				const FBlendSampleData& PoseSampleDataB = InstanceData->PerBoneSampleData[ChildIndex + 1];

				Context.AppendTask(FAnimNextBlendAddKeyframePerBoneWithScaleTask::Make(
					InstanceData->BlendProfileSkeleton,
					PoseSampleDataA,
					PoseSampleDataB,
					1.0f));
			}
			else
			{
				Context.AppendTask(FAnimNextBlendAddKeyframeWithScaleTask::Make(ChildWeight));
			}
		}

		// Once we are done, we normalize rotations
		Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
	}

	void FBlendSmootherPerBoneCoreTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		{
			const int32 NumChildren = InstanceData->PerChildBlendData.Num();
			check(InstanceData->ChildOrdering.Num() == NumChildren);
			check(InstanceData->PerBoneSampleData.Num() == NumChildren);		
		
			for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				InstanceData->PerChildBlendData[ChildIndex].OldWeight = DiscreteBlendTrait.GetBlendWeight(Context, ChildIndex);
			}
		}

		// If this is our first update, allocate our blend data
		if (InstanceData->PerChildBlendData.IsEmpty())
		{
			InitializeInstanceData(Context, Binding, SharedData, InstanceData);
		}

		// Update the traits below us, they might trigger a transition
		// ISmoothBlend does most of the bookkeeping here
		IUpdate::PreUpdate(Context, Binding, TraitState);

		const int32 NumChildren = InstanceData->PerChildBlendData.Num();
		for (int32 ChildOrderIndex = 0; ChildOrderIndex < NumChildren; ++ChildOrderIndex)
		{
			const int32 ChildIndex = InstanceData->ChildOrdering[ChildOrderIndex];
			const float ChildWeight = DiscreteBlendTrait.GetBlendWeight(Context, ChildIndex);

			FBlendSampleData& ChildSampleData = InstanceData->PerBoneSampleData[ChildIndex];
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];

			if (ChildBlendData.BlendProfile && ChildOrderIndex != 0)
			{
				const UUAFBlendProfile::FSkeletonBoneWeightArray& BoneProfileWeights = ChildBlendData.BlendProfile->GetSkeletonBoneWeights();

				if (ChildBlendData.BlendProfile->GetType() == EUAFBlendProfileType::WeightFactor)
				{
					for (int32 PerBoneIndex = 0; PerBoneIndex < ChildSampleData.PerBoneBlendData.Num(); ++PerBoneIndex)
					{
						const float ProfileValue = BoneProfileWeights[PerBoneIndex];
						const float Weight = (ProfileValue > ZERO_ANIMWEIGHT_THRESH) ? ChildWeight / ProfileValue : 1.0f;
						ChildSampleData.PerBoneBlendData[PerBoneIndex] = FMath::Max(Weight, ZERO_ANIMWEIGHT_THRESH);
					}
				}
				else
				{
					const FAlphaBlend* BlendState = DiscreteBlendTrait.GetBlendState(Context, ChildIndex);

					for (int32 PerBoneIndex = 0; PerBoneIndex < ChildSampleData.PerBoneBlendData.Num(); ++PerBoneIndex)
					{
						const float ProfileValue = BoneProfileWeights[PerBoneIndex];

						ChildSampleData.PerBoneBlendData[PerBoneIndex] = UBlendProfile::CalculateBoneWeight(
							ProfileValue,
							EBlendProfileMode::TimeFactor,
							*BlendState,
							ChildBlendData.OldWeight,
							ChildWeight,
							false);
					}
				}
			}
			else if (InstanceData->BlendProfileSkeleton)
			{
				for (int32 PerBoneIndex = 0; PerBoneIndex < ChildSampleData.PerBoneBlendData.Num(); ++PerBoneIndex)
				{
					ChildSampleData.PerBoneBlendData[PerBoneIndex] = ChildWeight;
				}
			}
			else
			{
				// Guaranteed no child is using blend profile
			}
		}

		// We need to normalize the per-bone weights of the active children
		NormalizeChildPerBoneWeights(InstanceData);
	}

	void FBlendSmootherPerBoneCoreTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		// First delegate to the traits below us
		// Again, ISmoothBlend does most of the work for us
		IDiscreteBlend::OnBlendTransition(Context, Binding, OldChildIndex, NewChildIndex);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<ISmoothBlendPerBone> SmoothBlendPerBoneTrait;
		Binding.GetStackInterface(SmoothBlendPerBoneTrait);

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		const int32 NumChildren = InstanceData->PerChildBlendData.Num();
		if (NewChildIndex >= NumChildren)
		{
			// We have a new child
			check(NewChildIndex == NumChildren);

			FBlendData& ChildBlendData = InstanceData->PerChildBlendData.AddDefaulted_GetRef();
			FBlendSampleData& NewSampleData = InstanceData->PerBoneSampleData.AddDefaulted_GetRef();
			if (InstanceData->BlendProfileSkeleton)
			{
				NewSampleData.PerBoneBlendData.SetNumUninitialized(InstanceData->BlendProfileSkeleton->GetReferenceSkeleton().GetNum());
			}
		}

		InstanceData->ChildOrdering.RemoveSwap(NewChildIndex);
		InstanceData->ChildOrdering.Add(NewChildIndex);

		{
			// Setup the new child to blend in
			FBlendData& ChildBlendData = InstanceData->PerChildBlendData[NewChildIndex];
			FBlendSampleData& ChildSampleData = InstanceData->PerBoneSampleData[NewChildIndex];
			
			const UUAFBlendProfile* BlendProfile = SmoothBlendPerBoneTrait.GetBlendProfile(Context, NewChildIndex);
			if (BlendProfile)
			{
				if (!InstanceData->BlendProfileSkeleton)
				{
					SetBlendProfileSkeleton(BlendProfile->GetSkeleton(), SharedData, InstanceData);
				}
				else if (BlendProfile->GetSkeleton() == InstanceData->BlendProfileSkeleton)
				{
					ChildBlendData.BlendProfile = BlendProfile;
				}
			}

			if (InstanceData->BlendProfileSkeleton)
			{
				const int32 BoneCount = InstanceData->BlendProfileSkeleton->GetReferenceSkeleton().GetNum();
				for (int32 Index = 0; Index < BoneCount; ++Index)
				{
					ChildSampleData.PerBoneBlendData[Index] = 1.0f;
				}
			}

			ChildBlendData.OldWeight = DiscreteBlendTrait.GetBlendWeight(Context, NewChildIndex);
			ChildBlendData.bIsBlending = true;
		}
	}

	void FBlendSmootherPerBoneCoreTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		IDiscreteBlend::OnBlendTerminated(Context, Binding, ChildIndex);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
		ChildBlendData.bIsBlending = false;
		ChildBlendData.BlendProfile = nullptr;
		
		FBlendSampleData& SampleData = InstanceData->PerBoneSampleData[ChildIndex];
		for (int32 PerBoneIndex = 0; PerBoneIndex < SampleData.PerBoneBlendData.Num(); ++PerBoneIndex)
		{
			SampleData.PerBoneBlendData[PerBoneIndex] = 0.0f;
		}
	}

	void FBlendSmootherPerBoneCoreTrait::InitializeInstanceData(FExecutionContext& Context, const FTraitBinding& Binding, const FSharedData* SharedData, FInstanceData* InstanceData)
	{
		check(InstanceData->PerBoneSampleData.IsEmpty());
		check(InstanceData->PerChildBlendData.IsEmpty());
		
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		TTraitBinding<ISmoothBlendPerBone> SmoothBlendPerBoneTrait;
		Binding.GetStackInterface(SmoothBlendPerBoneTrait);

		const uint32 NumChildren = IHierarchy::GetNumStackChildren(Context, Binding);

		InstanceData->PerBoneSampleData.SetNum(NumChildren);
		InstanceData->PerChildBlendData.SetNum(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			InstanceData->ChildOrdering.Add(ChildIndex);
			
			const UUAFBlendProfile* BlendProfile = SmoothBlendPerBoneTrait.GetBlendProfile(Context, ChildIndex);

			if (BlendProfile)
			{
				if (InstanceData->BlendProfileSkeleton)
				{
					if (BlendProfile->GetSkeleton() != InstanceData->BlendProfileSkeleton)
					{
						UE_LOGF(LogAnimation, Warning, "BlendSmootherPerBoneCoreTrait: Contains blend profiles using differing skeletons");
					}
				}
				else
				{
					SetBlendProfileSkeleton(BlendProfile->GetSkeleton(), SharedData, InstanceData);
				}
			}
		}

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			ChildBlendData.OldWeight = DiscreteBlendTrait.GetBlendWeight(Context, ChildIndex);

			const UUAFBlendProfile* BlendProfile = SmoothBlendPerBoneTrait.GetBlendProfile(Context, ChildIndex);
			if (BlendProfile && BlendProfile->GetSkeleton() == InstanceData->BlendProfileSkeleton)
			{
				ChildBlendData.BlendProfile = BlendProfile;
			}

			if (InstanceData->BlendProfileSkeleton)
			{
				FBlendSampleData& SampleData = InstanceData->PerBoneSampleData[ChildIndex];
				SampleData.PerBoneBlendData.Init(1.0f, InstanceData->BlendProfileSkeleton->GetReferenceSkeleton().GetNum());
			}
		}
	}

	void FBlendSmootherPerBoneCoreTrait::SetBlendProfileSkeleton(const USkeleton* Skeleton, const FSharedData* SharedData, FInstanceData* InstanceData)
	{
		if (!ensure(Skeleton))
		{
			return;
		}

		InstanceData->BlendProfileSkeleton = Skeleton;

		const int32 NumChildren = InstanceData->PerBoneSampleData.Num();
		const int32 NumBones = Skeleton->GetReferenceSkeleton().GetNum();

		InstanceData->PerBoneTotals.SetNumZeroed(NumBones);

		for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			InstanceData->PerBoneSampleData[ChildIndex].PerBoneBlendData.Init(1.0f, NumBones);
		}
	}

	void FBlendSmootherPerBoneCoreTrait::NormalizeChildPerBoneWeights(TNonNullPtr<FInstanceData> InstanceData)
	{
		if (!InstanceData->BlendProfileSkeleton)
		{
			return; // No per-bone weights to normalize
		}
		
		const int32 NumChildren = InstanceData->PerChildBlendData.Num();
		check(InstanceData->ChildOrdering.Num() == NumChildren);
		check(InstanceData->PerBoneSampleData.Num() == NumChildren);

		if (NumChildren < 2)
		{
			return;
		}
		
		const int32 NumBones = InstanceData->PerBoneTotals.Num();

		for (int32 PerBoneIndex = 0; PerBoneIndex < NumBones; ++PerBoneIndex)
		{
			InstanceData->PerBoneTotals[PerBoneIndex] = 0.0f;
		}
		
		for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			if (!ChildBlendData.bIsBlending)
			{
				continue;
			}
			
			const FBlendSampleData& ChildSampleData = InstanceData->PerBoneSampleData[ChildIndex];

			for (int32 PerBoneIndex = 0; PerBoneIndex < NumBones; ++PerBoneIndex)
			{
				InstanceData->PerBoneTotals[PerBoneIndex] += ChildSampleData.PerBoneBlendData[PerBoneIndex];
			}
		}

		for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			const FBlendData& ChildBlendData = InstanceData->PerChildBlendData[ChildIndex];
			if (!ChildBlendData.bIsBlending)
			{
				continue;
			}
			
			FBlendSampleData& ChildSampleData = InstanceData->PerBoneSampleData[ChildIndex];

			for (int32 PerBoneIndex = 0; PerBoneIndex < NumBones; ++PerBoneIndex)
			{
				ChildSampleData.PerBoneBlendData[PerBoneIndex] /= InstanceData->PerBoneTotals[PerBoneIndex];
			}
		}
	}

	const UUAFBlendProfile* FBlendSmootherPerBoneTrait::GetBlendProfile(FExecutionContext& Context, const TTraitBinding<ISmoothBlendPerBone>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		if (SharedData->BlendProfiles.IsValidIndex(ChildIndex))
		{
			return SharedData->BlendProfiles[ChildIndex];
		}
		else if (!SharedData->BlendProfiles.IsEmpty())
		{
			return SharedData->BlendProfiles.Last();
		}
		else
		{
			return ISmoothBlendPerBone::GetBlendProfile(Context, Binding, ChildIndex);
		}
	}
}
