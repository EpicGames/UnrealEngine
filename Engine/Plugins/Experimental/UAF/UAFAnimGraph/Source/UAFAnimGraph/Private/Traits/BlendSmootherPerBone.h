// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimationAsset.h"
#include "Traits/BlendSmootherPerBoneTraitData.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/ISmoothBlend.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/ISmoothBlendPerBone.h"
#include "HierarchyTableBlendProfile.h"

#include "BlendSmootherPerBone.generated.h"

class UUAFBlendProfile;

USTRUCT(meta = (DisplayName = "Blend Smoother Per Bone"))
struct FUAFBlendSmootherPerBoneTraitSharedData : public FAnimNextBlendSmootherPerBoneCoreTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	TArray<TObjectPtr<UUAFBlendProfile>> BlendProfiles;
};

namespace UE::UAF
{
	/**
	 * FBlendSmootherPerBoneCoreTrait
	 *
	 * An additive trait that smoothly blends with per-bone weights.
	 */
	struct FBlendSmootherPerBoneCoreTrait : FAdditiveTrait, IEvaluate, IUpdate, IDiscreteBlend
	{
		DECLARE_ANIM_TRAIT(FBlendSmootherPerBoneCoreTrait, FAdditiveTrait)

		using FSharedData = FAnimNextBlendSmootherPerBoneCoreTraitSharedData;

		// Struct for tracking blends for each pose
		struct FBlendData
		{
			// The blend profile to use to blend this child in with
			// All blend profiles must use the same skeleton
			const UUAFBlendProfile* BlendProfile = nullptr;

			// The cached value of this child's weight last update
			float OldWeight = 0.0f;

			// Flag to skip a child if its fully blended out
			bool bIsBlending = false;
		};

		struct FInstanceData : FTrait::FInstanceData
		{
			TArray<FBlendData> PerChildBlendData;

			// Per-bone blending data for each child
			TArray<FBlendSampleData> PerBoneSampleData;

			// Scratch data for norma
			TArray<float> PerBoneTotals;

			// The ordering of children
			TArray<int32> ChildOrdering;

			// The common skeleton all blend profiles must share
			const USkeleton* BlendProfileSkeleton;
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IDiscreteBlend impl
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;
		virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return false; }
#endif

	private:
		// Internal impl
		static void InitializeInstanceData(FExecutionContext& Context, const FTraitBinding& Binding, const FSharedData* SharedData, FInstanceData* InstanceData);
		static void SetBlendProfileSkeleton(const USkeleton* Skeleton, const FSharedData* SharedData, FInstanceData* InstanceData);
		static void NormalizeChildPerBoneWeights(TNonNullPtr<FInstanceData> InstanceData);
	};

	struct FBlendSmootherPerBoneTrait : public FBlendSmootherPerBoneCoreTrait, ISmoothBlendPerBone
	{
		DECLARE_ANIM_TRAIT(FBlendSmootherPerBoneTrait, FBlendSmootherPerBoneCoreTrait)

		using FSharedData = FUAFBlendSmootherPerBoneTraitSharedData;

		// ISmoothBlendPerBone
		virtual const UUAFBlendProfile* GetBlendProfile(FExecutionContext& Context, const TTraitBinding<ISmoothBlendPerBone>& Binding, int32 ChildIndex) const override;
	};
}
