// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendSpacePlayer.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Asset/UAFAnimGraphAssetData.h"
#include "Factory/AnimGraphFactory.h"
#include "GraphInterfaces/AnimNextNativeDataInterface_AnimSequencePlayer.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/IGraphFactory.h"


namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendSpacePlayerTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IContinuousBlend) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(ITimeline) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \
		GeneratorMacro(IGarbageCollection) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendSpacePlayerTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	
	void FBlendSpacePlayerTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->NumChildren >= 3)
		{
			TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
			Binding.GetStackInterface(ContinuousBlendTrait);

			int32 ChildIndex = InstanceData->NumChildren - 1;
			float ChildBlendWeight = ContinuousBlendTrait.GetBlendWeight(Context, ChildIndex);

			// The last child overrides the top keyframe and scales it
			Context.AppendTask(FAnimNextBlendOverwriteKeyframeWithScaleTask::Make(ChildBlendWeight));

			// Other children accumulate with scale
			ChildIndex--;
			for (; ChildIndex >= 0; --ChildIndex)
			{
				ChildBlendWeight = ContinuousBlendTrait.GetBlendWeight(Context, ChildIndex);

				// This trait controls the blend weight and owns it
				Context.AppendTask(FAnimNextBlendAddKeyframeWithScaleTask::Make(ChildBlendWeight));
			}

			// Once we are done, we normalize rotations
			Context.AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
		}
		else if (InstanceData->NumChildren == 2)
		{
			// We have two children, interpolate them

			TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
			Binding.GetStackInterface(ContinuousBlendTrait);

			const float BlendWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
			Context.AppendTask(FAnimNextBlendTwoKeyframesTask::Make(BlendWeight));
		}
		else
		{
			// We have only one child that is active, do nothing
		}
	}

	void FBlendSpacePlayerTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Cache the blend space we'll play during construction, we don't allow it to change afterwards
		InstanceData->BlendSpace = SharedData->GetBlendSpace(Binding);
		if (InstanceData->BlendSpace)
		{
			InstanceData->BlendSpace->InitializeFilter(&InstanceData->BlendFilter);

			FVector BlendParameters(SharedData->GetXAxisSamplePoint(Binding), SharedData->GetYAxisSamplePoint(Binding), 0.0f);

			// reset all the blend samples for the first frame
			if (InstanceData->BlendSpace->GetSamplesFromBlendInput(BlendParameters, InstanceData->BlendSamplesData, InstanceData->CachedTriangulationIndex, true))
			{
				// Get starting time for all samples.
				InstanceData->BlendSpace->ResetBlendSamples(InstanceData->BlendSamplesData, 0.0f, true, true);
			}

			// create graphs for all the blend samples
			const TArray<FBlendSample>& BlendSamples = InstanceData->BlendSpace->GetBlendSamples();
			if (InstanceData->SampleGraphs.Num() != BlendSamples.Num())
			{
				InstanceData->SampleGraphs.SetNum(BlendSamples.Num(), EAllowShrinking::No);
				for (int32 CurSampleIndex = 0; CurSampleIndex < BlendSamples.Num(); CurSampleIndex++)
				{
					const UAnimSequence* CurSequence = BlendSamples[CurSampleIndex].Animation;
					const EAnimAssetLoopMode LoopMode = SharedData->GetLoopMode(Binding);
					const float StartPosition = SharedData->GetStartPosition(Binding);

					FUAFGraphFactoryAsset_Animation Asset(CurSequence);

					FAnimNextFactoryParams FactoryParams = FAnimGraphFactory::GetDefaultParamsForAsset(Asset);
					FactoryParams.AddInitializeTask([SharedData, InstanceData, CurSequence, CurSampleIndex, StartPosition, LoopMode](const FInstanceTaskContext& InContext)
					{
						InContext.AccessVariablesStruct<FSequencePlayerData>([SharedData, InstanceData, CurSequence, CurSampleIndex, StartPosition, LoopMode](FSequencePlayerData& InSequencePlayerData)
						{
							InSequencePlayerData.AnimSequence = CurSequence;

							// Assume start position is normalized.
							const TArray<FBlendSample>& BlendSamples = InstanceData->BlendSpace->GetBlendSamples();
							InSequencePlayerData.StartPosition = StartPosition * BlendSamples[CurSampleIndex].GetSamplePlayLength();
							// Blendspace has authority over loopness, since non-looping and looping assets can't sync together.
							InSequencePlayerData.LoopMode = [&InstanceData, LoopMode]()
							{
								switch (LoopMode)
								{
								case EAnimAssetLoopMode::Auto:
									return InstanceData->BlendSpace->bLoop ? EAnimAssetLoopMode::ForceLoop : EAnimAssetLoopMode::ForceNonLoop;
								case EAnimAssetLoopMode::ForceLoop:
								case EAnimAssetLoopMode::ForceNonLoop:
								default:
									return LoopMode;
							}
							}();
						});
					});

					const UUAFAnimGraph* AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, Asset, FactoryParams);
					if (AnimationGraph != nullptr)
					{
						FSampleGraphState& TargetGraphState = InstanceData->SampleGraphs[CurSampleIndex];

						const FName EntryPoint = NAME_None;
						FAnimNextGraphInstance& Owner = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
						TargetGraphState.Instance = AnimationGraph->AllocateInstance(
							{
								.SystemReference = Owner.GetModuleInstance()->GetReference(),
								.ParentGraphInstance = Owner.AsShared(),
								.EntryPoint = EntryPoint,
							},
							MoveTemp(FactoryParams));
						if (TargetGraphState.Instance.IsValid())
						{
							TargetGraphState.ChildPtr = TargetGraphState.Instance->GetGraphRootPtr();
						}
					}
				}
			}
		}
	}

	uint32 FBlendSpacePlayerTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->NumChildren;

	}

	void FBlendSpacePlayerTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->NumChildren > 0)
		{
			for (const FSampleGraphState& SampleState : InstanceData->SampleGraphs)
			{
				if (SampleState.bSampledThisFrame)
				{
					Children.Add(SampleState.ChildPtr);
				}
			}
		}
		check(Children.Num() == InstanceData->NumChildren);
	}

	void FBlendSpacePlayerTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->NumChildren == 0)
		{
			return;
		}

		for (const FSampleGraphState& SampleState : InstanceData->SampleGraphs)
		{
			if (SampleState.bSampledThisFrame)
			{
				const bool bGraphHasNeverUpdated = SampleState.Instance.IsValid() && !SampleState.Instance->HasUpdated();
				const bool bNewlyRelevant = (SampleState.bSampledThisFrame && !SampleState.bSampledLastFrame) || bGraphHasNeverUpdated;

				const FTraitUpdateState SampleTraitState = TraitState
					.WithDeltaTime(SampleState.DeltaTime)
					.WithWeight(SampleState.Weight)
					.AsNewlyRelevant(bNewlyRelevant);

				TraversalQueue.Push(SampleState.ChildPtr, SampleTraitState);

				if (SampleState.Instance.IsValid())
				{
					SampleState.Instance->MarkAsUpdated();
				}
			}
		}
	}

	float FBlendSpacePlayerTrait::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (ChildIndex < InstanceData->NumChildren)
		{
			int CurChildIndex = 0;
			for (const FSampleGraphState& SampleState : InstanceData->SampleGraphs)
			{
				if (SampleState.bSampledThisFrame)
				{
					if (ChildIndex == CurChildIndex)
					{
						return SampleState.Weight;
					}
					CurChildIndex++;
				}
			}
		}
		return -1.0f;
	}

	void FBlendSpacePlayerTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PreUpdate(Context, Binding, TraitState);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		const float DeltaTime = TraitState.GetDeltaTime();

		// input data 
		FVector BlendParameters(SharedData->GetXAxisSamplePoint(Binding), SharedData->GetYAxisSamplePoint(Binding), 0.0f);
		for (FSampleGraphState& SampleState : InstanceData->SampleGraphs)
		{
			SampleState.bSampledLastFrame = SampleState.bSampledThisFrame;
			SampleState.bSampledThisFrame = false;
		}

		InstanceData->NumChildren = 0;

		if (const UBlendSpace* BlendSpacePtr = InstanceData->BlendSpace.Get())
		{
			if (BlendSpacePtr != nullptr && BlendSpacePtr->GetSkeleton() != nullptr)
			{
				const FVector FilteredBlendParams = BlendSpacePtr->FilterInput(&InstanceData->BlendFilter, BlendParameters, DeltaTime);
				if (BlendSpacePtr->UpdateBlendSamples(FilteredBlendParams, DeltaTime, InstanceData->BlendSamplesData, InstanceData->CachedTriangulationIndex))
				{
					const float BlendSpacePlayRate = SharedData->GetPlayRate(Binding);
					const float AxisScaleFactor = BlendSpacePtr->ComputeAxisScaleFactor(BlendParameters, FilteredBlendParams);

					const float BlendSpaceDeltaTime = DeltaTime * BlendSpacePlayRate * AxisScaleFactor;

					// Update our weights based on the new samples
					for (const FBlendSampleData& CurSample : InstanceData->BlendSamplesData)
					{
						if (CurSample.SampleDataIndex < InstanceData->SampleGraphs.Num())
						{
							FSampleGraphState& SampleState = InstanceData->SampleGraphs[CurSample.SampleDataIndex];
							SampleState.Weight = CurSample.TotalWeight;
							SampleState.DeltaTime = BlendSpaceDeltaTime * CurSample.SamplePlayRate;
							SampleState.bSampledThisFrame = true;

							InstanceData->NumChildren++;
						}
					}
				}
			}
		}
	}

	bool FBlendSpacePlayerTrait::GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->NumChildren > 0)
		{
			int32 HighestWeightIndex = 0;
			float HighestWeight = 0.0f;
			// using the blend samples set the weights and activity 
			for (const FBlendSampleData& CurSample : InstanceData->BlendSamplesData)
			{
				if (CurSample.SampleDataIndex < InstanceData->SampleGraphs.Num() && CurSample.TotalWeight > HighestWeight)
				{
					HighestWeight = CurSample.TotalWeight;
					HighestWeightIndex = CurSample.SampleDataIndex;
				}
			}
			if (HighestWeightIndex < InstanceData->SampleGraphs.Num())
			{
				const FSampleGraphState& SampleState = InstanceData->SampleGraphs[HighestWeightIndex];
				FTraitStackBinding ChildTraitStack;
				ensure(Context.GetStack(SampleState.ChildPtr, ChildTraitStack));
				UE::UAF::TTraitBinding<UE::UAF::ITimeline> Timeline;
				if (ChildTraitStack.GetInterface<UE::UAF::ITimeline>(Timeline))
				{
					return Timeline.GetState(Context, OutTimelineState);
				}
			}
		}
		return false;
	}

	void FBlendSpacePlayerTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		Collector.AddReferencedObject(InstanceData->BlendSpace);

		for (FBlendSpacePlayerTrait::FSampleGraphState& GraphState : InstanceData->SampleGraphs)
		{
			if (FAnimNextGraphInstance* ImplPtr = GraphState.Instance.Get())
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), ImplPtr);
			}
		}	
	}
}
