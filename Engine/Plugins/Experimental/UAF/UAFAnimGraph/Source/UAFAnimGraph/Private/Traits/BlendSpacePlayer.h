// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/ITimeline.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Traits/BlendSpacePlayerTraitData.h"

namespace UE::UAF
{
	/**
	 * FBlendSpacePlayerTrait
	 *
	 * A trait that can play a blend space
	 */
	struct FBlendSpacePlayerTrait : FBaseTrait, IEvaluate, IUpdate, IUpdateTraversal, IHierarchy, ITimeline, IContinuousBlend, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FBlendSpacePlayerTrait, FBaseTrait)

		using FSharedData = FAnimNextBlendSpacePlayerTraitSharedData;

		struct FSampleGraphState
		{
			// The graph instance
			TSharedPtr<FAnimNextGraphInstance> Instance;

			// Our child handle
			// Weak handle to the graph instance's root
			FTraitPtr ChildPtr;

			// The weight of this sample
			float Weight = 0.0f;

			// The scaled delta time for this sample
			float DeltaTime = 0.0f;

			bool bSampledLastFrame = false;

			bool bSampledThisFrame = false;

		};

		struct FInstanceData : FTrait::FInstanceData
		{
			// List of anim next graphs that represent each sample in the blend space
			TArray<FSampleGraphState> SampleGraphs;

			// Cached value of the blend space we are playing
			TObjectPtr<const UBlendSpace> BlendSpace;

			// Cached blend samples updated by task and used to interpolate between points over time
			TArray<FBlendSampleData> BlendSamplesData;

			/** Previous position in the triangulation/segmentation */
			int32 CachedTriangulationIndex = INDEX_NONE;

			// Filter used to dampen coordinate changes
			FBlendFilter BlendFilter;

			int32 NumChildren = 0;
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IContinuousBlend impl
		virtual float GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const override;

		// IUpdate impl
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// ITimeline impl
		virtual bool GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

	};
}
