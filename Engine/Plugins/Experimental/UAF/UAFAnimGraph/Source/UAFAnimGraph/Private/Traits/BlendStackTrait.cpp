// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendStackTrait.h"

#include "Animation/AnimSequence.h"
#include "Graph/AnimNextGraphInstance.h"
#include "GraphInterfaces/AnimNextNativeDataInterface_AnimSequencePlayer.h"
#include <limits>
#include "Logging/StructuredLog.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/IInertializerBlend.h"
#include "HierarchyTableBlendProfile.h"
#include "Factory/AnimGraphFactory.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendStackTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendStackCoreTrait)
	AUTO_REGISTER_ANIM_TRAIT(FBlendStackTrait)
	AUTO_REGISTER_ANIM_TRAIT(FBlendStackRequesterTrait)

	// Trait required interfaces implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IGarbageCollection) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(ISmoothBlend) \
		GeneratorMacro(ISmoothBlendPerBone) \
		GeneratorMacro(IInertializerBlend) \
		GeneratorMacro(ITimeline) \
		GeneratorMacro(IUpdateTraversal) \
		GeneratorMacro(IBlendStack) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendStackCoreTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	// Trait required interfaces implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendStackTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IGarbageCollection) \

		// Trait required interfaces implementation boilerplate
	#define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacroRequired) \
		GeneratorMacroRequired(IBlendStack) \

		GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendStackRequesterTrait, TRAIT_INTERFACE_ENUMERATOR, TRAIT_REQUIRED_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR


	
	void FBlendStackCoreTrait::FGraphState::Initialize(IBlendStack::FGraphRequest&& GraphRequest)
	{
		Request = MoveTemp(GraphRequest);
		Lifetime = 0.0f;
		State = FBlendStackCoreTrait::EGraphState::Active;
		bNewlyCreated = true;
	}

	void FBlendStackCoreTrait::FGraphState::Terminate()
	{
		Instance.Reset();
		ChildPtr.Reset();
		Lifetime = 0.0f;
		bNewlyCreated = false;
		State = FBlendStackCoreTrait::EGraphState::Inactive;
	}

	int32 FBlendStackCoreTrait::FindFreeGraphIndexOrAdd(FInstanceData& InstanceData)
	{
		// Find an empty graph we can use
		const int32 NumGraphs = InstanceData.ChildGraphs.Num();
		for (int32 ChildIndex = 0; ChildIndex < NumGraphs; ++ChildIndex)
		{
			if (InstanceData.ChildGraphs[ChildIndex].State == FBlendStackCoreTrait::EGraphState::Inactive)
			{
				// This graph is inactive, we can re-use it
				return ChildIndex;
			}
		}

		// All graphs are in use, add a new one
		return InstanceData.ChildGraphs.AddDefaulted();
	}

	void FBlendStackCoreTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PreUpdate(Context, Binding, TraitState);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		for (FGraphState& ChildGraph : InstanceData->ChildGraphs)
		{
			// Track lifetime for each child graph
			ChildGraph.Lifetime += TraitState.GetDeltaTime();
		}
	}

	uint32 FBlendStackCoreTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->ChildGraphs.Num();
	}

	void FBlendStackCoreTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (const FBlendStackTrait::FGraphState& ChildGraph : InstanceData->ChildGraphs)
		{
			// Even if the request is inactive, we queue an empty handle
			Children.Add(ChildGraph.GetChildPtr());
		}
	}

	FWeakTraitPtr FBlendStackCoreTrait::GetStackForwardingChild(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const int32 CurrentlyActiveGraphIndex = InstanceData->CurrentlyActiveGraphIndex;
		if (CurrentlyActiveGraphIndex != INDEX_NONE)
		{
			return InstanceData->ChildGraphs[CurrentlyActiveGraphIndex].GetChildPtr();
		}
		
		return FWeakTraitPtr();
	}

	void FBlendStackCoreTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const int32 NumGraphs = InstanceData->ChildGraphs.Num();

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		for (int32 ChildIndex = 0; ChildIndex < NumGraphs; ++ChildIndex)
		{
			FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];
			const float BlendWeight = DiscreteBlendTrait.GetBlendWeight(Context, ChildIndex);
			const bool bGraphHasNeverUpdated = Graph.Instance.IsValid() && !Graph.Instance->HasUpdated();

			// Flag the child instance as updated
			FTraitUpdateState ChildGraphTraitState = TraitState
				.WithWeight(BlendWeight)
				.AsBlendingOut(ChildIndex != InstanceData->CurrentlyActiveGraphIndex)
				.AsNewlyRelevant(Graph.bNewlyCreated || bGraphHasNeverUpdated);
			Graph.bNewlyCreated = false;
			
			if (Graph.Instance.IsValid())
			{
				Graph.Instance->MarkAsUpdated();
			}

			TraversalQueue.Push(InstanceData->ChildGraphs[ChildIndex].GetChildPtr(), ChildGraphTraitState);
		}
	}

	float FBlendStackCoreTrait::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (ChildIndex == InstanceData->CurrentlyActiveGraphIndex)
		{
			return 1.0f;	// Active child has full weight
		}
		else if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			return 0.0f;	// Other children have no weight
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}

	int32 FBlendStackCoreTrait::GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->CurrentlyActiveGraphIndex;
	}

	void FBlendStackCoreTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// We initiate immediately when we transition
		DiscreteBlendTrait.OnBlendInitiated(Context, NewChildIndex);

		// We terminate immediately when we transition
		DiscreteBlendTrait.OnBlendTerminated(Context, OldChildIndex);
	}

	void FBlendStackCoreTrait::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];

			switch(Graph.Request.Type)
			{
			case EGraphRequestType::Owned:
				{
					FAnimNextGraphInstance& Owner = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
					Graph.Instance = Graph.Request.AnimationGraph->AllocateInstance(
						{
							.SystemReference = Owner.GetModuleInstance()->GetReference(),
							.ParentGraphInstance = Owner.AsShared(),
						},
						MoveTemp(Graph.Request.FactoryParams));
					if (Graph.Request.FactoryObject)
					{
						Graph.Instance->SetDebugName(Graph.Request.FactoryObject->GetFName());
					}
					break;
				}
			case EGraphRequestType::Child:
				{
					Graph.SetChildPtr(Graph.Request.ChildPtr);
					break;
				}
			}
		}
	}

	void FBlendStackCoreTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			// We should never terminate the blend of the currently active graph index as it would mean we no
			// longer have an active pose to produce
			check(InstanceData->CurrentlyActiveGraphIndex != ChildIndex);

			FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];

			// Deallocate our graph
			Graph.Terminate();
		}
	}

	float FBlendStackCoreTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			const FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];
			if (Graph.Request.BlendMode == EBlendMode::Inertialization)
			{
				return Graph.Request.BlendArgs.BlendTime;
			}
			else
			{
				// Not an inertializing blend
				return 0.0f;
			}
		}
		else
		{
			// Unknown child
			return 0.0f;
		}
	}

	float FBlendStackCoreTrait::GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			const FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];
			if (Graph.Request.BlendMode == EBlendMode::Standard)
			{
				return Graph.Request.BlendArgs.BlendTime;
			}
			else
			{
				// Not a standard blend
				return 0.0f;
			}
		}
		else
		{
			// Unknown child
			return 0.0f;
		}
	}

	EAlphaBlendOption FBlendStackCoreTrait::GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			const FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];
			return Graph.Request.BlendArgs.BlendOption;
		}
		else
		{
			// Unknown child
			return EAlphaBlendOption::Linear;
		}
	}

	UCurveFloat* FBlendStackCoreTrait::GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			const FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];
			return Graph.Request.BlendArgs.CustomCurve;
		}
		else
		{
			// Unknown child
			return nullptr;
		}
	}

	bool FBlendStackCoreTrait::GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const int32 CurrentlyActiveGraphIndex = InstanceData->CurrentlyActiveGraphIndex;
		if (CurrentlyActiveGraphIndex != INDEX_NONE)
		{
			const FBlendStackCoreTrait::FGraphState& ActiveGraph = InstanceData->ChildGraphs[CurrentlyActiveGraphIndex];

			FTraitStackBinding ChildTraitStack;
			ensure(Context.GetStack(ActiveGraph.GetChildPtr(), ChildTraitStack));

			TTraitBinding<ITimeline> Timeline;
			FTraitStackBinding StackBinding;
			if (IHierarchy::GetForwardedStackInterface<ITimeline>(Context, ChildTraitStack, StackBinding, Timeline))
			{
				return Timeline.GetState(Context, OutTimelineState);
			}
			else
			{
				OutTimelineState = FTimelineState(ActiveGraph.Lifetime, std::numeric_limits<float>::infinity(), 1.0f, false);
				return true;
			}
		}

		return false;
	}

	void FBlendStackCoreTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (FBlendStackCoreTrait::FGraphState& Graph : InstanceData->ChildGraphs)
		{
			Collector.AddReferencedObject(Graph.Request.AnimationGraph);

			// Ignore inactive graphs. Could check the graph's state but use the shared instance for extra safety.
			if (FAnimNextGraphInstance* ImplPtr = Graph.Instance.Get())
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), ImplPtr);
			}
		}
	}

	int32 FBlendStackCoreTrait::PushGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequest&& GraphRequest) const
	{
		//@TODO: Add depth limit and saturation policies

		// Validate request
		switch(GraphRequest.Type)
		{
		case EGraphRequestType::Owned:
			{
				if (GraphRequest.AnimationGraph == nullptr)
				{
					return INDEX_NONE;
				}
				break;
			}
		case EGraphRequestType::Child:
			{
				if (!GraphRequest.ChildPtr.IsValid())
				{
					return INDEX_NONE;
				}
				break;
			}
		}

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const int32 OldChildIndex = InstanceData->CurrentlyActiveGraphIndex;
		const int32 NewChildIndex = FBlendStackCoreTrait::FindFreeGraphIndexOrAdd(*InstanceData);
		FBlendStackCoreTrait::FGraphState& Graph = InstanceData->ChildGraphs[NewChildIndex];
		Graph.Initialize(MoveTemp(GraphRequest));

		InstanceData->CurrentlyActiveGraphIndex = NewChildIndex;

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);
		DiscreteBlendTrait.OnBlendTransition(Context, OldChildIndex, NewChildIndex);

		return NewChildIndex;
	}

	int32 FBlendStackCoreTrait::GetActiveGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequestPtr& OutGraphRequest) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const int32 CurrentlyActiveGraphIndex = InstanceData->CurrentlyActiveGraphIndex;
		if (CurrentlyActiveGraphIndex != INDEX_NONE)
		{
			FGraphState& GraphState = InstanceData->ChildGraphs[CurrentlyActiveGraphIndex];
			OutGraphRequest = &GraphState.Request;
			return CurrentlyActiveGraphIndex;
		}

		return INDEX_NONE;
	}

	int32 FBlendStackCoreTrait::GetActiveGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding,
		FAnimNextGraphInstance*& OutGraphInstance) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const int32 CurrentlyActiveGraphIndex = InstanceData->CurrentlyActiveGraphIndex;
		if (CurrentlyActiveGraphIndex != INDEX_NONE)
		{
			FGraphState& GraphState = InstanceData->ChildGraphs[CurrentlyActiveGraphIndex];
			OutGraphInstance = GraphState.Instance.Get();
			return CurrentlyActiveGraphIndex;
		}
		OutGraphInstance = nullptr;
		return INDEX_NONE;
	}

	IBlendStack::FGraphRequestPtr FBlendStackCoreTrait::GetGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if(!InstanceData->ChildGraphs.IsValidIndex(InChildIndex))
		{
			return nullptr;
		}

		return &InstanceData->ChildGraphs[InChildIndex].Request;
	}

	FAnimNextGraphInstance* FBlendStackCoreTrait::GetGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if(!InstanceData->ChildGraphs.IsValidIndex(InChildIndex))
		{
			return nullptr;
		}

		return InstanceData->ChildGraphs[InChildIndex].Instance.Get();
	}

	const UUAFBlendProfile* FBlendStackCoreTrait::GetBlendProfile(FExecutionContext& Context, const TTraitBinding<ISmoothBlendPerBone>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->ChildGraphs.IsValidIndex(ChildIndex))
		{
			const FGraphState& Graph = InstanceData->ChildGraphs[ChildIndex];
			return Graph.Request.BlendProfile;
		}
		else
		{
			// Unknown child
			return nullptr;
		}
	}

	void FBlendStackTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FBlendStackCoreTrait::PreUpdate(Context, Binding, TraitState);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
 
		const UObject* DesiredObject = SharedData->GetObject(Binding);
		if (DesiredObject != nullptr)
		{
			FAnimNextFactoryParams FactoryParams;
			const UUAFAnimGraph* AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, DesiredObject, FactoryParams);
			if (AnimationGraph != nullptr)
			{
				const int32 CurrentlyActiveGraphIndex = InstanceData->CurrentlyActiveGraphIndex;
				const bool bForceBlend = SharedData->GetbForceBlend(Binding);
				const bool bIsEmpty = CurrentlyActiveGraphIndex == INDEX_NONE;
				if (bForceBlend || bIsEmpty || (DesiredObject != InstanceData->ChildGraphs[CurrentlyActiveGraphIndex].Request.FactoryObject))
				{
					TTraitBinding<IBlendStack> BlendStackTrait;
					Binding.GetStackInterface(BlendStackTrait);

					IBlendStack::FGraphRequest GraphRequest;
					GraphRequest.FactoryObject = DesiredObject;
					GraphRequest.AnimationGraph = AnimationGraph;
					GraphRequest.BlendArgs.BlendTime = SharedData->GetBlendTime(Binding);
					GraphRequest.FactoryParams = MoveTemp(FactoryParams);

					BlendStackTrait.PushGraph(Context, MoveTemp(GraphRequest));
				}
			}
		}
	}

	void FBlendStackRequesterTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PreUpdate(Context, Binding, TraitState);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		const UObject* DesiredObject = SharedData->GetObject(Binding);
		if (DesiredObject != nullptr)
		{
			FAnimNextFactoryParams FactoryParams;
			const UUAFAnimGraph* AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, DesiredObject, FactoryParams);
			if (AnimationGraph != nullptr)
			{
				TTraitBinding<IBlendStack> BlendStackTrait;
				Binding.GetStackInterface(BlendStackTrait);

				IBlendStack::FGraphRequestPtr ActiveGraphRequest;
				BlendStackTrait.GetActiveGraph(Context, ActiveGraphRequest);
				const bool bForceBlend = SharedData->GetbForceBlend(Binding);
				if (bForceBlend || !ActiveGraphRequest || (DesiredObject != ActiveGraphRequest->FactoryObject))
				{
					IBlendStack::FGraphRequest GraphRequest;
					GraphRequest.FactoryObject = DesiredObject;
					GraphRequest.AnimationGraph = AnimationGraph;
					GraphRequest.BlendArgs.BlendTime = SharedData->GetBlendTime(Binding);
					GraphRequest.FactoryParams = MoveTemp(FactoryParams);

					BlendStackTrait.PushGraph(Context, MoveTemp(GraphRequest));
				}
			}
		}
	}

	void FBlendStackRequesterTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		TObjectPtr<const UObject> Object = SharedData->GetObject(Binding);
		Collector.AddReferencedObject(Object);
	}
}
