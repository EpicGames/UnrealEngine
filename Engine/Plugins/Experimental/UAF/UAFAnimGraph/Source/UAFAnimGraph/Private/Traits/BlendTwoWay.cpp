// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/BlendTwoWay.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FBlendTwoWayTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IContinuousBlend) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FBlendTwoWayTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FBlendTwoWayTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->ChildA.IsValid() && InstanceData->ChildB.IsValid())
		{
			// We have two children, interpolate them

			TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
			Binding.GetStackInterface(ContinuousBlendTrait);

			const float BlendWeight = ContinuousBlendTrait.GetBlendWeight(Context, 1);
			Context.AppendTask(FAnimNextBlendTwoKeyframesTask::Make(BlendWeight));
		}
		else
		{
			// We have only one child that is active, do nothing
		}
	}

	void FBlendTwoWayTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
		Binding.GetStackInterface(ContinuousBlendTrait);

		const bool bAlwaysUpdateChildren = SharedData->GetbAlwaysUpdateChildren(Binding);
		const bool bResetChildOnActivation = SharedData->GetbResetChildOnActivation(Binding);

		const float BlendWeightB = ContinuousBlendTrait.GetBlendWeight(Context, 1);

		const bool bIsChildARelevant = !FAnimWeight::IsFullWeight(BlendWeightB);
		const bool bIsChildBRelevant = FAnimWeight::IsRelevant(BlendWeightB);

		const bool bIsChildANeeded = bAlwaysUpdateChildren || bIsChildARelevant;
		const bool bIsChildBNeeded = bAlwaysUpdateChildren || bIsChildBRelevant;

		InstanceData->bIsChildANewlyRelevant = false;
		InstanceData->bIsChildBNewlyRelevant = false;

		if (bIsChildANeeded)
		{
			if (!InstanceData->ChildA.IsValid())
			{
				// We need to blend a child that isn't instanced yet, allocate it
				InstanceData->ChildA = Context.AllocateNodeInstance(Binding, SharedData->ChildA);
				InstanceData->bIsChildANewlyRelevant = true;
			}
			else if (bResetChildOnActivation && bIsChildARelevant && !InstanceData->bWasChildARelevant)
			{
				InstanceData->bIsChildANewlyRelevant = true;
			}
		}
		else
		{
			// We no longer need this child, release it
			InstanceData->ChildA.Reset();
		}

		if (bIsChildBNeeded)
		{
			if (!InstanceData->ChildB.IsValid())
			{
				// We need to blend a child that isn't instanced yet, allocate it
				InstanceData->ChildB = Context.AllocateNodeInstance(Binding, SharedData->ChildB);
				InstanceData->bIsChildBNewlyRelevant = true;
			}
			else if (bResetChildOnActivation && bIsChildBRelevant && !InstanceData->bWasChildBRelevant)
			{
				InstanceData->bIsChildBNewlyRelevant = true;
			}
		}
		else
		{
			// We no longer need this child, release it
			InstanceData->ChildB.Reset();
		}

		InstanceData->bWasChildARelevant = bIsChildARelevant;
		InstanceData->bWasChildBRelevant = bIsChildBRelevant;
	}

	void FBlendTwoWayTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IContinuousBlend> ContinuousBlendTrait;
		Binding.GetStackInterface(ContinuousBlendTrait);

		const float BlendWeightB = ContinuousBlendTrait.GetBlendWeight(Context, 1);
		if (InstanceData->ChildA.IsValid())
		{
			const float BlendWeightA = 1.0f - BlendWeightB;
			TraversalQueue.Push(InstanceData->ChildA, TraitState.WithWeight(BlendWeightA).AsNewlyRelevant(InstanceData->bIsChildANewlyRelevant));
		}

		if (InstanceData->ChildB.IsValid())
		{
			TraversalQueue.Push(InstanceData->ChildB, TraitState.WithWeight(BlendWeightB).AsNewlyRelevant(InstanceData->bIsChildBNewlyRelevant));
		}
	}

	uint32 FBlendTwoWayTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 2;
	}

	void FBlendTwoWayTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the two children, even if the handles are empty
		Children.Add(InstanceData->ChildA);
		Children.Add(InstanceData->ChildB);
	}

	float FBlendTwoWayTrait::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		const float BlendWeight = SharedData->GetBlendWeight(Binding);
		const float ClampedWeight = FMath::Clamp(BlendWeight, 0.0f, 1.0f);

		if (ChildIndex == 0)
		{
			return 1.0f - ClampedWeight;
		}
		else if (ChildIndex == 1)
		{
			return ClampedWeight;
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}
}
