// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/CallFunction.h"

#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphContextData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "AnimNextAnimGraphStats.h"
#include "AnimNextExecuteContext.h"
#include "Script/UAFRigVMComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CallFunction)

DEFINE_STAT(STAT_AnimNext_CallFunction);

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FCallFunctionTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate)

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FCallFunctionTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)

	#undef TRAIT_INTERFACE_ENUMERATOR

	void FCallFunctionTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		CallFunctionForMatchingSite(Binding, TraitState, EAnimNextCallFunctionCallSite::BecomeRelevant);
		IUpdate::OnBecomeRelevant(Context, Binding, TraitState);
	}

	void FCallFunctionTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		CallFunctionForMatchingSite(Binding,  TraitState, EAnimNextCallFunctionCallSite::PreUpdate);
		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FCallFunctionTrait::PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PostUpdate(Context, Binding, TraitState);
		CallFunctionForMatchingSite(Binding, TraitState,  EAnimNextCallFunctionCallSite::PostUpdate);
	}

	void FCallFunctionTrait::CallFunctionForMatchingSite(const TTraitBinding<IUpdate>& InBinding, const FTraitUpdateState& InTraitState, EAnimNextCallFunctionCallSite InCallSite) const
	{
		const FSharedData& SharedData = *InBinding.GetSharedData<FSharedData>(); 
		if(SharedData.CallSite != InCallSite || SharedData.FunctionEvent.IsNone())
		{
			return;
		}

		SCOPE_CYCLE_COUNTER(STAT_AnimNext_CallFunction);

		FNodeInstance* ParentNodeInstance = InBinding.GetTraitPtr().GetNodeInstance();
		FAnimNextGraphInstance& GraphInstance = ParentNodeInstance->GetOwner();
		FUAFRigVMComponent* RigVMComponent = GraphInstance.TryGetComponent<FUAFRigVMComponent>();
		if (RigVMComponent == nullptr)
		{
			return;
		}

		FRigVMExtendedExecuteContext& ExtendedExecuteContext = RigVMComponent->GetExtendedExecuteContext();
		FAnimNextExecuteContext& AnimNextContext = ExtendedExecuteContext.GetPublicDataSafe<FAnimNextExecuteContext>();
		FAnimNextGraphContextData ContextData(GraphInstance, SharedData.FunctionEvent, AnimNextContext.GetDeltaTime<float>());
		RigVMComponent->CallEventByName(ContextData);
	}
}
