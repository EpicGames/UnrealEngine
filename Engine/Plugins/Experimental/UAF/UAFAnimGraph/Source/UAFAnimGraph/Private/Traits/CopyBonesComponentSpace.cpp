// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/CopyBonesComponentSpace.h"

#include "EvaluationVM/Tasks/CopyBones.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CopyBonesComponentSpace)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FCopyBonesComponentSpaceTrait)

		// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \

		GENERATE_ANIM_TRAIT_IMPLEMENTATION(FCopyBonesComponentSpaceTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	void FCopyBonesComponentSpaceTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
        
        Context.AppendTask(FAnimNextCopyBonesComponentSpaceTask::Make(SharedData->BoneMapping));
	}
}
