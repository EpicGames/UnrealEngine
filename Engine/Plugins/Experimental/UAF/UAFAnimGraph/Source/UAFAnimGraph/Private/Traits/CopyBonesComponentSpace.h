// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "Traits/BoneMapping.h"

#include "CopyBonesComponentSpace.generated.h"

/** A trait that allows you to copy bone transforms in component space. */
USTRUCT(meta = (DisplayName = "Copy Bones in Component Space", ShowTooltip=true))
struct FAnimNextCopyBonesComponentSpaceTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Bones to copy transforms between
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	TArray<FAnimNextBoneMapping> BoneMapping;
};

namespace UE::UAF
{
	/**
	 * FCopyBonesComponentSpaceTrait
	 *
	 * A trait that allows bone transforms to be copied in component space.
	 */
	struct FCopyBonesComponentSpaceTrait : FAdditiveTrait, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FCopyBonesComponentSpaceTrait, FAdditiveTrait)

		using FSharedData = FAnimNextCopyBonesComponentSpaceTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData {};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
	};
}
