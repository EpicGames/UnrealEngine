// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/InertializationDeadBlend.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitInterfaces/IHierarchy.h"
#include "EvaluationVM/Tasks/DeadBlending.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include "Curves/CurveFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InertializationDeadBlend)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FInertializationDeadBlendTrait)

		// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

		// Trait implementation boilerplate
#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FInertializationDeadBlendTrait::OnInertializationRequestEvent) \

		GENERATE_ANIM_TRAIT_IMPLEMENTATION(FInertializationDeadBlendTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR
#undef TRAIT_EVENT_ENUMERATOR

	ETraitStackPropagation FInertializationDeadBlendTrait::OnInertializationRequestEvent(const FExecutionContext& Context, FTraitBinding& Binding, FAnimNextInertializationRequestEvent& Event) const
	{
		if (!Event.IsHandled())
		{
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

            // If there is no request pending, or this request has a shorter blend
            // time than the previous request then make this the active request.
			if (!InstanceData->bRequestPending ||
				Event.Request.BlendTime < InstanceData->PendingRequest.BlendTime)
			{
				InstanceData->bRequestPending = true;
				InstanceData->PendingRequest = Event.Request;
			}

            // Always mark inertialization requests as handled
			Event.MarkHandled();
		}

		return ETraitStackPropagation::Continue;
	}

	void FInertializationDeadBlendTrait::PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PostUpdate(Context, Binding, TraitState);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
        
        // Accumulate time since last evaluation
		InstanceData->EvaluateDeltaTime += TraitState.GetDeltaTime();
	}

	void FInertializationDeadBlendTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
        
        if (InstanceData->bRequestPending)
        {
            if (InstanceData->CurrPose.IsEmpty() && InstanceData->PrevPose.IsEmpty())
            {
                // We've not recorded any poses yet so effectively we have just
                // become active and either there should not be a discontinuity
                // or we cannot handle it so we effectively ignore the request

                InstanceData->bRequestPending = false;
                InstanceData->PendingRequest = UE::UAF::FInertializationRequest();
            }
            else if (InstanceData->PrevPose.IsEmpty())
            {
                // Here we have a single pose recorded so we cannot know the velocity of 
                // the animation being transitioned from. In this case we assume zero
                // velocity and construct a transition from the last pose alone.

                check(!InstanceData->CurrPose.IsEmpty());

				UE::UAF::FDeadBlendTransitionTaskParameters Parameters;
				Parameters.ExtrapolationHalfLife = SharedData->ExtrapolationHalfLife;
				Parameters.ExtrapolationHalfLifeMin = SharedData->ExtrapolationHalfLifeMin;
				Parameters.ExtrapolationHalfLifeMax = SharedData->ExtrapolationHalfLifeMax;
				Parameters.MaximumTranslationVelocity = SharedData->MaximumTranslationVelocity;
				Parameters.MaximumRotationVelocity = FMath::DegreesToRadians(SharedData->MaximumRotationVelocity);
				Parameters.MaximumScaleVelocity = SharedData->MaximumScaleVelocity;

                Context.AppendTask(FAnimNextDeadBlendingTransitionTask::Make(
					&InstanceData->State, 
					&InstanceData->CurrPose, 
					Parameters));

                InstanceData->bRequestActive = true;
                InstanceData->ActiveRequest = InstanceData->PendingRequest;
                InstanceData->bRequestPending = false;
                InstanceData->PendingRequest = UE::UAF::FInertializationRequest();
                InstanceData->TimeSinceTransition = 0.0f;
            }
            else
            {
                // Transition as normal

                check(!InstanceData->CurrPose.IsEmpty());
                check(!InstanceData->PrevPose.IsEmpty());

				UE::UAF::FDeadBlendTransitionTaskParameters Parameters;
				Parameters.ExtrapolationHalfLife = SharedData->ExtrapolationHalfLife;
				Parameters.ExtrapolationHalfLifeMin = SharedData->ExtrapolationHalfLifeMin;
				Parameters.ExtrapolationHalfLifeMax = SharedData->ExtrapolationHalfLifeMax;
				Parameters.MaximumTranslationVelocity = SharedData->MaximumTranslationVelocity;
				Parameters.MaximumRotationVelocity = FMath::DegreesToRadians(SharedData->MaximumRotationVelocity);
				Parameters.MaximumScaleVelocity = SharedData->MaximumScaleVelocity;

                Context.AppendTask(FAnimNextDeadBlendingTransitionTask::Make(
                    &InstanceData->State,
                    &InstanceData->CurrPose,
                    &InstanceData->PrevPose,
                    InstanceData->PoseDeltaTime,
					Parameters));

                InstanceData->bRequestActive = true;
                InstanceData->ActiveRequest = InstanceData->PendingRequest;
                InstanceData->bRequestPending = false;
                InstanceData->PendingRequest = UE::UAF::FInertializationRequest();
                InstanceData->TimeSinceTransition = 0.0f;
            }
        } 

        // If we have a current Inertialization Request active then add task to smooth out transition
		if (InstanceData->bRequestActive)
		{
			// Accumulate time since transition.
			 
			// Since the transition pose recorded by FAnimNextDeadBlendingTransitionTask is actually 
			// from the previous evaluation (i.e. it is the pose stored in CurrPose) even if we have
			// just transitioned this from it still makes sense here to add the evaluation delta time 
			// so that we extrapolate this forward to a pose which matches in time what is currently 
			// on top of the evaluation stack.
			
			InstanceData->TimeSinceTransition += InstanceData->EvaluateDeltaTime;

			if (InstanceData->TimeSinceTransition > InstanceData->ActiveRequest.BlendTime)
			{
				// Deactivate Dead Blending since transition is complete

				InstanceData->bRequestActive = false;
				InstanceData->ActiveRequest = UE::UAF::FInertializationRequest();
				InstanceData->TimeSinceTransition = 0.0f;
				InstanceData->State.Empty();
			}
			else
			{
				// Add Extrapolation and Blending Task

				Context.AppendTask(FAnimNextDeadBlendingApplyTask::Make(
					&InstanceData->State,
					InstanceData->ActiveRequest.BlendTime,
					InstanceData->TimeSinceTransition,
					SharedData->DefaultBlendMode,
					SharedData->DefaultCustomBlendCurve));
			}
		}

        // We only need to swap recorded poses if we've been ticked inbetween evaluation with a non trivial DeltaTime
        if (InstanceData->EvaluateDeltaTime > UE_SMALL_NUMBER)
        {
			// Record new DelaTime for pose swap/storage and reset evaluation delta time

			InstanceData->PoseDeltaTime = InstanceData->EvaluateDeltaTime;
			InstanceData->EvaluateDeltaTime = 0.0f;

            // Add tasks for swapping and storing the new output pose

            Context.AppendTask(FAnimNextSwapTransformsTask::Make(&InstanceData->PrevPose, &InstanceData->CurrPose));
            Context.AppendTask(FAnimNextStoreKeyframeTransformsTask::Make(&InstanceData->CurrPose));
        }
	}

}
