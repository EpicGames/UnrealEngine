// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AlphaBlend.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "TransformArray.h"

#include "Traits/Inertialization.h"

#include "InertializationDeadBlend.generated.h"

class UCurveFloat;

/** A trait that inertializes animation by extrapolating from the point of transition. */
USTRUCT(meta = (DisplayName = "Dead Blending", ShowTooltip=true))
struct FAnimNextInertializationDeadBlendTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// The default blend duration to use when "Always Use Default Blend Settings" is set to true.
	UPROPERTY(EditAnywhere, Category = Blending, meta = (Inline))
	float DefaultBlendDuration = 0.25f;

	// Default blend mode to use when no blend mode is supplied with the inertialization request.
	UPROPERTY(EditAnywhere, Category = Blending, meta = (Inline))
	EAlphaBlendOption DefaultBlendMode = EAlphaBlendOption::Linear;

	// Default custom blend curve to use along with the default blend mode.
	UPROPERTY(EditAnywhere, Category = Blending, meta = (Inline))
	TObjectPtr<UCurveFloat> DefaultCustomBlendCurve = nullptr;

	/**
	 * The average half-life of decay in seconds to use when extrapolating the animation. To get the final half-life of
	 * decay, this value will be scaled by the amount to which the velocities of the animation being transitioned from
	 * are moving toward the animation being transitioned to.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"), DisplayName = "Extrapolation Half Life")
	float ExtrapolationHalfLife = 1.0f;

	/**
	 * The minimum half-life of decay in seconds to use when extrapolating the animation. This will be used when the
	 * velocities of the animation being transitioned from are very small or moving away from the animation being
	 * transitioned to.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"), DisplayName = "Minimum Extrapolation Half Life")
	float ExtrapolationHalfLifeMin = 0.05f;

	/**
	 * The maximum half-life of decay in seconds to use when extrapolating the animation. This will dictate the longest
	 * decay duration possible when velocities of the animation being transitioned from are small and moving towards the
	 * animation being transitioned to.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"), DisplayName = "Maximum Extrapolation Half Life")
	float ExtrapolationHalfLifeMax = 1.0f;

	/**
	 * The maximum velocity to allow for extrapolation of bone translations in centimeters per second. Smaller values
	 * may help prevent the pose breaking during blending but too small values can make the blend less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"))
	float MaximumTranslationVelocity = 500.0f;

	/**
	 * The maximum velocity to allow for extrapolation of bone rotations in degrees per second. Smaller values
	 * may help prevent the pose breaking during blending but too small values can make the blend less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"))
	float MaximumRotationVelocity = 360.0f;

	/**
	 * The maximum velocity to allow for extrapolation of bone scales. Smaller values may help prevent the pose
	 * breaking during blending but too small values can make the blend less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"))
	float MaximumScaleVelocity = 4.0f;

	/**
	 * The maximum velocity to allow for extrapolation of curves. Smaller values may help prevent extreme curve values
	 * during blending but too small values can make the blending of curves less smooth.
	 */
	UPROPERTY(EditAnywhere, Category = Extrapolation, meta = (Inline, Min = "0.0", UIMin = "0.0"))
	float MaximumCurveVelocity = 100.0f;
};

namespace UE::UAF
{
	/**
	 * FInertializationDeadBlendTrait
	 *
	 * A trait that inertializes animation by extrapolating from the point of transition.
	 */
	struct FInertializationDeadBlendTrait : FAdditiveTrait, IUpdate, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FInertializationDeadBlendTrait, FAdditiveTrait)

		using FSharedData = FAnimNextInertializationDeadBlendTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			// Pending request details
			bool bRequestPending = false;
			FInertializationRequest PendingRequest;

			// Current active request details
			bool bRequestActive = false;
			FInertializationRequest ActiveRequest;

			// Time since the last transition
			float TimeSinceTransition = 0.0f;

			// Delta Time since the last evaluation
			float EvaluateDeltaTime = 0.0f;

			// Delta Time between the storage of Curr and Prev Poses
			float PoseDeltaTime = 0.0f;

			// Recorded Current Pose State
			FTransformArraySoAHeap CurrPose;

			// Recorded Previous Pose State
			FTransformArraySoAHeap PrevPose;

			// Extrapolation State Data
			FDeadBlendingState State;
		};

		// InertializationRequestTrait handler
		ETraitStackPropagation OnInertializationRequestEvent(const FExecutionContext& Context, FTraitBinding& Binding, FAnimNextInertializationRequestEvent& Event) const;

		// IUpdate impl
		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	};
}
