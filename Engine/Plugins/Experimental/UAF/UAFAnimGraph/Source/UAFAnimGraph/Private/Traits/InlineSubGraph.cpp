// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/InlineSubGraph.h"

#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/VirtualValueBundle_InlineGraph.h"
#include "Module/AnimNextModuleInstance.h"
#include "UAF/ValueRuntime/ValueBundle.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FUAFInlineSubGraphTrait)

	// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IDiscreteBlend) \
	GeneratorMacro(IGarbageCollection) \
	GeneratorMacro(IHierarchy) \
	GeneratorMacro(IUpdate) \
	GeneratorMacro(IUpdateTraversal) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FUAFInlineSubGraphTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	void FUAFInlineSubGraphTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		// Allocate the reference pose fallback child
		ReferencePoseChildPtr = Context.AllocateNodeInstance(Binding, SharedData->ReferencePoseChild);
	}

	void FUAFInlineSubGraphTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Resolve the latent Inputs pin here, during update traversal, and cache the result.
		InstanceData->ResolvedInputs = SharedData->GetInputs(Binding);
	}

	void FUAFInlineSubGraphTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const bool bHasActiveSubGraph = (InstanceData->CurrentlyActiveSubGraphIndex != INDEX_NONE) && ensure(InstanceData->SubGraphSlots.IsValidIndex(InstanceData->CurrentlyActiveSubGraphIndex));

		TObjectPtr<const UUAFAnimGraph> CurrentActiveAnimationGraph;
		if (bHasActiveSubGraph)
		{
			FSubGraphSlot& ActiveSlot = InstanceData->SubGraphSlots[InstanceData->CurrentlyActiveSubGraphIndex];
			CurrentActiveAnimationGraph = ActiveSlot.AnimationGraph;
			ActiveSlot.bWasRelevant = true;
		}

		const TObjectPtr<const UUAFAnimGraph> DesiredAnimationGraph = SharedData->GetGraph(Binding);

		// Note: Not guarding against re-entrant sub-graph references here as higher-level recursion checks should catch this

		// Transition to a new sub-graph instance if the desired graph has changed
		if (!bHasActiveSubGraph || CurrentActiveAnimationGraph != DesiredAnimationGraph)
		{
			int32 FreeSlotIndex = INDEX_NONE;
			const int32 NumSlots = InstanceData->SubGraphSlots.Num();
			for (int32 SlotIndex = 0; SlotIndex < NumSlots; ++SlotIndex)
			{
				if (InstanceData->SubGraphSlots[SlotIndex].State == ESlotState::Inactive)
				{
					FreeSlotIndex = SlotIndex;
					break;
				}
			}

			if (FreeSlotIndex == INDEX_NONE)
			{
				FreeSlotIndex = InstanceData->SubGraphSlots.AddDefaulted();
			}

			FSubGraphSlot& NewSlot = InstanceData->SubGraphSlots[FreeSlotIndex];
			NewSlot.AnimationGraph = DesiredAnimationGraph;
			NewSlot.State = DesiredAnimationGraph ? ESlotState::ActiveWithGraph : ESlotState::ActiveWithReferencePose;

			const int32 OldChildIndex = InstanceData->CurrentlyActiveSubGraphIndex;
			InstanceData->CurrentlyActiveSubGraphIndex = FreeSlotIndex;

			TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
			Binding.GetStackInterface(DiscreteBlendTrait);
			DiscreteBlendTrait.OnBlendTransition(Context, OldChildIndex, FreeSlotIndex);
		}
	}

	void FUAFInlineSubGraphTrait::PushInputsToInnerGraph(const FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, const FSubGraphSlot& InSlot) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const TArray<FUAFInlineSubGraphInputBinding>& Inputs = InstanceData->ResolvedInputs;

		// Iterate in lock-step with the slot's InputChildPtrs, allocated in OnBlendInitiated
		const int32 NumInputs = FMath::Min(Inputs.Num(), InSlot.InputChildPtrs.Num());
		for (int32 InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
		{
			const FTraitPtr& ChildPtr = InSlot.InputChildPtrs[InputIndex];
			const FAnimNextVariableReference& InnerVar = Inputs[InputIndex].Variable;

			if (!ChildPtr.IsValid() || InnerVar.IsNone() || !InSlot.GraphInstance.IsValid())
			{
				continue;
			}

			// Wrap the outer child as an inline graph bundle and write it directly into the inner graph variable
			InSlot.GraphInstance->AccessVariable<FUAFValueBundle>(InnerVar, [&ChildPtr](FUAFValueBundle& InBundle)
			{
				InBundle.SetAs<FVirtualValueBundle_InlineGraph>(ChildPtr);
			});
		}
	}

	void FUAFInlineSubGraphTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Traverse active sub-graph slots
		const int32 NumSubGraphs = InstanceData->SubGraphSlots.Num();
		if (NumSubGraphs > 0)
		{
			TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
			Binding.GetStackInterface(DiscreteBlendTrait);

			for (int32 SlotIndex = 0; SlotIndex < NumSubGraphs; ++SlotIndex)
			{
				const FSubGraphSlot& Slot = InstanceData->SubGraphSlots[SlotIndex];
				if (Slot.State == ESlotState::ActiveWithGraph)
				{
					if (Slot.AnimationGraph == nullptr || !Slot.GraphInstance.IsValid())
					{
						continue;
					}

					const float BlendWeight = DiscreteBlendTrait.GetBlendWeight(Context, SlotIndex);
					const bool bGraphHasNeverUpdated = !Slot.GraphInstance->HasUpdated();

					FTraitUpdateState SlotTraitState = TraitState
						.WithWeight(BlendWeight)
						.AsBlendingOut(SlotIndex != InstanceData->CurrentlyActiveSubGraphIndex)
						.AsNewlyRelevant(!Slot.bWasRelevant || bGraphHasNeverUpdated);

					Slot.GraphInstance->MarkAsUpdated();
					TraversalQueue.Push(Slot.GraphInstance->GetGraphRootPtr().IsValid() ? Slot.GraphInstance->GetGraphRootPtr() : InstanceData->ReferencePoseChildPtr, SlotTraitState);
				}
				else if (Slot.State == ESlotState::ActiveWithReferencePose)
				{
					const float BlendWeight = DiscreteBlendTrait.GetBlendWeight(Context, SlotIndex);
					
					FTraitUpdateState SlotTraitState = TraitState
						.WithWeight(BlendWeight)
						.AsBlendingOut(SlotIndex != InstanceData->CurrentlyActiveSubGraphIndex)
						.AsNewlyRelevant(!Slot.bWasRelevant);
					
					TraversalQueue.Push(InstanceData->ReferencePoseChildPtr, SlotTraitState);
				}
			}
		}
	}

	uint32 FUAFInlineSubGraphTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		uint32 NumChildren = 0;

		for (const FSubGraphSlot& Slot : InstanceData->SubGraphSlots)
		{
			if (Slot.State != ESlotState::Inactive)
			{
				++NumChildren;
			}
		}

		return NumChildren;
	}

	void FUAFInlineSubGraphTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (const FSubGraphSlot& Slot : InstanceData->SubGraphSlots)
		{
			if (Slot.State == ESlotState::ActiveWithReferencePose)
			{
				Children.Add(InstanceData->ReferencePoseChildPtr);
			}
			else if (Slot.State == ESlotState::ActiveWithGraph)
			{
				Children.Add((Slot.GraphInstance.IsValid() && Slot.GraphInstance->GetGraphRootPtr().IsValid()) ? Slot.GraphInstance->GetGraphRootPtr() : InstanceData->ReferencePoseChildPtr);
			}
		}
	}

	float FUAFInlineSubGraphTrait::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (ChildIndex == InstanceData->CurrentlyActiveSubGraphIndex)
		{
			return 1.0f;
		}
		else if (InstanceData->SubGraphSlots.IsValidIndex(ChildIndex))
		{
			return 0.0f;
		}

		return -1.0f;
	}

	int32 FUAFInlineSubGraphTrait::GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->CurrentlyActiveSubGraphIndex;
	}

	void FUAFInlineSubGraphTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// Immediate initiate-then-terminate keeps behaviour consistent with the base SubGraph trait
		DiscreteBlendTrait.OnBlendInitiated(Context, NewChildIndex);
		DiscreteBlendTrait.OnBlendTerminated(Context, OldChildIndex);
	}

	void FUAFInlineSubGraphTrait::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->SubGraphSlots.IsValidIndex(ChildIndex))
		{
			FSubGraphSlot& Slot = InstanceData->SubGraphSlots[ChildIndex];
			if (Slot.State == ESlotState::ActiveWithGraph)
			{
				const TArray<FUAFInlineSubGraphInputBinding>& Inputs = InstanceData->ResolvedInputs;

				// Allocate a separate set of outer child ptrs for this slot so blending-in and
				// blending-out slots each independently re-enter the outer graph traversal
				Slot.InputChildPtrs.SetNum(Inputs.Num());
				for (int32 i = 0; i < Inputs.Num(); ++i)
				{
					Slot.InputChildPtrs[i] = Context.AllocateNodeInstance(Binding, Inputs[i].Input);
				}

				FAnimNextGraphInstance& Owner = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
				Slot.GraphInstance = Slot.AnimationGraph->AllocateInstance(
					{
						.SystemReference = Owner.GetModuleInstanceReference(),
						.ParentGraphInstance = Owner.AsShared(),
					});

				PushInputsToInnerGraph(Context, Binding, Slot);
			}
		}
	}

	void FUAFInlineSubGraphTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->SubGraphSlots.IsValidIndex(ChildIndex))
		{
			FSubGraphSlot& Slot = InstanceData->SubGraphSlots[ChildIndex];
			if (Slot.State == ESlotState::ActiveWithGraph)
			{
				Slot.GraphInstance.Reset();
			}
			Slot.InputChildPtrs.Reset();
			Slot.State = ESlotState::Inactive;
			Slot.bWasRelevant = false;
		}
	}

	void FUAFInlineSubGraphTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (FSubGraphSlot& Slot : InstanceData->SubGraphSlots)
		{
			Collector.AddReferencedObject(Slot.AnimationGraph);
			if (FAnimNextGraphInstance* ImplPtr = Slot.GraphInstance.Get())
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), ImplPtr);
			}
		}
	}

} // namespace UE::UAF
