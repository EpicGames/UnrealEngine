// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Traits/InlineSubGraphTraitData.h"

namespace UE::UAF
{

/**
 * FUAFInlineSubGraphTrait
 *
 * A trait that hosts and manages a sub-graph instance while maintaining a set of inline child
 * handles that act as passthrough connections to the outer owning graph.
 * Each blending slot owns its own set of allocated input child ptrs so that blending-in and
 * blending-out instances remain independent.
 */
struct FUAFInlineSubGraphTrait
	: FBaseTrait
	, IUpdate
	, IUpdateTraversal
	, IHierarchy
	, IDiscreteBlend
	, IGarbageCollection
{
	DECLARE_ANIM_TRAIT(FUAFInlineSubGraphTrait, FBaseTrait)

	enum class ESlotState : uint8
	{
		ActiveWithGraph,
		ActiveWithReferencePose,
		Inactive,
	};

	struct FSubGraphSlot
	{
		// The animation graph asset for this slot
		TObjectPtr<const UUAFAnimGraph> AnimationGraph;

		// The live graph instance for this slot
		TSharedPtr<FAnimNextGraphInstance> GraphInstance;

		// FTraitPtrs allocated from ResolvedInputs during OnBlendInitiated, one per Inputs entry.
		// Each ptr is wrapped as a FVirtualValueBundle_InlineGraph and written into the inner
		// graph's corresponding interface variable. Allocated per slot so blending-in and
		// blending-out instances each hold an independent set.
		TArray<FTraitPtr> InputChildPtrs;

		// Current lifecycle state of the slot
		ESlotState State = ESlotState::Inactive;

		// Whether this slot was relevant on the previous update tick
		bool bWasRelevant = false;
	};

	using FSharedData = FUAFInlineSubGraphTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
		void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);

		// Sub-graph slots managing hosted inner graph instances
		TArray<FSubGraphSlot> SubGraphSlots;

		// Index of the currently active sub-graph slot (INDEX_NONE if none)
		int32 CurrentlyActiveSubGraphIndex = INDEX_NONE;

		// Child used to output the reference pose when no graph is provided
		FTraitPtr ReferencePoseChildPtr;

		// Inputs resolved from the latent pin during OnBecomeRelevant, cached here so that
		// OnBlendInitiated (which runs in an execution context) can use them without re-accessing
		// the latent system outside of update traversal.
		TArray<FUAFInlineSubGraphInputBinding> ResolvedInputs;
	};

	// IUpdate impl
	virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IUpdateTraversal impl
	virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

	// IHierarchy impl
	virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
	virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

	// IDiscreteBlend impl
	virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
	virtual int32 GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const override;
	virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;
	virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
	virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;

	// IGarbageCollection impl
	virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

private:
	// For each entry in ResolvedInputs, wraps the slot's corresponding FTraitPtr as a
	// FVirtualValueBundle_InlineGraph and writes it into the inner graph instance via AccessVariable.
	void PushInputsToInnerGraph(const FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, const FSubGraphSlot& InSlot) const;
};

} // namespace UE::UAF
