// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/InputPoseTrait.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/PushPose.h"
#include "Traits/InputValueTrait.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputPoseTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FInputPoseTrait)

		// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \

		GENERATE_ANIM_TRAIT_IMPLEMENTATION(FInputPoseTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	void FInputPoseTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FAnimNextGraphLODPose& InputPose = SharedData->GetInput(Binding);

		if (InputPose.LODPose.IsValid())
		{
			Context.AppendTask(FAnimNextPushPoseTask::Make(&InputPose));
		}
	}

	FRigVMStructUpgradeInfo FInputPoseTrait::GetUpgradeInfo() const
	{
		return FRigVMStructUpgradeInfo(FAnimNextInputPoseTraitSharedData(), FUAFInputValueTraitSharedData());
	}
}
