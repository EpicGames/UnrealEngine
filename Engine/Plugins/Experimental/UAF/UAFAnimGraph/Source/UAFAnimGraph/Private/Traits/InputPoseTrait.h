// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "Graph/AnimNext_LODPose.h"

#include "InputPoseTrait.generated.h"

USTRUCT(meta = (Deprecated = "5.8", DisplayName = "Input Pose (Outdated)", ShowTooltip=true))
struct FAnimNextInputPoseTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimNextGraphLODPose Input;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Input)

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextInputPoseTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	struct FInputPoseTrait : FBaseTrait, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FInputPoseTrait, FBaseTrait)

		using FSharedData = FAnimNextInputPoseTraitSharedData;

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// FTrait interface
		virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
	};
}