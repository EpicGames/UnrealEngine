// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/InputValueTrait.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/PushPose.h"
#include "EvaluationVM/Tasks/PushReferenceKeyframe.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Graph/WeakAnimGraphReference.h"
#include "TraitCore/NodeInstance.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputValueTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FInputValueTrait)

	// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IUpdate) \
	GeneratorMacro(IEvaluate) \
	GeneratorMacro(IUpdateTraversal) \
	GeneratorMacro(IHierarchy) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FInputValueTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	const FInputValueTrait::FInstanceData* FInputValueTrait::UpdateInstanceData(const FTraitBinding& InBinding) const
	{
		const FSharedData* SharedData = InBinding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = InBinding.GetInstanceData<FInstanceData>();
		InstanceData->VariableReference = SharedData->GetInput(InBinding);
		InstanceData->GraphInstance = &InBinding.GetTraitPtr().GetNodeInstance()->GetOwner();
		check(InstanceData->GraphInstance);
		return InstanceData;
	}

	void FInputValueTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		(void)UpdateInstanceData(Binding);
	}

	void FInputValueTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FInstanceData* InstanceData = UpdateInstanceData(Binding);
		InstanceData->GraphInstance->AccessVariable<FUAFValueBundle>(InstanceData->VariableReference, [&Context, &Binding, &TraitState](FUAFValueBundle& InValueBundle)
		{
			if (const IVirtualValueBundle* Impl = InValueBundle.GetImpl())
			{
				if (const FWeakAnimGraphReference* GraphReference = Impl->GetAnimGraphReference())
				{
					(void)GraphReference->GetOrAllocate(Context, Binding);
				}
			}
		});
	}

	void FInputValueTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		bool bEvaluated = false;
		const FInstanceData* InstanceData = UpdateInstanceData(Binding);
		InstanceData->GraphInstance->AccessVariable<FUAFValueBundle>(InstanceData->VariableReference, [&bEvaluated, &Context, &Binding](FUAFValueBundle& InValueBundle)
		{
			if (const IVirtualValueBundle* Impl = InValueBundle.GetImpl())
			{
				// If this value bundle represents a concrete value, then we need to use it, otherwise assume it is virtual and any task generation is
				// handled by the PreEvaluate call above
				if (const FAnimNextGraphLODPose* LODPose = Impl->GetLODPose())
				{
					// TODO: Implement with new attribute runtime
					if (LODPose->LODPose.IsValid())
					{
						Context.AppendTask(FAnimNextPushPoseTask::Make(LODPose));
					}
				}

				bEvaluated = true;
			}

		});

		if (!bEvaluated)
		{
			Context.AppendTask(FAnimNextPushReferenceKeyframeTask::MakeFromSkeleton());
		}
	}

	void FInputValueTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = UpdateInstanceData(Binding);
		InstanceData->GraphInstance->AccessVariable<FUAFValueBundle>(InstanceData->VariableReference, [&Context, &Binding, &TraitState, &TraversalQueue](FUAFValueBundle& InValueBundle)
		{
			if (const IVirtualValueBundle* Impl = InValueBundle.GetImpl())
			{
				if (const FWeakAnimGraphReference* GraphReference = Impl->GetAnimGraphReference())
				{
					TSharedPtr<FAnimNextGraphInstance> GraphInstance = GraphReference->GetOrAllocate(Context, Binding);
					if (GraphInstance.IsValid() && GraphInstance->GetGraphRootPtr().IsValid())
					{
						// External graph instances need to avoid re-initializing based around host relevancy
						// Effectively they are in charge of their initial relevancy
						TraversalQueue.Push(GraphInstance->GetGraphRootPtr(), TraitState.AsNewlyRelevant(!GraphInstance->HasUpdated()));
						GraphInstance->MarkAsUpdated();
					}
				}
				else if (const FTraitPtr* TraitPtr = Impl->GetInlineGraph())
				{
					if (TraitPtr->IsValid())
					{
						TraversalQueue.Push(*TraitPtr, TraitState);
					}
				}
			}
		});
	}
	
	uint32 FInputValueTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		uint32 NumChildren = 0;
		const FInstanceData* InstanceData = UpdateInstanceData(Binding);
		InstanceData->GraphInstance->AccessVariable<FUAFValueBundle>(InstanceData->VariableReference, [&Context, &Binding, &NumChildren](FUAFValueBundle& InValueBundle)
		{
			if (const IVirtualValueBundle* Impl = InValueBundle.GetImpl())
			{
				if (const FWeakAnimGraphReference* GraphReference = Impl->GetAnimGraphReference())
				{
					TSharedPtr<FAnimNextGraphInstance> GraphInstance = GraphReference->GetOrAllocate(Context, Binding);
					if (GraphInstance.IsValid() && GraphInstance->GetGraphRootPtr().IsValid())
					{
						NumChildren = 1;
					}
				}
				else if (const FTraitPtr* TraitPtr = Impl->GetInlineGraph())
				{
					if (TraitPtr->IsValid())
					{
						NumChildren = 1;
					}
				}
			}
		});

		return NumChildren;
	}

	void FInputValueTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = UpdateInstanceData(Binding);
		InstanceData->GraphInstance->AccessVariable<FUAFValueBundle>(InstanceData->VariableReference, [&Context, &Binding, &Children](FUAFValueBundle& InValueBundle)
		{
			if (const IVirtualValueBundle* Impl = InValueBundle.GetImpl())
			{
				if (const FWeakAnimGraphReference* GraphReference = Impl->GetAnimGraphReference())
				{
					TSharedPtr<FAnimNextGraphInstance> GraphInstance = GraphReference->GetOrAllocate(Context, Binding);
					if (GraphInstance.IsValid() && GraphInstance->GetGraphRootPtr().IsValid())
					{
						Children.Add(GraphInstance->GetGraphRootPtr());
					}
				}
				else if (const FTraitPtr* TraitPtr = Impl->GetInlineGraph())
				{
					if (TraitPtr->IsValid())
					{
						Children.Add(*TraitPtr);
					}
				}
			}
		});
	}
}
