// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/MakeDynamicAdditiveTrait.h"

#include "EvaluationVM/Tasks/ConvertRotationsToMeshSpace.h"
#include "EvaluationVM/Tasks/MakeDynamicAdditive.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FMakeDynamicAdditiveTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMakeDynamicAdditiveTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FMakeDynamicAdditiveTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);
		
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (!InstanceData->Base.IsValid())
		{
			InstanceData->Base = Context.AllocateNodeInstance(Binding, SharedData->Base);
		}

		if (!InstanceData->Additive.IsValid())
		{
			InstanceData->Additive = Context.AllocateNodeInstance(Binding, SharedData->Additive);
		}
	}

	void FMakeDynamicAdditiveTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->Additive.IsValid() && InstanceData->Base.IsValid())
		{
			const bool bMeshSpaceAdditive = SharedData->GetbMeshSpaceAdditive(Binding);
			if (bMeshSpaceAdditive)
			{
				Context.AppendTask(FAnimNextConvertRotationsToMeshSpaceTask::Make(2));
			}
			
			Context.AppendTask(FUAFMakeDynamicAdditiveTask::Make(bMeshSpaceAdditive ? AAT_RotationOffsetMeshSpace : AAT_LocalSpaceBase));
		}
	}

	uint32 FMakeDynamicAdditiveTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 2;
	}

	void FMakeDynamicAdditiveTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Add the two children, even if the handles are empty
		Children.Add(InstanceData->Base);
		Children.Add(InstanceData->Additive);
	}
}
