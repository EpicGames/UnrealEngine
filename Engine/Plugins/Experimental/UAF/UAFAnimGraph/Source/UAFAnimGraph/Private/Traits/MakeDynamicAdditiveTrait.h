// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/MakeDynamicAdditiveTraitData.h"


namespace UE::UAF
{
	/**
	 * Make Dynamic Additive Trait
	 * 
	 * A trait that can generate an additive pose based on the input poses
	 * The Additive input is the pose to be turned additive 
	 * The Base input is the base to generate the additive against
	 */
	struct FMakeDynamicAdditiveTrait : FBaseTrait, IEvaluate, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FMakeDynamicAdditiveTrait, FBaseTrait)
		
		using FSharedData = FMakeDynamicAdditiveTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Base;
			FTraitPtr Additive;
			
			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};
		
		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
		
		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
	};
}
