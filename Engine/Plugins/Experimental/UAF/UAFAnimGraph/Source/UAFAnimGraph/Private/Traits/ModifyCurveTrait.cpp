// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifyCurveTrait.h"

#include "EvaluationVM/EvaluationVM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModifyCurveTrait)

namespace UE::UAF
{
AUTO_REGISTER_ANIM_TRAIT(FModifyCurveTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
GeneratorMacro(IEvaluate) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FModifyCurveTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR
}

void UE::UAF::FModifyCurveTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
{
	IEvaluate::PostEvaluate(Context, Binding);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	check(SharedData);

	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	check(InstanceData);

	float Alpha = SharedData->GetAlpha(Binding);
	EAnimNext_ModifyCurveApplyMode ApplyMode = SharedData->GetApplyMode(Binding);
	TConstArrayView<FModifyCurveParameters> CurveParameters = SharedData->GetModifyCurveParameters(Binding);;

#if ENABLE_ANIM_DEBUG 
	InstanceData->HostObject = Context.GetHostObject();
#endif // ENABLE_ANIM_DEBUG 

	Context.AppendTask(FModifyCurveTask::Make(Alpha, ApplyMode, CurveParameters));
}

FModifyCurveTask FModifyCurveTask::Make(float Alpha, EAnimNext_ModifyCurveApplyMode ApplyMode, TConstArrayView<FModifyCurveParameters> CurveParameters)
{
	FModifyCurveTask Task;
	Task.Alpha = Alpha;
	Task.ApplyMode = ApplyMode;
	Task.ModifyCurveParameters = CurveParameters;
	return Task;
}

void FModifyCurveTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		if (const TUniquePtr<FValueBundle>* Collection = VM.PeekValue<TUniquePtr<FValueBundle>>(ATTRIBUTE_STACK_NAME, 0))
		{
			FAttributeNamedSetPtr NamedSet = (*Collection)->GetNamedSet();
			if (FAttributeTypedSetPtr FloatTypedSet = NamedSet->FindTypedSet<FFloatAnimationAttribute>())
			{
				FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>();
				
				if (TUnboundValueMap<FFloatAnimationAttribute>* FloatMap = (*Collection)->GetUnboundValueMaps().Find<FFloatAnimationAttribute>())
				{
					for (const FModifyCurveParameters& CurveParameters : ModifyCurveParameters)
					{
						if (FFloatAnimationAttribute* Float = FloatMap->Find(CurveParameters.CurveName))
						{
							const float CurrentValue = (*Float).Value;
							const float UpdatedValue = ProcessCurveOperation(CurrentValue, CurveParameters.CurveValue, Alpha, ApplyMode);
							(*Float).Value = UpdatedValue;
						}
					}
				}

				if (TBoundValueMap<FFloatAnimationAttribute>* FloatMap = (*Collection)->GetBoundValueMaps().Find<FFloatAnimationAttribute>(MappingKey))
				{
					for (const FModifyCurveParameters& CurveParameters : ModifyCurveParameters)
					{
						if (const FAttributeSetIndex RootMotionAttributeIndex = FloatTypedSet->FindIndex(CurveParameters.CurveName))
						{
							const float CurrentValue = (*FloatMap)[RootMotionAttributeIndex].Value;
							const float UpdatedValue = ProcessCurveOperation(CurrentValue, CurveParameters.CurveValue, Alpha, ApplyMode);
							(*FloatMap)[RootMotionAttributeIndex].Value = UpdatedValue;
						}
					}
				}
			}
		}
	}
	
	if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
	{
		for (const FModifyCurveParameters& CurveParameters : ModifyCurveParameters)
		{
			float CurrentValue = Keyframe->Get()->Curves.Get(CurveParameters.CurveName);
			float UpdatedValue = ProcessCurveOperation(CurrentValue, CurveParameters.CurveValue, Alpha, ApplyMode);
			Keyframe->Get()->Curves.Set(CurveParameters.CurveName, UpdatedValue);
		}
	}
}

float FModifyCurveTask::ProcessCurveOperation(float CurrentValue, float NewValue, float Alpha, EAnimNext_ModifyCurveApplyMode ApplyMode)
{
	float UseNewValue = CurrentValue;

	// Use ApplyMode enum to decide how to apply
	if (ApplyMode == EAnimNext_ModifyCurveApplyMode::Add)
	{
		UseNewValue = CurrentValue + NewValue;
	}
	else if (ApplyMode == EAnimNext_ModifyCurveApplyMode::Scale)
	{
		UseNewValue = CurrentValue * NewValue;
	}
	else if (ApplyMode == EAnimNext_ModifyCurveApplyMode::Blend)
	{
		UseNewValue = NewValue;
	}

	const float UseAlpha = FMath::Clamp(Alpha, 0.f, 1.f);
	return FMath::Lerp(CurrentValue, UseNewValue, UseAlpha);
}
