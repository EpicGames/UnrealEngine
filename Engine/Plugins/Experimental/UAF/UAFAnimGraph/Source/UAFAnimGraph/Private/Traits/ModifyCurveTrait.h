// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/Args/IAlphaInputArgs.h"

#include "ModifyCurveTrait.generated.h"

UENUM()
enum class EAnimNext_ModifyCurveApplyMode : uint8
{
	/** Add new value to input curve value */
	Add,

	/** Scale input value by new value */
	Scale,

	/** Blend input with new curve value, using Alpha setting on the node */
	Blend,
};

USTRUCT()
struct FModifyCurveParameters
{
	GENERATED_BODY()

	UPROPERTY(Category=CurveData, EditAnywhere)
	FName CurveName;

	UPROPERTY(Category=CurveData, EditAnywhere)
	float CurveValue = 0.0f;
	
};

/** A trait that can write to a specific float curve value */
USTRUCT(meta = (DisplayName = "Modify Curve", ShowTooltip=true, AllowDuplicates = "true"))
struct FAnimNextModifyCurveTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(Category=Settings, EditAnywhere)
	EAnimNext_ModifyCurveApplyMode ApplyMode = EAnimNext_ModifyCurveApplyMode::Add;

	UPROPERTY(Category=Settings, EditAnywhere)
	TArray<FModifyCurveParameters> ModifyCurveParameters;

	UPROPERTY(Category=Settings, EditAnywhere, meta=(ClampMin=0, ClampMax=1))
	float Alpha = 1.0f;

#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(ApplyMode) \
	GeneratorMacro(Alpha) \
	GeneratorMacro(ModifyCurveParameters) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextModifyCurveTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

	/**
	 * FModifyCurveTrait
	 * 
	 * A trait that can write to a specific float curve value
	 */
	struct FModifyCurveTrait : FAdditiveTrait, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FModifyCurveTrait, FAdditiveTrait)

		using FSharedData = FAnimNextModifyCurveTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
#if ENABLE_ANIM_DEBUG 
			/** Debug Object for VisualLogger */
			TObjectPtr<const UObject> HostObject = nullptr;
#endif // ENABLE_ANIM_DEBUG 
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
	};
}

USTRUCT()
struct FModifyCurveTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FModifyCurveTask)

	static FModifyCurveTask Make(float Alpha, EAnimNext_ModifyCurveApplyMode ApplyMode, TConstArrayView<FModifyCurveParameters> CurveParameters);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:
	static float ProcessCurveOperation(float CurrentValue, float NewValue, float Alpha, EAnimNext_ModifyCurveApplyMode ApplyMode);

	float Alpha = 1.0f;
	EAnimNext_ModifyCurveApplyMode ApplyMode = EAnimNext_ModifyCurveApplyMode::Add;
	TArray<FModifyCurveParameters, TInlineAllocator<8>> ModifyCurveParameters;
};
