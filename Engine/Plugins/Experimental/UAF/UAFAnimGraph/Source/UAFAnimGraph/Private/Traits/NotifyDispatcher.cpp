// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/NotifyDispatcher.h"
#include "TraitInterfaces/INotifySource.h"
#include "TraitInterfaces/ITimeline.h"
#include "NotifyFilterTrait.h"
#include "Animation/AnimTrace.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFNotifyDispatcherComponent.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FNotifyDispatcherTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(ITimelinePlayer) \

		// Trait required interfaces implementation boilerplate
	#define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacroRequired) \
		GeneratorMacroRequired(ITimelinePlayer) \
		GeneratorMacroRequired(ITimeline) \
		GeneratorMacroRequired(INotifySource) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FNotifyDispatcherTrait, TRAIT_INTERFACE_ENUMERATOR, TRAIT_REQUIRED_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_REQUIRED_INTERFACE_ENUMERATOR
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FNotifyDispatcherTrait::AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		TTraitBinding<ITimeline> TimelineTrait;
		ensure(Binding.GetStackInterfaceSuper(TimelineTrait));

		TTraitBinding<ITimelinePlayer> TimelinePlayerTrait;
		ensure(Binding.GetStackInterfaceSuper(TimelinePlayerTrait));

		// Get current state from the stack, advance time, then get the delta state
		FTimelineState PreAdvanceState;
		const bool bHasState = TimelineTrait.GetState(Context, PreAdvanceState);

		TimelinePlayerTrait.AdvanceBy(Context, DeltaTime, bDispatchEvents);

		if (!bDispatchEvents || !bHasState)
		{
			return;
		}

		if (!FNotifyFilterTrait::AreNotifiesEnabledInScope(Context))
		{
			return;
		}

		FTimelineDelta Delta;
		if (ensure(TimelineTrait.GetDelta(Context, Delta)))
		{
			TTraitBinding<INotifySource> NotifySourceTrait;
			ensure(Binding.GetStackInterfaceSuper(NotifySourceTrait));

			// Query for notifies
			TArray<FAnimNotifyEventReference> Notifies;
			NotifySourceTrait.GetNotifies(Context, PreAdvanceState.GetPosition(), Delta.GetDeltaTime(), PreAdvanceState.IsLooping(), Notifies);

			// Dispatch if we found any
			if(Notifies.Num() > 0)
			{
				// Ensure we have a handler component on the module
				if(FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
				{
					(void)ModuleInstance->GetOrAddComponent<FUAFNotifyDispatcherComponent>();
				}

				auto NotifyDispatchEvent = MakeTraitEvent<FNotifyDispatchEvent>();
				NotifyDispatchEvent->Notifies = MoveTemp(Notifies);
				NotifyDispatchEvent->Weight = 1.0f;
				Context.RaiseOutputTraitEvent(NotifyDispatchEvent);
			}
		}
	}
}
