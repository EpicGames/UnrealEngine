// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/ITimelinePlayer.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/NotifyDispatcherTraitData.h"

namespace UE::UAF
{
	/**
	 * FNotifyDispatcherTrait
	 * 
	 * A trait that dispatches notifies according to a timeline advancing
	 */
	struct FNotifyDispatcherTrait : FAdditiveTrait, ITimelinePlayer
	{
		DECLARE_ANIM_TRAIT(FNotifyDispatcherTrait, FAdditiveTrait)

		using FSharedData = FAnimNextNotifyDispatcherTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
		};

		// ITimelinePlayer impl
		virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const override;
	};
}
