// Copyright Epic Games, Inc. All Rights Reserved.

#include "NotifyFilterTrait.h"
#include "TraitCore/ExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NotifyFilterTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FNotifyFilterTrait)

	FLazyName FNotifyFilterTrait::DisableNotifiesTag("DisableNotifies"); 
	
	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IScopedTagInterface) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FNotifyFilterTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	bool FNotifyFilterTrait::AreNotifiesEnabledInScope(const FExecutionContext& Context)
	{
		return !IScopedTagInterface::IsTagInScope(Context, DisableNotifiesTag);		
	}
}

