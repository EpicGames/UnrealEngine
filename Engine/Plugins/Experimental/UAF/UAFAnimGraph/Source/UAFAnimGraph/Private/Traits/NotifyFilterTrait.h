// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/IScopedTraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "TraitInterfaces/IScopedTag.h"
#include "TraitInterfaces/IUpdate.h"

#include "NotifyFilterTrait.generated.h"

USTRUCT(meta = (DisplayName = "Notify Filter"))
struct FAnimNextNotifyFilterTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Filtering")
	bool bShouldDisableNotifies = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(bShouldDisableNotifies) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextNotifyFilterTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	struct FNotifyFilterTrait : FAdditiveTrait, IScopedTagInterface, IUpdate
	{
		DECLARE_ANIM_TRAIT(FNotifyFilterTrait, FAdditiveTrait)

		using FSharedData = FAnimNextNotifyFilterTraitSharedData;
		
		static bool AreNotifiesEnabledInScope(const FExecutionContext& Context);

		// IScopedTagInterface impl
		virtual FName GetTag(const FExecutionContext& Context, const TTraitBinding<IScopedTagInterface>& Binding) const override
		{
			return DisableNotifiesTag;
		}		

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
			if (SharedData->GetbShouldDisableNotifies(Binding))
			{
				Context.PushScopedInterface<IScopedTagInterface>(Binding);
			}
			
			IUpdate::PreUpdate(Context, Binding, TraitState);
		}

		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{			
			const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
			if (SharedData->GetbShouldDisableNotifies(Binding))
			{
				ensure(Context.PopScopedInterface<IScopedTagInterface>(Binding));
			}

			IUpdate::PostUpdate(Context, Binding, TraitState);
		}
	private:
		static FLazyName DisableNotifiesTag; 
	};
}
