// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/PassthroughTraitData.h"

namespace UE::UAF
{
	/**
	 * FPassthroughTrait
	 *
	 * A trait that passes through the input without modification.
	 */
	struct FPassthroughTrait : FBaseTrait, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FPassthroughTrait, FBaseTrait)

		using FSharedData = FAnimNextPassthroughSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Input;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		};

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
	};
}
