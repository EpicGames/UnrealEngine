// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/PassthroughBlendTrait.h"

#include "Animation/AnimTypes.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PassthroughBlendTrait)

namespace UE::UAF
{

AUTO_REGISTER_ANIM_TRAIT(FPassthroughBlendTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IUpdate) \
	GeneratorMacro(IEvaluate) \

// Trait implementation boilerplate
GENERATE_ANIM_TRAIT_IMPLEMENTATION(FPassthroughBlendTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR


void FPassthroughBlendTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

	ComputedAlphaValue = SharedData->GetAlpha(Binding);
	AlphaBoolBlend = SharedData->GetAlphaBoolBlend(Binding);
	AlphaScaleBiasClamp = SharedData->AlphaScaleBiasClamp;
}

void FPassthroughBlendTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	InstanceData->ComputedAlphaValue = ComputeAlphaValue(SharedData->GetAlphaInputType(Binding), SharedData, InstanceData, Binding, TraitState.GetDeltaTime());

	InstanceData->AlphaBoolBlend.Reinitialize();
	InstanceData->AlphaScaleBiasClamp.Reinitialize();
}

void FPassthroughBlendTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::PreUpdate(Context, Binding, TraitState);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	const float DeltaTime = TraitState.GetDeltaTime();
	InstanceData->DeltaTime = DeltaTime;

	InstanceData->ComputedAlphaValue = ComputeAlphaValue(SharedData->GetAlphaInputType(Binding), SharedData, InstanceData, Binding, DeltaTime);
}

// IEvaluate impl
void FPassthroughBlendTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	const EAnimAlphaInputType AlphaInputType = SharedData->GetAlphaInputType(Binding);
	if (AlphaInputType == EAnimAlphaInputType::Curve)
	{
		// Duplicate existing pose in the stack
		Context.AppendTask(FAnimNextDuplicateTopKeyframeTask::Make());

		// Let children perform their tasks on the duplicated pose
		IEvaluate::PostEvaluate(Context, Binding);

		// Add a BlendTwoKey with the original pose and the children modified copy, using alpha curve data (value will be calculated / clamped at the task)
		const float DeltaTime = InstanceData->DeltaTime;
		const FName AlphaCurveName = SharedData->GetAlphaCurveName(Binding);
		FInputScaleBiasClamp* InputScaleBiasClamp = &InstanceData->AlphaScaleBiasClamp;
		static constexpr int32 AlphaSourceInputKeyframeIndex = 0;
		Context.AppendTask(FAnimNextBlendTwoKeyframesTask::Make(AlphaCurveName, AlphaSourceInputKeyframeIndex, [DeltaTime, InputScaleBiasClamp](float Alpha) -> float
			{
				return InputScaleBiasClamp->ApplyTo(Alpha, DeltaTime);
			}));
	}
	else
	{
		const float BlendWeight = InstanceData->ComputedAlphaValue;
		if (FAnimWeight::IsFullWeight(BlendWeight))
		{
			IEvaluate::PostEvaluate(Context, Binding);	// full means children result without any modification
		}
		else if (FAnimWeight::IsRelevant(BlendWeight)) // if weight is relevant (> 0) we duplicate keyframe, call children to add the task (if any) and blend
		{
			// Duplicate existing pose in the stack
			Context.AppendTask(FAnimNextDuplicateTopKeyframeTask::Make());

			IEvaluate::PostEvaluate(Context, Binding);

			// Add a BlendTwoKey with the original pose and the children modified copy, using alpha value as float blend weight
			Context.AppendTask(FAnimNextBlendTwoKeyframesTask::Make(BlendWeight));
		}
		//else 
		//{
			// for BlendWeight == 0 we just need the input keyframe from children as is, so not calling base post evaluate 
		//}
	}
}

float FPassthroughBlendTrait::ComputeAlphaValue(const EAnimAlphaInputType AlphaInputType, const FPassthroughBlendTraitSharedData* SharedData, FInstanceData* InstanceData, const TTraitBinding<IUpdate>& Binding, float DeltaTime)
{
	// alpha handlers
	float CurrentAlpha = 0.f;

	switch (AlphaInputType)
	{
		case EAnimAlphaInputType::Float:
		{
			CurrentAlpha = SharedData->AlphaScaleBias.ApplyTo(InstanceData->AlphaScaleBiasClamp.ApplyTo(SharedData->GetAlpha(Binding), DeltaTime));
			break;
		}
		case EAnimAlphaInputType::Bool:
		{
			CurrentAlpha = InstanceData->AlphaBoolBlend.ApplyTo(SharedData->GetbAlphaBoolEnabled(Binding), DeltaTime);
			break;
		}
		case EAnimAlphaInputType::Curve:
		{
			CurrentAlpha = 1.0; // This will be calculated on the task when reading the curve, but we need both branches at full weight
			break;
		}
	};

	const float ClampedAlpha = FMath::Clamp(CurrentAlpha, 0.0f, 1.0f);
	return ClampedAlpha;
}

} // namespace UE::UAF
