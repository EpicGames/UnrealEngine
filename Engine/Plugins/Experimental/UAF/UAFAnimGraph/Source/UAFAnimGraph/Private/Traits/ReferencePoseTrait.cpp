// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/ReferencePoseTrait.h"

#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/PushReferenceKeyframe.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReferencePoseTrait)

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FReferencePoseTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FReferencePoseTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FReferencePoseTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		FAnimNextPushReferenceKeyframeTask Task;
		Task.bIsAdditive = SharedData->ReferencePoseType == EAnimNextReferencePoseType::AdditiveIdentity;

		Context.AppendTask(Task);
	}
}
