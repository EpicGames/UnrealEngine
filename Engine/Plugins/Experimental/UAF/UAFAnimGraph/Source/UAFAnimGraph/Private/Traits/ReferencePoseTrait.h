// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"

#include "ReferencePoseTrait.generated.h"

// TODO: Ideally, the reference pose we output should be a tag and the task that consumes the reference pose should
// be the one to determine whether it should be in local space or the additive identity (or something else).
// This way, we have a single option in the graph, and the system can figure out what to use removing the room the user mistakes.
// It will also be more efficient since we don't need to manipulate the reference pose until the point of use where we might
// be able to avoid the copy.

UENUM()
enum class EAnimNextReferencePoseType : int32
{
	MeshLocalSpace,
	AdditiveIdentity,
};

/** A trait that outputs a reference pose. */
USTRUCT(meta = (DisplayName = "Reference Pose", ShowTooltip=true))
struct FAnimNextReferencePoseTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** The type of the reference pose. */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	EAnimNextReferencePoseType ReferencePoseType = EAnimNextReferencePoseType::MeshLocalSpace;
};

namespace UE::UAF
{
	/**
	 * FReferencePoseTrait
	 * 
	 * A trait that outputs a reference pose.
	 */
	struct FReferencePoseTrait : FBaseTrait, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FReferencePoseTrait, FBaseTrait)

		using FSharedData = FAnimNextReferencePoseTraitSharedData;

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
	};
}
