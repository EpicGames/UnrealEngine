// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/SequencePlayer.h"

#include "Animation/AnimSequenceBase.h"
#include "AnimationRuntime.h"
#include "TraitCore/ExecutionContext.h"
#include "EvaluationVM/Tasks/PushAnimSequenceKeyframe.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FSequencePlayerTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IAttributeProvider) \
		GeneratorMacro(ITimeline) \
		GeneratorMacro(ITimelinePlayer) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IGarbageCollection) \
		GeneratorMacro(INotifySource) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FSequencePlayerTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	static bool IsSequenceLoopingHelper(const UAnimSequence* InAnimSequence, EAnimAssetLoopMode InLoopMode)
	{
		switch (InLoopMode)
		{
			case EAnimAssetLoopMode::Auto:
				return InAnimSequence ? InAnimSequence->bLoop : false;
			case EAnimAssetLoopMode::ForceLoop:
				return true;
			case EAnimAssetLoopMode::ForceNonLoop:
				return false;
			default:
				return false;
		}
	}

	void FSequencePlayerTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const bool bInterpolate = true;

		FAnimNextAnimSequenceKeyframeTask Task;
		Task.AnimSequence = InstanceData->AnimSequence;
		Task.SampleTime = InstanceData->InternalTimeAccumulator;
		Task.bInterpolate = bInterpolate;
		Task.DeltaTimeRecord = InstanceData->DeltaTimeRecord;
		Task.bLooping = IsSequenceLoopingHelper(InstanceData->AnimSequence, SharedData->GetLoopMode(Binding));
		Task.bExtractTrajectory = true;	/*Output.AnimInstanceProxy->ShouldExtractRootMotion()*/

		Context.AppendTask(Task);
	}

	FOnExtractRootMotionAttribute FSequencePlayerTrait::GetOnExtractRootMotionAttribute(FExecutionContext& Context, const TTraitBinding<IAttributeProvider>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->AnimSequence)
		{
			TTraitBinding<ITimeline> TimelineTrait;
			Binding.GetStackInterface(TimelineTrait);

			FTimelineState State;
			ensure(TimelineTrait.GetState(Context, State));

			const float PlayRate = State.GetPlayRate();
			const bool bIsLooping = State.IsLooping();

			auto ExtractRootMotionAttribute = [AnimSequence = InstanceData->AnimSequence, PlayRate, bIsLooping](float StartTime, float DeltaTime, bool bAllowLooping)
			{
				// We do not check for lifetimes, assume the sequence is alive during pose list execution.
				check(AnimSequence->IsValidLowLevel());

				// Make sure we don't extract outside our valid range
				StartTime = FMath::Clamp(StartTime, 0.0f, AnimSequence->GetPlayLength());

				// Account for play rate
				DeltaTime *= PlayRate;

				return AnimSequence->ExtractRootMotion(FAnimExtractContext(static_cast<double>(StartTime), true, FDeltaTimeRecord(DeltaTime), bAllowLooping && bIsLooping));
			};
			
			return FOnExtractRootMotionAttribute::CreateLambda(ExtractRootMotionAttribute);
		}

		return FOnExtractRootMotionAttribute();
	}

	void FSequencePlayerTrait::GetSyncMarkers(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineSyncMarkerArray& OutSyncMarkers) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (const UAnimSequence* AnimSeq = InstanceData->AnimSequence.Get())
		{
			OutSyncMarkers.Reserve(AnimSeq->AuthoredSyncMarkers.Num());

			for (const FAnimSyncMarker& SyncMarker : AnimSeq->AuthoredSyncMarkers)
			{
				OutSyncMarkers.Add(FTimelineSyncMarker(SyncMarker.MarkerName, SyncMarker.Time));
			}
		}
	}

	bool FSequencePlayerTrait::GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (const UAnimSequence* AnimSeq = InstanceData->AnimSequence.Get())
		{
			const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

			OutTimelineState = FTimelineState(
				InstanceData->InternalTimeAccumulator,
				AnimSeq->GetPlayLength(),
				SharedData->GetPlayRate(Binding) * AnimSeq->RateScale,
				IsSequenceLoopingHelper(AnimSeq, SharedData->GetLoopMode(Binding))).WithDebugName(AnimSeq->GetFName());
			return true;
		}

		return false;
	}

	bool FSequencePlayerTrait::GetDelta(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineDelta& OutTimelineDelta) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (const UAnimSequence* AnimSeq = InstanceData->AnimSequence.Get())
		{
			OutTimelineDelta = FTimelineDelta(
				InstanceData->DeltaTimeRecord.Delta,
				InstanceData->LastAdvanceType);
			return true;
		}

		return false;
	}

	void FSequencePlayerTrait::AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// We only advance if we have a valid anim sequence
		if (const UAnimSequence* AnimSeq = InstanceData->AnimSequence.Get())
		{
			TTraitBinding<ITimeline> TimelineTrait;
			Binding.GetStackInterface(TimelineTrait);

			FTimelineState State;
			ensure(TimelineTrait.GetState(Context, State));

			const float PlayRate = State.GetPlayRate();
			const bool bIsLooping = State.IsLooping();
			const float SequenceLength = State.GetDuration();

			const float PreviousTime = InstanceData->InternalTimeAccumulator;
			const float MoveDelta = DeltaTime * PlayRate;

			InstanceData->DeltaTimeRecord.Set(PreviousTime, MoveDelta);

			InstanceData->LastAdvanceType = FAnimationRuntime::AdvanceTime(bIsLooping, MoveDelta, InstanceData->InternalTimeAccumulator, SequenceLength);
		}
	}

	void FSequencePlayerTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		// Cache the anim sequence we'll play during construction, we don't allow it to change afterwards
		InstanceData->AnimSequence = SharedData->GetAnimSequence(Binding);

		float InternalTimeAccumulator = 0.0f;
		if (const UAnimSequence* AnimSeq = InstanceData->AnimSequence.Get())
		{
			const float StartPosition = SharedData->GetStartPosition(Binding);
			const float SequenceLength = AnimSeq->GetPlayLength();
			InternalTimeAccumulator = FMath::Clamp(StartPosition, 0.0f, SequenceLength);
		}

		InstanceData->InternalTimeAccumulator = InternalTimeAccumulator;
	}

	void FSequencePlayerTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		// We just advance the timeline
		TTraitBinding<ITimelinePlayer> TimelinePlayerTrait;
		Binding.GetStackInterface(TimelinePlayerTrait);

		TimelinePlayerTrait.AdvanceBy(Context, TraitState.GetDeltaTime(), true);
	}

	void FSequencePlayerTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		Collector.AddReferencedObject(InstanceData->AnimSequence);
	}

	void FSequencePlayerTrait::GetNotifies(FExecutionContext& Context, const TTraitBinding<INotifySource>& Binding, float StartPosition, float Duration, bool bLooping, TArray<FAnimNotifyEventReference>& OutNotifies) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (const UAnimSequence* AnimSeq = InstanceData->AnimSequence.Get())
		{
			FAnimTickRecord TickRecord;
			TickRecord.TimeAccumulator = &StartPosition;
			TickRecord.bLooping = bLooping;
			FAnimNotifyContext NotifyContext(TickRecord);
			AnimSeq->GetAnimNotifies(StartPosition, Duration, NotifyContext);
			OutNotifies = MoveTemp(NotifyContext.ActiveNotifies);
		}
	}
}
