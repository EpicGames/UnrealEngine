// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/ITimeline.h"
#include "TraitInterfaces/ITimelinePlayer.h"
#include "TraitInterfaces/IUpdate.h"
#include "Animation/AnimSequence.h"
#include "TraitInterfaces/INotifySource.h"
#include "Traits/SequencePlayerTraitData.h"

namespace UE::UAF
{
	/**
	 * FSequencePlayerTrait
	 * 
	 * A trait that can play an animation sequence.
	 */
	struct FSequencePlayerTrait : FBaseTrait, IEvaluate, IAttributeProvider, ITimeline, ITimelinePlayer, IUpdate, IGarbageCollection, INotifySource
	{
		DECLARE_ANIM_TRAIT(FSequencePlayerTrait, FBaseTrait)

		using FSharedData = FAnimNextSequencePlayerTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			// Cached value of the anim sequence we are playing
			TObjectPtr<const UAnimSequence> AnimSequence;

			/** Delta time range required for root motion extraction **/
			FDeltaTimeRecord DeltaTimeRecord;

			// Current time accumulator
			float InternalTimeAccumulator = 0.0f;

			// The last advance type when FAnimationRuntime::AdvanceTime was called
			ETypeAdvanceAnim LastAdvanceType = ETAA_Default;
		};

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IAttributeProvider impl
		virtual FOnExtractRootMotionAttribute GetOnExtractRootMotionAttribute(FExecutionContext& Context, const TTraitBinding<IAttributeProvider>& Binding) const override;

		// ITimeline impl
		virtual void GetSyncMarkers(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineSyncMarkerArray& OutSyncMarkers) const override;
		virtual bool GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const override;
		virtual bool GetDelta(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineDelta& OutTimelineDelta) const override;

		// ITimelinePlayer impl
		virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const override;

		// IUpdate impl
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

		// INotifySource impl
		virtual void GetNotifies(FExecutionContext& Context, const TTraitBinding<INotifySource>& Binding, float StartPosition, float Duration, bool bLooping, TArray<FAnimNotifyEventReference>& OutNotifies) const override;
	};
}
