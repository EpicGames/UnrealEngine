// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/SubGraphHost.h"

#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FSubGraphHostTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IDiscreteBlend) \
		GeneratorMacro(IGarbageCollection) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FSubGraphHostTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FSubGraphHostTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		ReferencePoseChildPtr = Context.AllocateNodeInstance(Binding, SharedData->ReferencePoseChild);
	}

	uint32 FSubGraphHostTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->SubGraphSlots.Num();
	}

	void FSubGraphHostTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (const FSubGraphSlot& SubGraphEntry : InstanceData->SubGraphSlots)
		{
			if (SubGraphEntry.State == ESlotState::ActiveWithReferencePose)
			{
				Children.Add(InstanceData->ReferencePoseChildPtr);
			}
			else
			{
				// Even if the slot is inactive, we queue an empty handle
				Children.Add(SubGraphEntry.GraphInstance.IsValid() ? SubGraphEntry.GraphInstance->GetGraphRootPtr() : FTraitPtr());
			}
		}
	}

	void FSubGraphHostTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const bool bHasActiveSubGraph = InstanceData->CurrentlyActiveSubGraphIndex != INDEX_NONE;

		TObjectPtr<const UUAFAnimGraph> CurrentActiveAnimationGraph;
		FName CurrentActiveEntryPoint = NAME_None;
		if (bHasActiveSubGraph)
		{
			FSubGraphSlot& SubGraphSlot = InstanceData->SubGraphSlots[InstanceData->CurrentlyActiveSubGraphIndex];
			CurrentActiveAnimationGraph = SubGraphSlot.AnimationGraph;
			CurrentActiveEntryPoint = SubGraphSlot.EntryPoint;

			SubGraphSlot.bWasRelevant = true;
		}

		const TObjectPtr<const UUAFAnimGraph> DesiredAnimationGraph = SharedData->GetAnimationGraph(Binding);
		const FName EntryPoint = SharedData->GetEntryPoint(Binding);

		// Check for re-entrancy and early-out if we are linking back to the current instance or one of its parents
		const FAnimNextGraphInstance* OwnerGraphInstance = &Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
		while (OwnerGraphInstance != nullptr)
		{
			if (OwnerGraphInstance->UsesAnimationGraph(DesiredAnimationGraph) && OwnerGraphInstance->UsesEntryPoint(EntryPoint))
			{
				return;
			}

			OwnerGraphInstance = OwnerGraphInstance->GetParentGraphInstance();
		}

		if (!bHasActiveSubGraph || CurrentActiveAnimationGraph != DesiredAnimationGraph || CurrentActiveEntryPoint != EntryPoint)
		{
			// Find an empty slot we can use
			int32 FreeSlotIndex = INDEX_NONE;

			const int32 NumSubGraphSlots = InstanceData->SubGraphSlots.Num();
			for (int32 SlotIndex = 0; SlotIndex < NumSubGraphSlots; ++SlotIndex)
			{
				if (InstanceData->SubGraphSlots[SlotIndex].State == ESlotState::Inactive)
				{
					// This slot is inactive, we can re-use it
					FreeSlotIndex = SlotIndex;
					break;
				}
			}

			if (FreeSlotIndex == INDEX_NONE)
			{
				// All slots are in use, add a new one
				FreeSlotIndex = InstanceData->SubGraphSlots.AddDefaulted();
			}

			FSubGraphSlot& SubGraphSlot = InstanceData->SubGraphSlots[FreeSlotIndex];
			SubGraphSlot.AnimationGraph = DesiredAnimationGraph;
			SubGraphSlot.State = DesiredAnimationGraph ? ESlotState::ActiveWithGraph : ESlotState::ActiveWithReferencePose;
			SubGraphSlot.EntryPoint = EntryPoint;

			const int32 OldChildIndex = InstanceData->CurrentlyActiveSubGraphIndex;
			const int32 NewChildIndex = FreeSlotIndex;

			InstanceData->CurrentlyActiveSubGraphIndex = FreeSlotIndex;

			TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
			Binding.GetStackInterface(DiscreteBlendTrait);

			DiscreteBlendTrait.OnBlendTransition(Context, OldChildIndex, NewChildIndex);
		}
	}

	void FSubGraphHostTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		const int32 NumSubGraphs = InstanceData->SubGraphSlots.Num();
		if (NumSubGraphs == 0)
		{
			return;
		}

		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		for (int32 SubGraphIndex = 0; SubGraphIndex < NumSubGraphs; ++SubGraphIndex)
		{
			const FSubGraphSlot& SubGraphSlot = InstanceData->SubGraphSlots[SubGraphIndex];
			if(SubGraphSlot.AnimationGraph == nullptr || !SubGraphSlot.GraphInstance.IsValid())
			{
				continue;
			}

			const float BlendWeight = DiscreteBlendTrait.GetBlendWeight(Context, SubGraphIndex);
			const bool bGraphHasNeverUpdated = SubGraphSlot.GraphInstance.IsValid() && !SubGraphSlot.GraphInstance->HasUpdated();

			FTraitUpdateState SubGraphTraitState = TraitState
				.WithWeight(BlendWeight)
				.AsBlendingOut(SubGraphIndex != InstanceData->CurrentlyActiveSubGraphIndex)
				.AsNewlyRelevant(!SubGraphSlot.bWasRelevant || bGraphHasNeverUpdated);

			if (SubGraphSlot.GraphInstance.IsValid())
			{
				SubGraphSlot.GraphInstance->MarkAsUpdated();
			}

			TraversalQueue.Push(SubGraphSlot.GraphInstance->GetGraphRootPtr(), SubGraphTraitState);
		}
	}

	float FSubGraphHostTrait::GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (ChildIndex == InstanceData->CurrentlyActiveSubGraphIndex)
		{
			return 1.0f;	// Active child has full weight
		}
		else if (InstanceData->SubGraphSlots.IsValidIndex(ChildIndex))
		{
			return 0.0f;	// Other children have no weight
		}
		else
		{
			// Invalid child index
			return -1.0f;
		}
	}

	int32 FSubGraphHostTrait::GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		return InstanceData->CurrentlyActiveSubGraphIndex;
	}

	void FSubGraphHostTrait::OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const
	{
		TTraitBinding<IDiscreteBlend> DiscreteBlendTrait;
		Binding.GetStackInterface(DiscreteBlendTrait);

		// We initiate immediately when we transition
		DiscreteBlendTrait.OnBlendInitiated(Context, NewChildIndex);

		// We terminate immediately when we transition
		DiscreteBlendTrait.OnBlendTerminated(Context, OldChildIndex);
	}

	void FSubGraphHostTrait::OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->SubGraphSlots.IsValidIndex(ChildIndex))
		{
			// Allocate our new sub-graph instance
			FSubGraphSlot& SubGraphEntry = InstanceData->SubGraphSlots[ChildIndex];

			if (SubGraphEntry.State == ESlotState::ActiveWithGraph)
			{
				FAnimNextGraphInstance& Owner = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
				SubGraphEntry.GraphInstance = SubGraphEntry.AnimationGraph->AllocateInstance(
					{
						.SystemReference = Owner.GetModuleInstance()->GetReference(),
						.ParentGraphInstance = Owner.AsShared(),
						.EntryPoint = SubGraphEntry.EntryPoint
					});
			}
		}
	}

	void FSubGraphHostTrait::OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->SubGraphSlots.IsValidIndex(ChildIndex))
		{
			// Deallocate our sub-graph instance
			FSubGraphSlot& SubGraphEntry = InstanceData->SubGraphSlots[ChildIndex];

			if (SubGraphEntry.State == ESlotState::ActiveWithGraph)
			{
				SubGraphEntry.GraphInstance.Reset();
			}

			SubGraphEntry.State = ESlotState::Inactive;
			SubGraphEntry.bWasRelevant = false;
		}
	}

	void FSubGraphHostTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		for (FSubGraphSlot& SubGraphEntry : InstanceData->SubGraphSlots)
		{
			Collector.AddReferencedObject(SubGraphEntry.AnimationGraph);
			if(FAnimNextGraphInstance* ImplPtr = SubGraphEntry.GraphInstance.Get())
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), ImplPtr);
			}
		}
	}
}
