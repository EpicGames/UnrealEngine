// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Traits/SubGraphHostTraitData.h"

namespace UE::UAF
{
	/**
	 * FSubGraphHostTrait
	 * 
	 * A trait that hosts and manages a sub-graph instance.
	 */
	struct FSubGraphHostTrait : FBaseTrait, IUpdate, IUpdateTraversal, IHierarchy, IDiscreteBlend, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FSubGraphHostTrait, FBaseTrait)

		enum class ESlotState : uint8
		{
			ActiveWithGraph,
			ActiveWithReferencePose,
			Inactive,
		};

		struct FSubGraphSlot
		{
			// The animation graph to use
			TObjectPtr<const UUAFAnimGraph> AnimationGraph;

			// The graph instance
			TSharedPtr<FAnimNextGraphInstance> GraphInstance;

			// The entry point to use
			FName EntryPoint;

			// The current slot state
			ESlotState State = ESlotState::Inactive;

			// Whether or not this slot state was previously relevant
			bool bWasRelevant = false;
		};

		using FSharedData = FAnimNextSubGraphHostTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);

			// List of sub-graph slots
			TArray<FSubGraphSlot> SubGraphSlots;

			// The index of the currently active sub-graph slot
			// All other sub-graphs are blending out
			int32 CurrentlyActiveSubGraphIndex = INDEX_NONE;

			// Our child node pointer. This child is shared between all slots that have no graph provided.
			FTraitPtr ReferencePoseChildPtr;
		};

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

		// IDiscreteBlend impl
		virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual int32 GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const override;
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;
		virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;

		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	};
}
