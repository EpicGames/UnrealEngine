// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/SynchronizeUsingGroups.h"

#include "InstanceTaskContext.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Graph/SyncGroup_GraphInstanceComponent.h"
#include "GraphInterfaces/AnimNextNativeDataInterface_SynchronizeUsingGroups.h"
#include "UAFLogging.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FSynchronizeUsingGroupsTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IGraphFactory) \
		GeneratorMacro(IGroupSynchronization) \
		GeneratorMacro(ITimelinePlayer) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FSynchronizeUsingGroupsTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FSynchronizeUsingGroupsTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		SyncGroupComponent = &Context.GetOrAddComponent<FSyncGroupGraphInstanceComponent>();

		// If our timeline can't advance, we can't synchronize; assume we have no timeline
		// We only check the base trait since it is the one that would own the timeline
		FTraitBinding BaseTrait;
		ensure(Binding.GetStackBaseTrait(BaseTrait));
		bHasTimelinePlayer = BaseTrait.HasInterface<ITimelinePlayer>();
	}

	void FSynchronizeUsingGroupsTrait::FInstanceData::Destruct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		if (!UniqueGroupName.IsNone())
		{
			SyncGroupComponent->ReleaseUniqueGroupName(UniqueGroupName);
		}
	}

	void FSynchronizeUsingGroupsTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		InstanceData->bHasReachedFullWeight = false;
	}

	void FSynchronizeUsingGroupsTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (!InstanceData->bHasTimelinePlayer)
		{
			// If we don't have a timeline player beneath us on the trait stack, there is nothing to synchronize
			UAF_TRAIT_LOG(Warning, TEXT("FSynchronizeUsingGroupsTrait::PreUpdate: Could not synchronize - No valid timeline player beneath us on the trait stack."));
			IUpdate::PreUpdate(Context, Binding, TraitState);
			return;
		}

		TTraitBinding<IGroupSynchronization> GroupSyncTrait;
		Binding.GetStackInterface(GroupSyncTrait);

		const FSyncGroupParameters GroupParameters = GroupSyncTrait.GetGroupParameters(Context);
		const bool bHasGroupName = !GroupParameters.GroupName.IsNone();

		// If we have a group name, we are active
		// Freeze the timeline, our sync group will control it
		InstanceData->bFreezeTimeline = bHasGroupName;
		InstanceData->bHasReachedFullWeight |= FAnimWeight::IsFullWeight(TraitState.GetTotalWeight());

		// Forward the PreUpdate call, if the timeline attempts to update, we'll do nothing if we are frozen
		IUpdate::PreUpdate(Context, Binding, TraitState);

		if (!bHasGroupName)
		{
			// If no group name is specified, this trait is inactive
			return;
		}

		// Append this trait to our group, we'll need to synchronize it
		InstanceData->SyncGroupComponent->RegisterWithGroup(GroupParameters, Binding.GetTraitPtr(), TraitState);
	}

	FSyncGroupParameters FSynchronizeUsingGroupsTrait::GetGroupParameters(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		FName GroupName;
		EAnimGroupSynchronizationRole GroupRole = SharedData->GetGroupRole(Binding);
		EAnimGroupSynchronizationMode SyncMode = SharedData->GetSyncMode(Binding);

		switch (SyncMode)
		{
		case EAnimGroupSynchronizationMode::NoSynchronization:
			GroupName = NAME_None;
			break;
		case EAnimGroupSynchronizationMode::SynchronizeUsingGroupName:
			GroupName = SharedData->GetGroupName(Binding);
			break;
		case EAnimGroupSynchronizationMode::SynchronizeUsingUniqueGroupName:
			if (InstanceData->UniqueGroupName.IsNone())
			{
				InstanceData->UniqueGroupName = InstanceData->SyncGroupComponent->CreateUniqueGroupName();
			}

			GroupName = InstanceData->UniqueGroupName;
			break;
		}

		// When the role is set to TransitionLeader or TransitionFollower, we will not be part of a sync group until we are full weight
		// meaning we ignore the sync group while blending in, but not while blending out
		if (GroupRole == EAnimGroupSynchronizationRole::TransitionLeader || GroupRole == EAnimGroupSynchronizationRole::TransitionFollower)
		{
			if (InstanceData->bHasReachedFullWeight)
			{
				GroupName = NAME_None;
			}
		}

		FSyncGroupParameters Parameters;
		Parameters.GroupName = GroupName;
		Parameters.GroupRole = GroupRole;
		Parameters.SyncMode = SyncMode;
		Parameters.bMatchSyncPoint = SharedData->GetbMatchSyncPoint(Binding);

		return Parameters;
	}

	void FSynchronizeUsingGroupsTrait::AdvanceBy(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (!InstanceData->bHasTimelinePlayer)
		{
			return;	//  No timeline player beneath us on the trait stack, nothing to synchronize
		}

		// When the group advances the timeline, we thaw it to advance
		InstanceData->bFreezeTimeline = false;

		TTraitBinding<ITimelinePlayer> TimelinePlayerTrait;
		Binding.GetStackInterface(TimelinePlayerTrait);

		TimelinePlayerTrait.AdvanceBy(Context, DeltaTime, bDispatchEvents);

		InstanceData->bFreezeTimeline = true;
	}

	void FSynchronizeUsingGroupsTrait::AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->bFreezeTimeline)
		{
			return;	// If the timeline is frozen, we don't advance
		}

		ITimelinePlayer::AdvanceBy(Context, Binding, DeltaTime, bDispatchEvents);
	}

	void FSynchronizeUsingGroupsTrait::GetFactoryParams(FExecutionContext& Context, const TTraitBinding<IGraphFactory>& Binding, FGraphAssetHandleConstView InAssetData, FAnimNextFactoryParams& InOutParams) const
	{
		IGraphFactory::GetFactoryParams(Context, Binding, InAssetData, InOutParams);

		TTraitBinding<IGroupSynchronization> GroupSyncTrait;
		Binding.GetStackInterface(GroupSyncTrait);

		const FSyncGroupParameters GroupParameters = GroupSyncTrait.GetGroupParameters(Context);

		// If synchronization is enabled, request & populate the parameters
		if (GroupParameters.SyncMode != EAnimGroupSynchronizationMode::NoSynchronization)
		{
			FSynchronizeUsingGroupsData SynchronizeUsingGroups;
			SynchronizeUsingGroups.GroupName = GroupParameters.GroupName;
			SynchronizeUsingGroups.GroupRole = GroupParameters.GroupRole;
			SynchronizeUsingGroups.SyncMode = GroupParameters.SyncMode;
			SynchronizeUsingGroups.bMatchSyncPoint = GroupParameters.bMatchSyncPoint;
			InOutParams.AddTrait(ETraitVariableMapping::All, 0, SynchronizeUsingGroups);
		}
	}
}
