// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IGroupSynchronization.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/ITimelinePlayer.h"
#include "Traits/SynchronizeUsingGroupsTraitData.h"

struct FSyncGroupGraphInstanceComponent;

namespace UE::UAF
{
	/**
	 * FSynchronizeUsingGroupsTrait
	 * 
	 * A trait that synchronizes animation sequence playback using named groups.
	 */
	struct FSynchronizeUsingGroupsTrait : FAdditiveTrait, IUpdate, IGroupSynchronization, ITimelinePlayer, IGraphFactory
	{
		DECLARE_ANIM_TRAIT(FSynchronizeUsingGroupsTrait, FAdditiveTrait)

		using FSharedData = FAnimNextSynchronizeUsingGroupsTraitSharedData;

		struct FInstanceData : FTraitInstanceData
		{
			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
			void Destruct(const FExecutionContext& Context, const FTraitBinding& Binding);

			// Cached pointer to our sync group component
			FSyncGroupGraphInstanceComponent* SyncGroupComponent = nullptr;

			// If our sync mode requires a unique group name, we'll cache it and re-use it as it is unique to our instance
			FName UniqueGroupName;

			bool bFreezeTimeline = false;
			bool bHasReachedFullWeight = false;
			bool bHasTimelinePlayer = false;
		};

#if WITH_EDITOR
		// A trait stack has a single timeline, we can't support multiple instances
		virtual bool MultipleInstanceSupport() const override { return false; }
#endif

		// IUpdate impl
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IGroupSynchronization impl
		virtual FSyncGroupParameters GetGroupParameters(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding) const override;
		virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding, float DeltaTime, bool bDispatchEvents) const override;

		// ITimelinePlayer impl
		virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const override;

		// IGraphFactory impl
		virtual void GetFactoryParams(FExecutionContext& Context, const TTraitBinding<IGraphFactory>& Binding, FGraphAssetHandleConstView AssetHandle, FAnimNextFactoryParams& InOutParams) const override;
	};
}
