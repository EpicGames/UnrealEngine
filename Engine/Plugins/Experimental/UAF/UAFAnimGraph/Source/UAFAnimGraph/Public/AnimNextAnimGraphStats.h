// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "AnimNextStats.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Allocate Graph Instance"), STAT_AnimNext_Graph_AllocateInstance, STATGROUP_AnimNext, UAFANIMGRAPH_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Graph RigVM"), STAT_AnimNext_Graph_RigVM, STATGROUP_AnimNext, UAFANIMGRAPH_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Execute Evaluation Program"), STAT_AnimNext_EvaluationProgram_Execute, STATGROUP_AnimNext, UAFANIMGRAPH_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Call Function"), STAT_AnimNext_CallFunction, STATGROUP_AnimNext, UAFANIMGRAPH_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Keyframe CopyFrom"), STAT_AnimNext_Keyframe_CopyFrom, STATGROUP_AnimNext, UAFANIMGRAPH_API);
