// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UAF/UAFAssetData.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Factory/AnimAssetParams.h"
#include "Graph/AnimNextAnimationGraph.h"

#include "UAFAnimGraphAssetData.generated.h"


USTRUCT(DisplayName="Graph Asset")
struct FUAFGraphFactoryAsset_Graph : public FUAFGraphFactoryAsset
{
	GENERATED_BODY()

	FUAFGraphFactoryAsset_Graph() = default; 
	FUAFGraphFactoryAsset_Graph(const UUAFAnimGraph* Graph) : AnimationGraph(Graph) {}

	virtual bool Validate() const override { return AnimationGraph != nullptr; }

	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const override { OutReferencedObjects.Add(AnimationGraph); }
	
	UPROPERTY(EditAnywhere, Category = AssetData)
	TObjectPtr<const UUAFAnimGraph> AnimationGraph;
};

UENUM()
enum class EAnimationStartTimeType : uint8
{
	TimeInSeconds,
	TimePercent
};

USTRUCT(DisplayName="Animation Asset")
struct FUAFGraphFactoryAsset_Animation : public FUAFGraphFactoryAsset
{
	GENERATED_BODY()
	
	FUAFGraphFactoryAsset_Animation(const UAnimSequence* AnimSequence) { AnimationSequence = AnimSequence; }
	FUAFGraphFactoryAsset_Animation() = default;

	virtual bool Validate() const override { return AnimationSequence != nullptr; }

	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const override { OutReferencedObjects.Add(AnimationSequence); }

	float CalculateStartTime() const
	{
		float StartTime = 0.0f;
		float Duration = AnimationSequence != nullptr ? AnimationSequence->GetPlayLength() : 0.0f;
		bool IsLooping = LoopMode == UE::UAF::EAnimAssetLoopMode::ForceLoop || (LoopMode == UE::UAF::EAnimAssetLoopMode::Auto && AnimationSequence != nullptr && AnimationSequence->bLoop);
		switch (StartTimeType)
		{
		case EAnimationStartTimeType::TimeInSeconds:
			StartTime = StartTimeSeconds;
			break;

		case EAnimationStartTimeType::TimePercent:
			StartTime = Duration > UE_SMALL_NUMBER ? StartTimePercent * Duration * 0.01f : 0.0f;
			break;

		default:
			checkNoEntry();
		}

		// Only clamp to max length if we are not looping
		if (IsLooping)
		{
			return FMath::Max(StartTime, 0.0f);
		}
		else
		{
			return FMath::Clamp(StartTime, 0.0f, Duration);
		}
	}
	
	UPROPERTY(EditAnywhere, Category = AssetData)
	TObjectPtr<const UAnimSequence> AnimationSequence;

	UPROPERTY(EditAnywhere, Category ="Playback Options")
	float PlayRate = 1.0f;

	UPROPERTY(EditAnywhere, Category ="Playback Options")
	UE::UAF::EAnimAssetLoopMode LoopMode = UE::UAF::EAnimAssetLoopMode::Auto;

	UPROPERTY(EditAnywhere, Category ="Playback Options")
	EAnimationStartTimeType StartTimeType = EAnimationStartTimeType::TimeInSeconds;
	
	UPROPERTY(EditAnywhere, Category ="Playback Options", meta=(EditConditionHides, EditCondition="StartTimeType==EAnimationStartTimeType::TimeInSeconds", Units="s", ClampMin = "0.0", UIMin = "0.0"))
	float StartTimeSeconds = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Playback Options", meta=(EditConditionHides, EditCondition="StartTimeType==EAnimationStartTimeType::TimePercent", Units="Percent", ClampMin = "0.0", ClampMax = "100.0", UIMin = "0.0", UIMax = "100.0"))
	float StartTimePercent = 0.0f;
};

USTRUCT(DisplayName="Blend Space Asset")
struct FUAFGraphFactoryAsset_BlendSpace : public FUAFGraphFactoryAsset
{
	GENERATED_BODY()
	
	FUAFGraphFactoryAsset_BlendSpace(const UBlendSpace* BlendSpace) : BlendSpace(BlendSpace) {}
	FUAFGraphFactoryAsset_BlendSpace() = default;

	virtual bool Validate() const override { return BlendSpace != nullptr; }

	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const override { OutReferencedObjects.Add(BlendSpace); }
	
	UPROPERTY(EditAnywhere, Category = AssetData)
	TObjectPtr<const UBlendSpace> BlendSpace;

	UPROPERTY(EditAnywhere, Category = "Blend Space Options")
	float XAxisSamplePoint = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Blend Space Options")
	float YAxisSamplePoint = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Blend Space Options")
	float PlayRate = 1.0f;
};
