// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	// Flags to control what to compute when evaluating an animation node
	enum class EEvaluationFlags
	{
		// Empty flags
		None = 0x00,

		// Evaluates the trajectory of the root bone
		Trajectory = 0x01,

		// Evaluates the animation pose of all bones
		Bones = 0x02,

		// Evaluates the animation curves
		Curves = 0x04,

		// Evaluates the animation attributes
		Attributes = 0x08,

		// Evaluates everything
		All = 0xff,
	};

	ENUM_CLASS_FLAGS(EEvaluationFlags)
}
