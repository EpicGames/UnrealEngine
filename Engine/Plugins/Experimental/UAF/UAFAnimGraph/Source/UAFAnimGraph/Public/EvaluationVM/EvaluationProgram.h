// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include <type_traits>

#define UE_API UAFANIMGRAPH_API


struct FSerializableEvaluationProgram;

namespace UE::UAF
{
	struct FEvaluationVM;

	/*
	 * Evaluation Program
	 *
	 * This struct holds a sequence of evaluation tasks that form a program within our
	 * evaluation virtual machine framework. Programs are immutable once written.
	 *
	 * @see FAnimNextEvaluationTask
	 */
	struct FEvaluationProgram
	{
		FEvaluationProgram() = default;

		// Allow moving
		FEvaluationProgram(FEvaluationProgram&&) = default;
		FEvaluationProgram& operator=(FEvaluationProgram&&) = default;

		// Returns whether or not this program is empty
		UE_API bool IsEmpty() const;

		// Appends a new task into the program, tasks mutate state in the order they have been appended in
		// This means that child nodes need to evaluate first, tasks will usually be appended in IEvaluate::PostEvaluate
		// Tasks are moved into their final memory location, caller can allocate the task anywhere, it is no longer needed after this operation
		template<class TaskType>
		void AppendTask(TaskType&& Task);

		// Same as above, but uses TaskType copy constructor to move the task into the program.
		template<class TaskType>
		void AppendTaskPtr(const TSharedPtr<TaskType>& TaskPtr);

		// Executes the current program on the provided virtual machine
		UE_API void Execute(FEvaluationVM& VM) const;

		// Returns the program as a string suitable for debug purposes
		UE_API FString ToString() const;

		friend struct ::FSerializableEvaluationProgram;
	private:
		// Disallow copy
		FEvaluationProgram(const FEvaluationProgram&) = delete;
		FEvaluationProgram& operator=(const FEvaluationProgram&) = delete;

		// List of tasks
		TArray<TSharedPtr<FAnimNextEvaluationTask>> Tasks;
	};

	//////////////////////////////////////////////////////////////////////////
	// Implementation of inline functions

	template<class TaskType>
	inline void FEvaluationProgram::AppendTask(TaskType&& Task)
	{
		static_assert(std::is_base_of<FAnimNextEvaluationTask, TaskType>::value, "Task type must derive from FAnimNextEvaluationTask");

		// TODO: Use a fancy allocator to ensure all tasks are contiguous and tightly packed
		TSharedPtr<FAnimNextEvaluationTask> TaskCopy = MakeShared<TaskType>(MoveTemp(Task));
		Tasks.Add(MoveTemp(TaskCopy));
	}

	template<class TaskType>
	inline void FEvaluationProgram::AppendTaskPtr(const TSharedPtr<TaskType>& TaskPtr)
	{
		static_assert(std::is_base_of<FAnimNextEvaluationTask, TaskType>::value, "Task type must derive from FAnimNextEvaluationTask");

		Tasks.Add(TaskPtr);
	}
}

#undef UE_API
