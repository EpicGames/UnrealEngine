// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EvaluationTask.generated.h"

namespace UE::UAF
{
	struct FEvaluationVM;
}

// Implements various implementation details for an animation evaluation task
#define DECLARE_ANIM_EVALUATION_TASK(TaskName) \
	virtual const UScriptStruct* GetStruct() const override { return TaskName::StaticStruct(); }

/*
 * Evaluation Task
 *
 * This is the base class for our evaluation program tasks. They represent macro
 * instructions within our evaluation virtual machine system. They operate on the
 * VM internal state to generate inputs and produce outputs.
 *
 * @see FEvaluationProgram, FEvaluationVM
 */
USTRUCT()
struct FAnimNextEvaluationTask
{
	GENERATED_BODY()

	virtual ~FAnimNextEvaluationTask() {}

	// Returns the derived type for the task
	virtual const UScriptStruct* GetStruct() const { return FAnimNextEvaluationTask::StaticStruct(); }

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const { LowLevelFatalError(TEXT("Pure virtual not implemented (FAnimNextEvaluationTask::Execute)")); }
};
