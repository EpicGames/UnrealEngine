// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#define UE_API UAFANIMGRAPH_API

class UAbstractSkeletonSetBinding;
class USkeleton;

namespace UE::UAF
{
	/*
	 * Evaluation Task Context
	 *
	 * Encapsulates the state that controls what values tasks should produce.
	 */
	class FEvaluationTaskContext final
	{
	public:
		UE_API FEvaluationTaskContext();
		UE_API FEvaluationTaskContext(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName);

		// Returns whether or not this context is valid
		[[nodiscard]] UE_API bool IsValid() const;

		// Returns the abstract skeleton set binding asset we are evaluating with (or nullptr if we aren't valid)
		[[nodiscard]] UE_API const UAbstractSkeletonSetBinding* GetSetBinding() const;

		// Returns the concrete skeleton we are evaluating with (or nullptr if we aren't valid)
		[[nodiscard]] UE_API const USkeleton* GetSkeleton() const;

		// Returns the skeletal mesh we are evaluating with, may be nullptr)
		[[nodiscard]] UE_API const USkeletalMesh* GetSkeletalMesh() const;

		// Returns the named set we are currently evaluating with (or nullptr if we aren't valid)
		[[nodiscard]] UE_API const FAttributeNamedSetPtr& GetNamedSet() const;

		// Sets the named set
		// Returns true if we succeeded and the set exists in the current binding, false otherwise
		[[nodiscard]] UE_API bool SetNamedSet(FName SetName);

		// Returns the default attribute values for the current named set (or nullptr if we aren't valid)
		[[nodiscard]] UE_API const FValueBundlePtr& GetDefaultAttributeValues() const;

	private:
		// The set binding we are evaluating with
		const UAbstractSkeletonSetBinding* SetBinding = nullptr;

		// The target skeletal mesh with are evaluating with, may be nullptr
		const USkeletalMesh* SkeletalMesh = nullptr;
		
		// The target skeleton we are evaluating with
		const USkeleton* Skeleton = nullptr;

		// Binding data owned by the set binding
		FAttributeBindingDataPtr BindingData;

		// Named set instance to produce an output for
		FAttributeNamedSetPtr NamedSet;

		// Default values as specified by the current set binding and named set
		FValueBundlePtr DefaultAttributeValues;
	};
}

#undef UE_API
