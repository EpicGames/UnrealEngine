// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ConstExprUID.h"
#include "Animation/AnimCurveFilter.h"
#include "EvaluationVM/EvaluationFlags.h"
#include "EvaluationVM/EvaluationTaskContext.h"
#include "EvaluationVM/KeyframeState.h"
#include "Hash/Fnv.h"
#include "ReferencePose.h"
#include "Misc/Crc.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"

#define UE_API UAFANIMGRAPH_API

// If you wish to use a custom type with a stack in the FEvaluationVM, you'll have to enable
// its usage through this macro to ensure that UE::UAF::GetTypeID() is declared.
// Use this macro inside the UE::UAF namespace
#define ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE(Type) \
	template<> \
	constexpr uint32 GetTypeID<Type>() \
	{ \
		return UE::HashStringFNV1a32(#Type); \
	}

namespace UE::UAF
{
	// Helper function that returns a UID for the specified type
	// @see ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE
	template<typename Type>
	constexpr uint32 GetTypeID()
	{
		static_assert(sizeof(Type) != sizeof(Type),
			"Not implemented! See ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE for details and register your type");
		return 0;
	}

	// Helper function that destructs a pointer of the provided type
	template<typename Type>
	inline void DestructType(void* Ptr)
	{
		reinterpret_cast<Type*>(Ptr)->~Type();
	}

	template<typename Type>
	inline TFunction<void(void*)> GetTypeDestructor()
	{
		if constexpr (std::is_trivially_destructible_v<Type>)
		{
			return TFunction<void(void*)>();
		}
		else
		{
			return TFunction<void(void*)>(&DestructType<Type>);
		}
	}

	/*
	 * Evaluation VM Stack Name
	 * 
	 * A small struct to wrap an FName and cache its type hash.
	 */
	struct FEvaluationVMStackName final
	{
		// Implicitly coerce or construct a stack name from an FName
		FEvaluationVMStackName(const FName& InName)
			: Name(InName)
			, NameHash(GetTypeHash(InName))
		{}

		FName Name;
		uint32 NameHash = 0;
	};

	/*
	 * Evaluation VM Stack Entry
	 * 
	 * A small header for each stack entry.
	 */
	struct alignas(16) FEvaluationVMStackEntry final
	{
		union
		{
			// Pointer to the previous entry beneath us on the stack
			FEvaluationVMStackEntry* Prev = nullptr;

			// Dummy padding to ensure this structure remains 16 bytes even on 32-bit architectures
			uint64 DummyPadding;
		};

		// Allocated size of this entry, includes sizeof(FEvaluationVMStackEntry)
		uint32 Size = 0;

		// Metadata flags
		uint32 Flags = 0;

		// Entry's actual data follows in memory, aligned to 16 bytes

		void* GetValuePtr() { return reinterpret_cast<uint8*>(this) + sizeof(FEvaluationVMStackEntry); }
		const void* GetValuePtr() const { return reinterpret_cast<const uint8*>(this) + sizeof(FEvaluationVMStackEntry); }
	};

	static_assert(sizeof(FEvaluationVMStackEntry) == 16, "Expected 16 bytes to ensure proper alignment");

	/*
	 * Evaluation VM Stack
	 * 
	 * Represents a named and typed VM stack.
	 */
	struct FEvaluationVMStack final
	{
		// Name of the stack
		FName Name;

		// Type ID used for runtime checks to make sure we always read/write using the same type
		// @see ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE
		uint32 TypeID = 0;

		// Pointer to the entry on top of the stack or nullptr if the stack is empty
		FEvaluationVMStackEntry* Top = nullptr;

		// For non-trivially destructible types, the destructor to call
		// Only used if the stack is not empty when it is destroyed
		TFunction<void(void*)> TypeDestructor;

		FEvaluationVMStack() = default;

		// Disable copy but enable move
		FEvaluationVMStack(const FEvaluationVMStack&) = delete;
		FEvaluationVMStack(FEvaluationVMStack&&) = default;
		FEvaluationVMStack& operator=(const FEvaluationVMStack&) = delete;
		FEvaluationVMStack& operator=(FEvaluationVMStack&&) = default;

		// The destructor will free remaining entries
		UE_API ~FEvaluationVMStack();
	};

	/*
	 * Evaluation VM
	 *
	 * This struct holds the internal state when we execute and FEvaluationProgram.
	 * Our virtual machine is stack based, values are generally pushed and popped.
	 * 
	 * @see FEvaluationProgram
	 */
	struct FEvaluationVM
	{
		UE_API FEvaluationVM(EEvaluationFlags InEvaluationFlags, const FReferencePose& InReferencePose, int32 InCurrentLOD);
		UE_API FEvaluationVM(EEvaluationFlags EvaluationFlags, TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName, int32 CurrentLOD);

		// REMOVE ME: Exclusively for testing that before/after yields the same results
		UE_API FEvaluationVM(EEvaluationFlags InEvaluationFlags, const FReferencePose& IReferencePose, const UAbstractSkeletonSetBinding* SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName, int32 InCurrentLOD);

		// Allow move
		FEvaluationVM(FEvaluationVM&&) = default;
		FEvaluationVM& operator=(FEvaluationVM&&) = default;

		// Returns true if we are fully initialized
		[[nodiscard]] UE_API bool IsValid() const;

		// Returns the evaluation flags
		[[nodiscard]] EEvaluationFlags GetFlags() const { return EvaluationFlags; }

		// Pushes and moves the specified value on its evaluation stack
		template<typename ValueType>
		void PushValue(const FEvaluationVMStackName& StackName, ValueType&& Value);

		// Pops the top value from the evaluation stack
		// Returns true if the stack was not empty and a value is returned, false otherwise
		template<typename ValueType>
		[[nodiscard]] bool PopValue(const FEvaluationVMStackName& StackName, ValueType& OutValue);

		// Returns an immutable pointer to a value at some offset from the top of the evaluation stack
		// An offset of 0 means the top of the stack
		// Returns valid pointer if an entry exists, nullptr otherwise
		template<typename ValueType>
		[[nodiscard]] const ValueType* PeekValue(const FEvaluationVMStackName& StackName, uint32 Offset) const;

		// Returns a mutable pointer to a value at some offset from the top of the evaluation stack
		// An offset of 0 means the top of the stack
		// Returns valid pointer if an entry exists, nullptr otherwise
		template<typename ValueType>
		[[nodiscard]] ValueType* PeekValueMutable(const FEvaluationVMStackName& StackName, uint32 Offset);

		// Trims internal containers to only use as much memory as required to avoid slack
		UE_API void Shrink();

		// Returns a reference keyframe (bind pose or additive identity)
		[[nodiscard]] UE_API FKeyframeState MakeReferenceKeyframe(bool bAdditiveKeyframe) const;

		// Returns an uninitialized keyframe with memory pre-allocated
		[[nodiscard]] UE_API FKeyframeState MakeUninitializedKeyframe(bool bAdditiveKeyframe) const;

		// Returns the evaluation context that controls what state we produce
		[[nodiscard]] UE_API const FEvaluationTaskContext& GetActiveEvaluationContext() const;

		// Returns the named set we are currently evaluating with
		[[nodiscard]] UE_API const FAttributeNamedSetPtr& GetActiveNamedSet() const;

		// Pushes a new binding and named set to evaluate with
		// Returns true if we succeeded and the set exists in the specified binding, false otherwise
		[[nodiscard]] UE_API bool PushEvaluationContext(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName);

		// Pushes a new named set to evaluate with
		// Returns true if we succeeded and the set exists in the currently active binding, false otherwise
		[[nodiscard]] UE_API bool PushEvaluationContext(FName SetName);

		// Pops the current active named set
		// Returns whether or not we succeeded (we need to retain at least one named set)
		UE_API bool PopEvaluationContext();

		// Returns the current LOD
		[[nodiscard]] int32 GetCurrentLOD() const { return CurrentLOD; }

		// Returns the value transformer map
		[[nodiscard]] UE_API const FValueTransformerMapPtr& GetTransformerMap() const;

	private:
		// Disallow copy
		FEvaluationVM(const FEvaluationVM&) = delete;
		FEvaluationVM& operator=(const FEvaluationVM&) = delete;

		// Retrieves and optionally creates the stack (if not found) by the specified name
		UE_API FEvaluationVMStack& GetOrCreateStack(const FEvaluationVMStackName& StackName, uint32 TypeID);

		// Returns the specified stack if found, nullptr otherwise
		UE_API FEvaluationVMStack* FindStack(const FEvaluationVMStackName& StackName, uint32 TypeID);
		UE_API const FEvaluationVMStack* FindStack(const FEvaluationVMStackName& StackName, uint32 TypeID) const;

		UE_API const void* PeekValueImpl(const FEvaluationVMStackName& StackName, uint32 TypeID, uint32 Offset) const;

		// Various internal stacks that tasks can use (e.g. keyframe state)
		TMap<FName, FEvaluationVMStack> InternalStacks;

		// Reference pose
		const FReferencePose* ReferencePose = nullptr;

		// The internal context that controls what state we are producing
		TArray<FEvaluationTaskContext> EvaluationContextStack;

		// Shared map of value transformers
		FValueTransformerMapPtr ValueTransformerMap;

		// Current LOD we are evaluating at
		int32 CurrentLOD = 0;

		// Flags that control what we wish to evaluate
		EEvaluationFlags EvaluationFlags = EEvaluationFlags::All;

		// Default curve filter
		UE::Anim::FCurveFilter CurveFilter;

		// TODO: Use a stack based allocator for the stack entries and mark freed entries on Pop and coalesce when we can
		// Can allocate stack segments from memstack, limit allocation size to 1024 bytes, allocate segments of 8192 bytes
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementations

	template<typename ValueType>
	inline void FEvaluationVM::PushValue(const FEvaluationVMStackName& StackName, ValueType&& Value)
	{
		static_assert(alignof(ValueType) <= alignof(FEvaluationVMStackEntry), "Expected an alignment smaller or equal to 16");

		const uint32 TypeID = GetTypeID<ValueType>();
		FEvaluationVMStack& Stack = GetOrCreateStack(StackName, TypeID);

		if (!ensureMsgf(Stack.TypeID == TypeID, TEXT("Type mismatch! This evaluation stack is being queried with a different type than it was created with")))
		{
			return;
		}

		const uint32 EntrySize = sizeof(FEvaluationVMStackEntry) + sizeof(Value);
		FEvaluationVMStackEntry* EntryPtr = reinterpret_cast<FEvaluationVMStackEntry*>(FMemory::Malloc(EntrySize, alignof(FEvaluationVMStackEntry)));
		EntryPtr->Size = EntrySize;
		EntryPtr->Prev = Stack.Top;
		EntryPtr->Flags = 0;

		ValueType* ValuePtr = reinterpret_cast<ValueType*>(EntryPtr->GetValuePtr());
		new(ValuePtr) ValueType(MoveTemp(Value));

		if (Stack.Top == nullptr)
		{
			Stack.TypeDestructor = GetTypeDestructor<ValueType>();
		}

		Stack.Top = EntryPtr;
	}

	template<typename ValueType>
	[[nodiscard]] inline bool FEvaluationVM::PopValue(const FEvaluationVMStackName& StackName, ValueType& OutValue)
	{
		FEvaluationVMStack* Stack = FindStack(StackName, GetTypeID<ValueType>());
		if (Stack == nullptr)
		{
			// Stack not found
			return false;
		}

		FEvaluationVMStackEntry* EntryPtr = Stack->Top;
		if (EntryPtr == nullptr)
		{
			// Stack is empty
			return false;
		}

		Stack->Top = EntryPtr->Prev;

		ValueType* ValuePtr = reinterpret_cast<ValueType*>(EntryPtr->GetValuePtr());
		OutValue = MoveTemp(*ValuePtr);

		ValuePtr->~ValueType();
		FMemory::Free(EntryPtr);

		return true;
	}

	template<typename ValueType>
	[[nodiscard]] inline const ValueType* FEvaluationVM::PeekValue(const FEvaluationVMStackName& StackName, uint32 Offset) const
	{
		return reinterpret_cast<const ValueType*>(PeekValueImpl(StackName, GetTypeID<ValueType>(), Offset));
	}

	template<typename ValueType>
	[[nodiscard]] inline ValueType* FEvaluationVM::PeekValueMutable(const FEvaluationVMStackName& StackName, uint32 Offset)
	{
		// Re-use const impl and cast it away
		return const_cast<ValueType*>(reinterpret_cast<const ValueType*>(PeekValueImpl(StackName, GetTypeID<ValueType>(), Offset)));
	}

	//////////////////////////////////////////////////////////////////////////
	// Various commonly used VM stacks

	// A stack of FKeyframeState instances used when sampling sequences and blending their results
	extern UAFANIMGRAPH_API const FEvaluationVMStackName KEYFRAME_STACK_NAME;
	ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE(TUniquePtr<FKeyframeState>)

	// A stack of FValueBundle instances used when sampling sequences and blending their results
	extern UAFANIMGRAPH_API const FEvaluationVMStackName ATTRIBUTE_STACK_NAME;
	ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE(TUniquePtr<FValueBundle>)
}

#undef UE_API
