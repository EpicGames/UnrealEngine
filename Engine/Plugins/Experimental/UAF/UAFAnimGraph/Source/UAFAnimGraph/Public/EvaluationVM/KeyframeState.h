// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimCurveTypes.h"
#include "Animation/AttributesRuntime.h"
#include "LODPose.h"

#define UE_API UAFANIMGRAPH_API

struct FAnimNextGraphLODPose;

namespace UE::UAF
{
	/*
	 * Key Frame State
	 * 
	 * This struct holds sampled keyframe state or interpolated keyframe state.
	 * It holds a pose, trajectory, curves, attributes, etc that one might find in
	 * an animation sequence or as an output of an animation node.
	 */
	struct FKeyframeState
	{
		// Joint transforms at a particular LOD (on memstack)
		FLODPoseStack Pose;

		// Float curves at a particular LOD (on memstack)
		FBlendedCurve Curves;

		// Attributes at a particular LOD (on memstack)
		UE::Anim::FStackAttributeContainer Attributes;

		[[nodiscard]] bool IsValid() const { return Pose.IsValid(); }
		[[nodiscard]] bool IsAdditive() const { return Pose.IsAdditive(); }

		/** Copy from a graph pose, transparently handling cross-skeleton remapping. */
		UE_API void CopyFrom(const FAnimNextGraphLODPose& Source);
	};
}

#undef UE_API
