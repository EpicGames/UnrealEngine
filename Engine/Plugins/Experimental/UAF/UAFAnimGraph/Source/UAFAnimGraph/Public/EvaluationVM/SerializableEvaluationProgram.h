// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"

#include "SerializableEvaluationProgram.generated.h"

namespace UE::UAF
{
	struct FEvaluationProgram;
}

USTRUCT()
struct FSerializableEvaluationProgram
{
	GENERATED_BODY()
	
	FSerializableEvaluationProgram()
	{
	}
	
	FSerializableEvaluationProgram(const UE::UAF::FEvaluationProgram& Other);
	
	UPROPERTY(EditAnywhere, Category = "Tasks", meta = (ExpandByDefault))
	TArray<FInstancedStruct> Tasks;	
};
