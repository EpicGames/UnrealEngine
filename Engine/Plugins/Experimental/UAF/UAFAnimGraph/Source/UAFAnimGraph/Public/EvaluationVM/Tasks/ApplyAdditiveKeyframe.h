// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "ApplyAdditiveKeyframe.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FInputScaleBiasClamp;

namespace UE::UAF
{
	class FValueBundle;
	struct FKeyframeState;
}

/*
 * Apply Additive Keyframe Task
 *
 * This pops the top two keyframes from the VM keyframe stack, it applies an additive keyframe onto its base, and pushes
 * back the result onto the stack.
 * The top pose should be the additive keyframe and the second to the top the base keyframe.
 */
USTRUCT()
struct FAnimNextApplyAdditiveKeyframeTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextApplyAdditiveKeyframeTask)

	static UE_API FAnimNextApplyAdditiveKeyframeTask Make(float BlendWeight);
	static UE_API FAnimNextApplyAdditiveKeyframeTask Make(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// @TODO: Move this into shared method
	// Compute the current alpha value for curves or return the interpolation alpha otherwise
	UE_API float GetInterpolationAlpha(const UE::UAF::FKeyframeState* KeyframeA, const UE::UAF::FKeyframeState* KeyframeB) const;
	UE_API float GetInterpolationAlpha(const UE::UAF::FValueBundle& CollectionA, const UE::UAF::FValueBundle& CollectionB) const;

	// How much weight between the additive identity and the additive pose to apply: lerp(identity, additive, weight)
	UPROPERTY()
	float BlendWeight = 0.0f;

	// The curve to evaluate and extract the interpolation alpha between the two input keyframes
	FName AlphaSourceCurveName = NAME_None;

	int8 AlphaCurveInputIndex = INDEX_NONE;

	TFunction<float(float)> InputScaleBiasClampFn = nullptr;
};

#undef UE_API
