// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/Tasks/ApplyAdditiveKeyframe.h"

#include "ApplyAdditiveKeyframePerBone.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FBlendSampleData;
class IBlendProfileInterface;
class UUAFBlendMask;

/*
 * Apply Additive Keyframe Per Bone Task
 *
 * This pops the top two keyframes from the VM keyframe stack, it applies an additive keyframe with a weight per bone onto its base, 
 * and pushes back the result onto the stack.
 * If no valid blend data is provided it will revert back to a full pose additive application. 
 * 
 * The top pose should be the additive keyframe and the second to the top the base keyframe.
 */
USTRUCT()
struct FUAFApplyAdditiveKeyframePerBoneTask : public FAnimNextApplyAdditiveKeyframeTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FUAFApplyAdditiveKeyframePerBoneTask)

	static UE_API FUAFApplyAdditiveKeyframePerBoneTask Make(const float BlendWeight, const FBlendSampleData& BlendData, bool bPerBoneDataUsesSkeletonIndex);
	static UE_API FUAFApplyAdditiveKeyframePerBoneTask Make(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn, const FBlendSampleData& BlendData, bool bPerBoneDataUsesSkeletonIndex);

	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	const FBlendSampleData* BlendData = nullptr;
	bool bPerBoneDataUsesSkeletonIndex = false;
};

/*
 * Apply Additive Keyframe Per Bone Task
 *
 * This pops the top two keyframes from the VM keyframe stack, 
 * it applies an additive keyframe with a weight per bone based on the given blend mask onto its base and pushes back the result onto the stack.
 * If no valid blend mask is provided it will revert back to a full pose additive application. 
 * 
 * The top pose should be the additive keyframe and the second to the top the base keyframe.
 * 
 * This task currently only supports skeleton-based blend masks.
 */
USTRUCT()
struct FUAFApplyAdditiveKeyframeWithBlendMaskTask : public FAnimNextApplyAdditiveKeyframeTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FUAFApplyAdditiveKeyframeWithBlendMaskTask)
	
	static UE_API FUAFApplyAdditiveKeyframeWithBlendMaskTask Make(const float InBlendWeight, const UUAFBlendMask* InBlendMask);
	
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
	
	UPROPERTY()
	TObjectPtr<const UUAFBlendMask> BlendMask = nullptr;
};


#undef UE_API
