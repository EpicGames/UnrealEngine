// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "ApplyNamedSetTask.generated.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	/*
	 * Push Named Set Task
	 *
	 * This pushes a new named set to evaluate with. Subsequent tasks will use that set to evaluate.
	 */
	USTRUCT()
	struct FPushNamedSetTask : public FAnimNextEvaluationTask
	{
		GENERATED_BODY()

		DECLARE_ANIM_EVALUATION_TASK(FPushNamedSetTask)

		static UE_API FPushNamedSetTask Make(FName SetName);

		// Task entry point
		UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

		// The named set to push
		FName SetName = NAME_None;
	};

	/*
	 * Pop Named Set Task
	 *
	 * This pops the current active named set. Subsequent tasks will no longer use that named set.
	 * We only pop the active named set if it is the one we expect.
	 */
	USTRUCT()
	struct FPopNamedSetTask : public FAnimNextEvaluationTask
	{
		GENERATED_BODY()

		DECLARE_ANIM_EVALUATION_TASK(FPopNamedSetTask)

		static UE_API FPopNamedSetTask Make(FName ExpectedSetName);

		// Task entry point
		UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

		// The named set expected to pop
		FName ExpectedSetName = NAME_None;
	};
}

#undef UE_API
