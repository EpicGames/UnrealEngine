// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "BlendKeyframes.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FInputScaleBiasClamp;

namespace UE::UAF
{
	class FValueBundle;
	struct FKeyframeState;
}

/*
 * Blend Two Keyframes Task
 *
 * This pops the top two keyframes from the VM keyframe stack, it blends/interpolates them, and pushes
 * back the result onto the stack.
 * Let B be the input keyframe at the top of the stack and A be the second from the top.
 * Then we have:
 *     Result = Interpolate(A, B, Alpha)
 *     Top = Interpolate(Top-1, Top, Alpha)
 * An Alpha of 0.0 returns A while 1.0 returns B
 * 
 * Internally, this breaks down into the following tasks:
 *		* FAnimNextBlendOverwriteKeyframeWithScaleTask(1.0 - InterpolationAlpha)	// todo clarify alpha
 *		* FAnimNextBlendAddKeyframeWithScaleTask(InterpolationAlpha)
 *		* FAnimNextNormalizeKeyframeRotationsTask
 * 
 * In order to blend more than two keyframes together, simply add more FAnimNextBlendAddKeyframeWithScaleTask
 * tasks each with its corresponding weight before normalizing rotations.
 */

USTRUCT()
struct FAnimNextBlendTwoKeyframesTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendTwoKeyframesTask)

	static UE_API FAnimNextBlendTwoKeyframesTask Make(float InterpolationAlpha);
	static UE_API FAnimNextBlendTwoKeyframesTask Make(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn);
	

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// Compute the current alpha value for curves or return the interpolation alpha otherwise
	UE_API float GetInterpolationAlpha(const UE::UAF::FKeyframeState* KeyframeA, const UE::UAF::FKeyframeState* KeyframeB) const;
	UE_API float GetInterpolationAlpha(const UE::UAF::FValueBundle& CollectionA, const UE::UAF::FValueBundle& CollectionB) const;

	// The interpolation alpha between the two input keyframes
	UPROPERTY(VisibleAnywhere, Category = Properties)
	float InterpolationAlpha = 0.0f;

	// The curve to evaluate and extract the interpolation alpha between the two input keyframes
	FName AlphaSourceCurveName = NAME_None;

	int8 AlphaCurveInputIndex = INDEX_NONE;

	TFunction<float(float)> InputScaleBiasClampFn = nullptr;
};

/*
 * Blend Overwrite Keyframe With Scale Task
 *
 * This pops the top keyframe from the VM keyframe stack, it scales it by a factor, and pushes
 * back the result onto the stack.
 * Top = Top * ScaleFactor
 * 
 * Note that rotations will not be normalized after this task.
 */
USTRUCT()
struct FAnimNextBlendOverwriteKeyframeWithScaleTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendOverwriteKeyframeWithScaleTask)

	static UE_API FAnimNextBlendOverwriteKeyframeWithScaleTask Make(float ScaleFactor);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// The scale factor to apply to the keyframe
	UPROPERTY(VisibleAnywhere, Category = Properties)
	float ScaleFactor = 1.0f;
};

/*
 * Blend Add Keyframe With Scale Task
 *
 * This pops the top two keyframes (A and B) from the VM keyframe stack (let B be at the top).
 * B is our intermediary result that we add on top of; while A is the keyframe we scale.
 * The result is pushed back onto the stack.
 * Top = Top + (Top-1 * ScaleFactor)
 * 
 * Note that rotations will not be normalized after this task.
 */
USTRUCT()
struct FAnimNextBlendAddKeyframeWithScaleTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendAddKeyframeWithScaleTask)

	static UE_API FAnimNextBlendAddKeyframeWithScaleTask Make(float ScaleFactor);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// The scale factor to apply to the keyframe
	UPROPERTY(VisibleAnywhere, Category = Properties)
	float ScaleFactor = 0.0f;
};

#undef UE_API
