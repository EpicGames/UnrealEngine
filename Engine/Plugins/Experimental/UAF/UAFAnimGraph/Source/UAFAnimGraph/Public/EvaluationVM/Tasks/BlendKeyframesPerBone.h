// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"
#include "Animation/NamedValueArray.h"
#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "Animation/AttributesContainer.h"
#include "UAF/BlendMask/UAFBlendMask.h"

#include "BlendKeyframesPerBone.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FBlendSampleData;
class IBlendProfileInterface;
class IInterpolationIndexProvider;
class USkeleton;

/*
 * Blend Overwrite Keyframe Per Bone With Scale Task
 *
 * This pops the top keyframe from the VM keyframe stack, it scales each bone by a factor, and pushes
 * back the result onto the stack.
 * Top = Top * ScaleFactor
 * 
 * If no blend profile is provided but the blend data contains valid per bone blend weights,
 * it will try to use that data to blend each bone together with the assumption the provided data encompasses the full pose.
 * If neither is provided. this task behaves like FAnimNextBlendOverwriteKeyframeWithScaleTask
 * Note that rotations will not be normalized after this task.
 */
USTRUCT()
struct FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask : public FAnimNextBlendOverwriteKeyframeWithScaleTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask)

	static UE_API FAnimNextBlendOverwriteKeyframePerBoneWithScaleTask Make(const USkeleton* Skeleton, const FBlendSampleData& BlendData, float ScaleFactor);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// The blend data associated with the keyframe to overwrite
	// Cannot use a UPROPERTY with a pointer to a struct
	//UPROPERTY()
	const FBlendSampleData* BlendData = nullptr;

	// Optional skeleton to perform skeleton remapping with
	const USkeleton* SourceSkeleton = nullptr;
};

/*
 * Blend Add Keyframe Per Bone With Scale Task
 *
 * This pops the top two keyframes (A and B) from the VM keyframe stack (let B be at the top).
 * B is our intermediary result that we add on top of; while A is the keyframe we scale.
 * The result is pushed back onto the stack.
 * Top = Top + (Top-1 * ScaleFactor)
 * 
 * If no blend profile is provided but the blend data contains valid per bone blend weights, 
 * it will try to use that data to blend each bone together with the assumption the provided data encompasses the full pose. 
 * If neither is provided, this task behaves like FAnimNextBlendAddKeyframeWithScaleTask
 * 
 * Note that rotations will not be normalized after this task.
 */
USTRUCT()
struct FAnimNextBlendAddKeyframePerBoneWithScaleTask : public FAnimNextBlendAddKeyframeWithScaleTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendAddKeyframePerBoneWithScaleTask)

	static UE_API FAnimNextBlendAddKeyframePerBoneWithScaleTask Make(const USkeleton* Skeleton, const FBlendSampleData& BlendDataA, const FBlendSampleData& BlendDataB, float ScaleFactor);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// The blend data associated with the keyframe A
	// Cannot use a UPROPERTY with a pointer to a struct
	//UPROPERTY()
	const FBlendSampleData* BlendDataA = nullptr;

	// The blend data associated with the keyframe B
	// Cannot use a UPROPERTY with a pointer to a struct
	//UPROPERTY()
	const FBlendSampleData* BlendDataB = nullptr;

	// Optional skeleton to perform skeleton remapping with
	const USkeleton* SourceSkeleton = nullptr;
};


USTRUCT()
struct FAnimNextBlendKeyframePerBoneWithScaleTask : public FAnimNextBlendAddKeyframeWithScaleTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendKeyframePerBoneWithScaleTask)

	static UE_API FAnimNextBlendKeyframePerBoneWithScaleTask Make(const UUAFBlendMask* InBlendMask, const float ScaleFactor);

	// Curve Weights and Attribute Weights can be provided as empty views
	static UE_API FAnimNextBlendKeyframePerBoneWithScaleTask Make(
		const USkeleton* Skeleton,
		const UUAFBlendMask::FSkeletonBoneWeightArray* InBoneWeights,
		const UUAFBlendMask::FSkeletonCurveWeightArray* InCurveWeights,
		const UUAFBlendMask::FSkeletonAttributeWeightArray* InAttributeWeights,
		float ScaleFactor);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:
	const UUAFBlendMask::FSkeletonBoneWeightArray* BoneMaskWeights;
	const UUAFBlendMask::FSkeletonCurveWeightArray* CurveMaskWeights;
	const UUAFBlendMask::FSkeletonAttributeWeightArray* AttributeMaskWeights;
	
	const USkeleton* SourceSkeleton = nullptr;
};

#undef UE_API
