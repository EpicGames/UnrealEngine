// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "ConvertRotationsToMeshSpace.generated.h"

#define UE_API UAFANIMGRAPH_API

USTRUCT()
struct FAnimNextConvertRotationsToMeshSpaceTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextConvertRotationsToMeshSpaceTask)

	static UE_API FAnimNextConvertRotationsToMeshSpaceTask Make(const int32 NumPoses);

	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:
	// The number of poses on the stack to convert
	UPROPERTY(VisibleAnywhere, Category = Properties)
	int32 NumPoses = 0;
};

#undef UE_API
