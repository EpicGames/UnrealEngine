// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "TransformArray.h"
#include "TransformArrayView.h"
#include "AlphaBlend.h"

#include "DeadBlending.generated.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FDeadBlendingState;

	/**
	 * FDeadBlendTransitionTaskParameters
	 *
	 * Parameters for the Dead Blending Transition Task. This should be filled according to the 
	 * values provided by the Dead Blending Trait shared data.
	 */
	struct FDeadBlendTransitionTaskParameters
	{
		float ExtrapolationHalfLife = 0.0f;
		float ExtrapolationHalfLifeMin = 0.0f;
		float ExtrapolationHalfLifeMax = 0.0f;
		float MaximumTranslationVelocity = 0.0f;
		float MaximumRotationVelocity = 0.0f;
		float MaximumScaleVelocity = 0.0f;
	};
}

class UCurveFloat;

/**
 * FAnimNextDeadBlendingTransitionTask
 * 
 * Task for performing a Dead Blending transition. Stores the current pose state, computes the 
 * velocity using finite difference, and fits extrapolation half-lives based on the state being
 * transitioned to.
 */
USTRUCT()
struct FAnimNextDeadBlendingTransitionTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextDeadBlendingTransitionTask)

	// Make a DeadBlendingTransitionTask using two previous poses
	static UE_API FAnimNextDeadBlendingTransitionTask Make(
		UE::UAF::FDeadBlendingState* State,
		const UE::UAF::FTransformArraySoAHeap* CurrPose,
		const UE::UAF::FTransformArraySoAHeap* PrevPose,
		const float DeltaTime,
		const UE::UAF::FDeadBlendTransitionTaskParameters& Parameters);

	// Make a DeadBlendingTransitionTask using only a single previous poses
	static UE_API FAnimNextDeadBlendingTransitionTask Make(
		UE::UAF::FDeadBlendingState* State,
		const UE::UAF::FTransformArraySoAHeap* CurrPose,
		const UE::UAF::FDeadBlendTransitionTaskParameters& Parameters);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// Output

	UE::UAF::FDeadBlendingState* State = nullptr;

	// Inputs

	const UE::UAF::FTransformArraySoAHeap* CurrPose = nullptr;
	const UE::UAF::FTransformArraySoAHeap* PrevPose = nullptr;

	// Parameters

	float DeltaTime = 0.0f;
	UE::UAF::FDeadBlendTransitionTaskParameters Parameters;
};

/**
 * FAnimNextDeadBlendingApplyTask
 *
 * Task for applying dead blending to remove a discontinuity. Will extrapolate the FDeadBlendingState forward
 * in time and then blend it with the Keyframe which is on top of the stack according to the BlendDuration. Part
 * of the FDeadBlendingState will be updated to ensure the blend does not "flip sides" part way through.
 */
USTRUCT()
struct FAnimNextDeadBlendingApplyTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextDeadBlendingApplyTask)

	// Make a FAnimNextDeadBlendingApplyTask task
	static UE_API FAnimNextDeadBlendingApplyTask Make(
		UE::UAF::FDeadBlendingState* State,
		const float BlendDuration,
		const float TimeSinceTransition,
		const EAlphaBlendOption& BlendMode,
		const TWeakObjectPtr<UCurveFloat> CustomBlendCurve);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// Input / Output

	UE::UAF::FDeadBlendingState* State = nullptr;

	// Parameters

	float BlendDuration = 0.0f;
	float TimeSinceTransition = 0.0f;
	EAlphaBlendOption BlendMode = EAlphaBlendOption::Linear;
	
	UPROPERTY(VisibleAnywhere, Category = Properties)
	TWeakObjectPtr<UCurveFloat> CustomBlendCurve = nullptr;
};


#undef UE_API
