// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ReferencePose.h"
#include "LODPose.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AttributesRuntime.h"

#define UE_API UAFANIMGRAPH_API

struct FCompactPose;
struct FCompressedAnimSequence;
struct FAnimExtractContext;
struct FAnimSequenceDecompressionContext;
struct FRootMotionReset;
class UAnimSequence;

namespace UE::UAF
{
	class FValueBundle;
	class FEvaluationTaskContext;

class FDecompressionTools
{
public:
	// Returns whether decompression should use raw data or not
	static UE_API bool ShouldUseRawData(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext);

	// Extracted from UAnimSequence
	// Extracts Animation Data from the provided AnimSequence, using AnimExtractContext extraction parameters
	static UE_API void GetAnimationPose(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData, bool bForceUseRawData = false);
	static UE_API void GetAnimationPose(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FEvaluationTaskContext& VMContext, FValueBundle& OutCollection, bool bForceUseRawData = false);

	/**
	* Get Bone Transform of the Time given, relative to Parent for all RequiredBones
	* This returns different transform based on additive or not. Or what kind of additive.
	*/
	static UE_API void GetBonePose(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData, bool bForceUseRawData = false);

	static UE_API void GetBonePose_Additive(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData);
	static UE_API void GetBonePose_AdditiveMeshRotationOnly(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FLODPose& OutAnimationPoseData);

	static UE_API void GetAnimationCurves(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, FBlendedCurve& OutCurves, bool bForceUseRawData = false);
	static UE_API void GetAnimationCurves(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FEvaluationTaskContext& VMContext, FValueBundle& OutCollection, bool bForceUseRawData = false);

	static UE_API void GetAnimationAttributes(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FReferencePose& RefPose, UE::Anim::FStackAttributeContainer& OutAttributes, bool bForceUseRawData = false);
	static UE_API void GetAnimationAttributes(const UAnimSequence* AnimSequence, const FAnimExtractContext& ExtractionContext, const FEvaluationTaskContext& VMContext, FValueBundle& OutCollection, bool bForceUseRawData = false);

	// Decompress and retarget animation data using provided RetargetTransforms
	static UE_API void DecompressPose(FLODPose& OutAnimationPoseData,
		const FCompressedAnimSequence& CompressedData,
		const FAnimExtractContext& ExtractionContext,
		FAnimSequenceDecompressionContext& DecompressionContext,
		const TArray<FTransform>& RetargetTransforms,
		const FRootMotionReset& RootMotionReset);

	// Decompress and retarget animation data
	static UE_API void DecompressPose(FLODPose& OutAnimationPoseData,
		const FCompressedAnimSequence& CompressedData,
		const FAnimExtractContext& ExtractionContext,
		FAnimSequenceDecompressionContext& DecompressionContext,
		FName RetargetSource,
		const FRootMotionReset& RootMotionReset);
};

} // end namespace UE::UAF

#undef UE_API
