// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "ExecuteProgram.generated.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FEvaluationProgram;
}

/*
 * Execute Program Task
 *
 * This allows external caching of evaluation programs by deferring evaluation
 * or repeated evaluations.
 */
USTRUCT()
struct FAnimNextExecuteProgramTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextExecuteProgramTask)

	static UE_API FAnimNextExecuteProgramTask Make(TSharedPtr<const UE::UAF::FEvaluationProgram> Program);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// The program to execute
	TSharedPtr<const UE::UAF::FEvaluationProgram> Program = nullptr;
};

#undef UE_API
