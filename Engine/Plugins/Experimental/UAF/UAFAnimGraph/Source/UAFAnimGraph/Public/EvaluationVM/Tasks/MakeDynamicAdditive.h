// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EvaluationVM/EvaluationTask.h"
#include "Animation/AttributeTypes.h"

#include "MakeDynamicAdditive.generated.h"

#define UE_API UAFANIMGRAPH_API

/* Make Dynamic Additive Task
 * 
 * Pops the top two keyframes of the stack, creates a new additive pose and pushes it back onto the stack.
 * The top pose should be the keyframe to turn into an additive and the second to the top should be the base keyframe. 
*/
USTRUCT()
struct FUAFMakeDynamicAdditiveTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FUAFMakeDynamicAdditiveTask)

	static UE_API FUAFMakeDynamicAdditiveTask Make(const EAdditiveAnimationType InAdditiveType);
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
	
	EAdditiveAnimationType AdditiveType = AAT_LocalSpaceBase;
};

#undef UE_API
