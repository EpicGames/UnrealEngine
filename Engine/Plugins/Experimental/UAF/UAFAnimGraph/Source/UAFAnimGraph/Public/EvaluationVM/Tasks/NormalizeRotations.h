// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "NormalizeRotations.generated.h"

#define UE_API UAFANIMGRAPH_API

/*
 * Normalize Keyframe Rotations Task
 *
 * This pop the top keyframe from the VM keyframe stack, it normalizes its rotations, and pushes
 * back the result onto the stack.
 */
USTRUCT()
struct FAnimNextNormalizeKeyframeRotationsTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextNormalizeKeyframeRotationsTask)

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
};

#undef UE_API
