// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Animation/AnimationAsset.h"

#include "PushAnimSequenceKeyframe.generated.h"

#define UE_API UAFANIMGRAPH_API

class UAnimSequence;

/*
 * Push Anim Sequence Keyframe Task
 *
 * This pushes an anim sequence keyframe onto the top of the VM keyframe stack.
 */
USTRUCT()
struct FAnimNextAnimSequenceKeyframeTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextAnimSequenceKeyframeTask)

	static UE_API FAnimNextAnimSequenceKeyframeTask MakeFromSampleTime(TWeakObjectPtr<const UAnimSequence> AnimSequence, double SampleTime, bool bInterpolate);
	static UE_API FAnimNextAnimSequenceKeyframeTask MakeFromKeyframeIndex(TWeakObjectPtr<const UAnimSequence> AnimSequence, uint32 KeyframeIndex);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// Anim Sequence to grab the keyframe from
	UPROPERTY(VisibleAnywhere, Category = Properties)
	TWeakObjectPtr<const UAnimSequence> AnimSequence;

	/** Delta time range required for root motion extraction **/
	FDeltaTimeRecord DeltaTimeRecord;

	// The point in time within the animation sequence at which we sample the keyframe.
	// If negative, the sample time hasn't been provided and we should use the keyframe index.
	UPROPERTY(VisibleAnywhere, Category = Properties)
	double SampleTime = -1.0;

	// The specific keyframe within the animation sequence to retrieve.
	// If ~0, the keyframe index hasn't been provided and we should use the sample time.
	UPROPERTY(VisibleAnywhere, Category = Properties)
	uint32 KeyframeIndex = ~0;

	// Whether to interpolate or step the animation sequence.
	// Only used when the sample time is used.
	UPROPERTY(VisibleAnywhere, Category = Properties)
	bool bInterpolate = false;

	// Whether to extract trajectory or not
	UPROPERTY(VisibleAnywhere, Category = Properties)
	bool bExtractTrajectory = false;

	UPROPERTY(VisibleAnywhere, Category = Properties)
	bool bLooping = false;
};

#undef UE_API
