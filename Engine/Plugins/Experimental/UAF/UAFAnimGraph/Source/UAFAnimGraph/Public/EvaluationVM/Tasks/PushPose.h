// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Graph/AnimNext_LODPose.h"
#include "AnimNextStats.h"

#include "PushPose.generated.h"

#define UE_API UAFANIMGRAPH_API

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF Task: PushPose"), STAT_AnimNext_Task_PushPose, STATGROUP_AnimNext, UAFANIMGRAPH_API);

USTRUCT()
struct FAnimNextPushPoseTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextPushPoseTask)

	FAnimNextPushPoseTask() = default;
	UE_API FAnimNextPushPoseTask(const FAnimNextGraphLODPose* InGraphPose);

	static UE_API FAnimNextPushPoseTask Make(const FAnimNextGraphLODPose* InGraphPose);

	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UE_API FAnimNextPushPoseTask& operator=(const FAnimNextPushPoseTask& Other);

	const FAnimNextGraphLODPose* GraphPose = nullptr;
};

#undef UE_API
