// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"

#include "PushReferenceKeyframe.generated.h"

#define UE_API UAFANIMGRAPH_API

/*
 * Push Reference Keyframe Task
 *
 * This pushes a reference keyframe onto the top of the VM keyframe stack.
 * This task can be used to push the reference keyframe from a source skeleton
 * or the additive identity.
 */
USTRUCT()
struct FAnimNextPushReferenceKeyframeTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextPushReferenceKeyframeTask)

	static UE_API FAnimNextPushReferenceKeyframeTask MakeFromSkeleton();
	static UE_API FAnimNextPushReferenceKeyframeTask MakeFromAdditiveIdentity();

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// Whether or not the reference pose comes from the skeleton or is the additive identity
	UPROPERTY(VisibleAnywhere, Category = Properties)
	bool bIsAdditive = false;
};

#undef UE_API
