// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "TransformArray.h"

#include "StoreKeyframe.generated.h"

#define UE_API UAFANIMGRAPH_API

/**
 * Swap the two given Transform Arrays
 */
USTRUCT()
struct FAnimNextSwapTransformsTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextSwapTransformsTask)

	static UE_API FAnimNextSwapTransformsTask Make(UE::UAF::FTransformArraySoAHeap* A, UE::UAF::FTransformArraySoAHeap* B);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UE::UAF::FTransformArraySoAHeap* A = nullptr;
	UE::UAF::FTransformArraySoAHeap* B = nullptr;
};

/*
 * Store the pose on top of the stack in the given Transform Array
 */
USTRUCT()
struct FAnimNextStoreKeyframeTransformsTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextStoreKeyframeTransformsTask)

	static UE_API FAnimNextStoreKeyframeTransformsTask Make(UE::UAF::FTransformArraySoAHeap* Dest);

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UE::UAF::FTransformArraySoAHeap* Dest = nullptr;
};

/*
 * Duplicates the pose on top of the stack and pushes it as new top
 */
USTRUCT()
struct FAnimNextDuplicateTopKeyframeTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextDuplicateTopKeyframeTask)

	static UE_API FAnimNextDuplicateTopKeyframeTask Make();

	// Task entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
};

#undef UE_API
