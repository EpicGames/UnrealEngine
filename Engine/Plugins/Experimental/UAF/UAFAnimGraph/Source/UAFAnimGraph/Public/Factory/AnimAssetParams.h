// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimAssetParams.generated.h"

namespace UE::UAF
{
	UENUM(BlueprintType)
	enum class EAnimAssetLoopMode : uint8
	{
		/** Asset will use the looping value defined in the asset. */
		Auto,

		/** Asset will loop regardless of the value defined in the asset. */
		ForceLoop,

		/** Asset will NOT loop regardless of the value defined in the asset. */
		ForceNonLoop,
	};
}