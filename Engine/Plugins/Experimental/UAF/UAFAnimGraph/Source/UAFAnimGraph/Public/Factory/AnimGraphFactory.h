// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Templates/SubScriptStructOf.h"
#include "UAF/UAFAssetData.h"

class IUAFAssetData;
class UUAFAnimGraph;
class UObject;

namespace UE::UAF
{
	class FTraitWriter;
	struct IAnimGraphBuilder;
	struct FAnimGraphFactory;

	namespace AnimGraph
	{
		class FAnimNextAnimGraphModule;
	}
}

namespace UE::UAF
{

// Creates or recycles programmatically-generated graphs
// Uses the hash of the recipe to determine if the graph has been created already.
// If the hash matches one that has already been created but graph has already been GCed it will be re-created as needed.
struct FAnimGraphFactory
{
	// warning: duplicate functionality with IGraphFactory
	UAFANIMGRAPH_API static const UUAFAnimGraph* GetOrBuildGraph(UE::UAF::FGraphAssetHandleConstView InAsset, const FAnimNextFactoryParams& Params = FAnimNextFactoryParams());

	// Make a graph from the specified recipe
	UAFANIMGRAPH_API static const UUAFAnimGraph* BuildGraph(const IAnimGraphBuilder& InBuilder);

	// Make the default graph for running a UUAFAnimGraph asset (allowing graphs to blend between each other)
	UAFANIMGRAPH_API static const UUAFAnimGraph* GetDefaultGraphHost();

	UAFANIMGRAPH_API static FAnimNextFactoryParams GetDefaultParamsForAsset(FGraphAssetHandleConstView Asset);

	// Applies default initializer to the params (usually injecting the object pointer) 
	UAFANIMGRAPH_API static void InitializeDefaultParamsForAsset(FGraphAssetHandleConstView Asset, FAnimNextFactoryParams& InOutParams);

	template<typename TAssetData>
	using TParamsInitializer = TFunction<void(const TAssetData&, FAnimNextFactoryParams&)>;

	template<typename TAssetData>
	static void RegisterAsset(FAnimNextFactoryParams&& InParams, TParamsInitializer<TAssetData>&& InInitializer = nullptr)
	{
		static_assert(std::is_base_of_v<FUAFGraphFactoryAsset, TAssetData> == true); // TAssetData must derive from FUAFGraphAssetData

		if (InInitializer != nullptr)
		{
			RegisterAsset(TAssetData::StaticStruct(), MoveTemp(InParams), [InInitializer](const TConstStructView<FUAFGraphFactoryAsset>& AssetData, FAnimNextFactoryParams& OutParams)
			{
				if (InInitializer.IsSet())
				{
					InInitializer(*AssetData.GetPtr<TAssetData>(), OutParams);
				}
			});
		}
		else
		{
			RegisterAsset(TAssetData::StaticStruct(), MoveTemp(InParams), nullptr);
		}

	}

	static TConstArrayView<TSoftObjectPtr<UScriptStruct>> GetSupportedAssetDataTypes() { return AllFactoryStructs; };
	
private:
	// Called on module init
	UAFANIMGRAPH_API static void Init();

	// Called on module shutdown to unload everything
	UAFANIMGRAPH_API static void Destroy();

	// Called before engine shutdown to clear internal state while UObjects are still valid
	static void OnPreExit();

	// Function used to initialize the object for the factory-generated graph
	using FParamsInitializer = TFunction<void(const TConstStructView<FUAFGraphFactoryAsset>&, FAnimNextFactoryParams&)>;
	
	// Register default params used to build & initialize a graph according to the supplied asset class
	UAFANIMGRAPH_API static void RegisterAsset(const TSubScriptStructOf<FUAFGraphFactoryAsset>& AssetStructType, FAnimNextFactoryParams&& InParams, FParamsInitializer&& InInitializer);

	static const TPair<FAnimNextFactoryParams, FParamsInitializer>* FindGraphParamsForAssetData(FGraphAssetHandleConstView AssetData);
	static TMap<FTopLevelAssetPath, TPair<FAnimNextFactoryParams, FParamsInitializer>> DefaultParamsForObject;

	static TArray<TSoftObjectPtr<UScriptStruct>> AllFactoryStructs;

	friend AnimGraph::FAnimNextAnimGraphModule;
};

}