// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AnimNextAnimGraphBuilder.generated.h"

class UUAFAnimGraph;

namespace UE::UAF
{
	struct FAnimGraphFactory;
	class FTraitWriter;
	struct IAnimGraphBuilder;
	struct FAnimGraphBuilderContext;
}

namespace UE::UAF
{

enum class ETraitVariableMapping : uint8
{
	// Dont add trait's variables to the public variables of the graph
	None,
	// Add the trait's variables to the public variables and map all of them to the trait
	All,
};

// A method for making a graph in FAnimGraphFactory procedurally
struct IAnimGraphBuilder
{
	virtual ~IAnimGraphBuilder() = default;

private:
	friend FAnimGraphFactory;

	// Primary entry point for factory method for procedurally generating graphs
	// Fills the supplied context used to generate the graph
	// @return true if the graph factory context was written successfully, false otherwise
	virtual bool Build(FAnimGraphBuilderContext& InContext) const { return false; };

	// Get a uniquely-identifying key that we use to avoid rebuilding graphs
	UAFANIMGRAPH_API virtual uint64 GetKey() const;
};

}

// Base struct for procedural graph methods that need to persist or be referenced in data
USTRUCT()
struct FAnimNextAnimGraphBuilder
#if CPP
	: public UE::UAF::IAnimGraphBuilder
#endif
{
	GENERATED_BODY()

protected:
	// Invalidate the key if the method's params change, it will be lazily recalculated
	void InvalidateKey()
	{
		CachedKey = 0;
	}
	
	// Recalculates the key - called lazily when the key is retrieved
	virtual uint64 RecalculateKey() const { return 0; }

	// IAnimGraphBuilder interface
	UAFANIMGRAPH_API virtual uint64 GetKey() const override;

private:
	// Cached key based on serialized properties
	mutable uint64 CachedKey = 0;
};