// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextAnimGraphBuilder.h"
#include "InstanceTask.h"
#include "InstanceTaskContext.h"
#include "AnimNextSimpleAnimGraphBuilder.h"
#include "UAFAssetInstanceComponent.h"
#include "TraitCore/TraitSharedData.h"
#include "AnimNextFactoryParams.generated.h"

class UUAFAnimGraph;
class UAnimNextAnimGraphSettings;
class UUAFRigVMAsset;
struct FAnimNextFactoryParams;
struct FAnimNextGraphInstanceTaskInstanceData;
struct FAnimNextGraphInstance;

namespace UE::UAF
{
	struct FAnimGraphFactory;
}

namespace UE::UAF::Editor
{
	class FAnimNextFactoryParamsDetails;
};

// Set of parameters that describe a way of generating & initializing an animation graph
// TODO: To add an internal trait, use AddInternalTrait.
// TODO: Interface to map public interface to internal traits 
USTRUCT(BlueprintType)
struct FAnimNextFactoryParams
{
	GENERATED_BODY()

	FAnimNextFactoryParams() = default;

	// Whether these params are valid for application
	bool IsValid() const
	{
		return Builder.IsValid();
	}
	
	// Push a trait struct onto the specified stack in the graph builder, initialized to the struct's defaults.
	// The trait struct forms part of the created instance's 'public interface' if InMapping is 'All'.
	// If a trait already exists of this type then its values be overriden
	template<typename StructType>
	void AddTraitStruct(UE::UAF::ETraitVariableMapping InMapping, int32 InStackIndex)
	{
		TInstancedStruct<FAnimNextTraitSharedData> InstancedStruct;
		InstancedStruct.InitializeAsScriptStruct(TBaseStructure<StructType>::Get());
		Builder.AddTraitInstancedStruct(InStackIndex, InMapping, MoveTemp(InstancedStruct));
	}

	// Access a trait's struct during initialization (for instance to inject an AnimSequence)
	template<typename StructType>
	bool AccessTraitStruct(int32 InStackIndex, TFunctionRef<void(StructType&)> InFunction)
	{
		if (!Builder.Stacks.IsValidIndex(InStackIndex))
		{
			return false;
		}

		for (FAnimNextSimpleAnimGraphBuilderTraitDesc& StructData : Builder.Stacks[InStackIndex].TraitDescs)
		{
			if (StructData.TraitData.GetScriptStruct() == TBaseStructure<StructType>::Get())
			{
				InFunction(StructData.TraitData.GetMutable<StructType>());
				return true;
			}
		}
		return false;
	}

	void AddPublicVariablesStructByType(const TObjectPtr<const UScriptStruct>& InStruct)
	{
		Builder.AddVariableStruct(InStruct);
	} 

	// Add a public variables rig VM asset to these params. This will act as part of the public API of the system.
	// If a asset reference already exists of this type then its values be overriden by virtue of the initializer running later
	void AddPublicVariablesRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAsset)
	{
		Builder.AddVariablesRigVMAsset(RigVMAsset);
	}

	// Add a public shared variables asset to these params. This will act as part of the public API of the system.
	// If a asset reference already exists of this type then its values be overriden by virtue of the initializer running later
	void AddPublicVariablesAsset(const TObjectPtr<const UUAFRigVMAsset>& UAFAsset)
	{
		Builder.AddVariablesAsset(UAFAsset);
	}

	// Push a trait struct onto the specified stack in the graph builder, initialized to the supplied value
	// The trait struct forms part of the created instance's 'public interface' if InMapping is 'All'.
	// If a trait already exists of this type then its values be overriden
	template<typename StructType>
	void AddTrait(UE::UAF::ETraitVariableMapping InMapping, int32 InStackIndex, const StructType& InValue)
	{
		Builder.AddTraitStructView(InStackIndex, InMapping, InValue);
	}

	// Add a link between the output of the specified stack and the input stack's child
	void AddLink(int32 InOutputStackIndex, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputChildName)
	{
		Builder.AddLink(InOutputStackIndex, InInputStackIndex, InInputTraitIndex, InInputChildName);
	}

	// Add a variable struct
	template<typename StructType>
	void AddVariableStruct()
	{
		Builder.AddVariableStruct(TBaseStructure<StructType>::Get());
	}

	// Add a variable mapping between a public variable and a specific trait's latent property
	FAnimNextFactoryParams& AddVariableMapping(FAnimNextVariableReference InVariable, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputPropertyName)
	{
		Builder.AddVariableMapping(InVariable, InInputStackIndex, InInputTraitIndex, InInputPropertyName);
		return *this;
	}

	// Add a variable mapping between a public variable and all trait latent properties of the specified name
	FAnimNextFactoryParams& AddVariableMappingToAll(FAnimNextVariableReference InVariable, FName InInputPropertyName)
	{
		Builder.AddVariableMappingToAll(InVariable, InInputPropertyName);
		return *this;
	}

	// Add a component
	template<typename ComponentType>
	void AddComponent()
	{
		static_assert(std::is_base_of_v<FUAFAssetInstanceComponent,  ComponentType>, "Components must derive from FUAFAssetInstanceComponent");
		Builder.AddComponentStruct(TBaseStructure<ComponentType>::Get());
	}

	// Get the builder that these params hold
	const UE::UAF::IAnimGraphBuilder& GetBuilder() const
	{
		return Builder;
	}

	// Add an initialize task used to set up an instance when these params are applied
	FAnimNextFactoryParams& AddInitializeTask(UE::UAF::FInstanceTask&& InTask)
	{
		InitializeTasks.Add(MoveTemp(InTask));
		return *this;
	}

	// Clear these parameters back to empty
	UAFANIMGRAPH_API void Reset();

private:
	friend UE::UAF::FAnimGraphFactory;
	friend UE::UAF::Editor::FAnimNextFactoryParamsDetails;
	friend FAnimNextGraphInstanceTaskInstanceData;	// For old data upgrade
	friend UUAFAnimGraph;
	friend FAnimNextGraphInstance;

	// Apply these params to the supplied instance.
	UAFANIMGRAPH_API void InitializeInstance(FUAFAssetInstance& InInstance) const;

	// Method used to create a graph
	UPROPERTY(EditAnywhere, Category = Parameters)
	FAnimNextSimpleAnimGraphBuilder Builder;

	// Tasks to run on the factory-generated graph instance via InitializeInstance, used for initial setup of variables etc.
	TArray<UE::UAF::FInstanceTask> InitializeTasks;
};
