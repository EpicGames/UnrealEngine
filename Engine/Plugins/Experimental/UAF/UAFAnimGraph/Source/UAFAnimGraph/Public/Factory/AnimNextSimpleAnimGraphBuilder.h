// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextAnimGraphBuilder.h"
#include "StructUtils/StructView.h"
#include "Variables/AnimNextVariableReference.h"
#include "AnimNextSimpleAnimGraphBuilder.generated.h"

#define UE_API UAFANIMGRAPH_API

class IRigVMRuntimeAssetInterface;
struct FAnimNextTraitSharedData;
struct FAnimNextSimpleAnimGraphBuilder;
class FAnimationAnimNextRuntimeTest_SimpleGraphBuilder;
struct FAnimNextGraphInstanceTaskInstanceData;
struct FAnimNextFactoryParams;
struct FUAFAssetDataTests;

namespace UE::UAF
{
struct FAnimGraphFactory;
};

namespace UE::UAF::Editor
{
	class FAnimNextFactoryParamsDetails;
};

USTRUCT()
struct FAnimNextSimpleAnimGraphBuilderLinkDesc
{
	GENERATED_BODY()

private:
	friend UE::UAF::FAnimGraphFactory;
	friend FAnimNextSimpleAnimGraphBuilder;

	// The index of the stack this stack's output is linked to
	UPROPERTY()
	int32 StackIndex = INDEX_NONE;

	// The index of the trait this stack's output is linked to
	UPROPERTY()
	int32 TraitIndex = INDEX_NONE;

	// The name of the child this stack's output is linked to
	UPROPERTY()
	FName ChildName;
};

USTRUCT()
struct FAnimNextSimpleAnimGraphBuilderVariableMapping
{
	GENERATED_BODY()

private:
	friend UE::UAF::FAnimGraphFactory;
	friend FAnimNextSimpleAnimGraphBuilder;
	
	UPROPERTY()
	FAnimNextVariableReference VariableReference;

	// The name of the property the variable should be applied to
	UPROPERTY()
	FName PropertyName;
};

USTRUCT()
struct FAnimNextSimpleAnimGraphBuilderTraitDesc
{
	GENERATED_BODY()

private:
	friend UE::UAF::FAnimGraphFactory;
	friend FAnimNextSimpleAnimGraphBuilder;
	friend FAnimNextFactoryParams;
	friend FUAFAssetDataTests;
	friend FAnimNextGraphInstanceTaskInstanceData;	// For old data upgrade
	friend UE::UAF::Editor::FAnimNextFactoryParamsDetails;

	UPROPERTY(EditAnywhere, Category = Parameters)
	TInstancedStruct<FAnimNextTraitSharedData> TraitData;
	
	UPROPERTY()
	TArray<FAnimNextSimpleAnimGraphBuilderVariableMapping> VariableMappings;
};

USTRUCT()
struct FAnimNextSimpleAnimGraphBuilderTraitStackDesc
{
	GENERATED_BODY()

private:
	friend UE::UAF::FAnimGraphFactory;
	friend FAnimNextSimpleAnimGraphBuilder;
	friend FAnimNextGraphInstanceTaskInstanceData;	// For old data upgrade
	friend UE::UAF::Editor::FAnimNextFactoryParamsDetails;
	friend FAnimNextFactoryParams;
	friend struct FUAFAssetDataTests;

#if WITH_EDITORONLY_DATA
	// Traits for this stack
	UPROPERTY()
	TArray<TInstancedStruct<FAnimNextTraitSharedData>> TraitStructs_DEPRECATED;
#endif

	// Traits for this stack
	UPROPERTY(EditAnywhere, Category = Parameters)
	TArray<FAnimNextSimpleAnimGraphBuilderTraitDesc> TraitDescs;

	// The link for this stack's output pose
	UPROPERTY()
	FAnimNextSimpleAnimGraphBuilderLinkDesc Link;
};

USTRUCT()
struct FAnimNextSimpleAnimGraphBuilder : public FAnimNextAnimGraphBuilder
{
	GENERATED_BODY()

	// Push a trait struct onto the supplied stack. Traits are ordered from bottom to top, stack-wise.
	// Struct must be a sub-struct of FAnimNextTraitSharedData
	UE_API void AddTraitStructView(int32 InStackIndex, UE::UAF::ETraitVariableMapping InMapping, TConstStructView<FAnimNextTraitSharedData> InStruct);
	UE_API void AddTraitInstancedStruct(int32 InStackIndex, UE::UAF::ETraitVariableMapping InMapping, TInstancedStruct<FAnimNextTraitSharedData>&& InStruct);

	// Add a link between the output of the specified stack and the input stack's child
	UE_API void AddLink(int32 InOutputStackIndex, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputChildName);

	// Add a variable mapping between a public variable and a trait's latent property
	UE_API void AddVariableMapping(FAnimNextVariableReference InVariable, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputPropertyName);

	// Add a variable mapping between a public variable and all trait latent properties of the specified name
	UE_API void AddVariableMappingToAll(FAnimNextVariableReference InVariable, FName InInputPropertyName);

	// Add a variable struct
	UE_API void AddVariableStruct(const UScriptStruct* InStruct);

	UE_API void AddVariablesRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAssetInterface);
	UE_API void AddVariablesAsset(const TObjectPtr<const UUAFRigVMAsset>& UAFAsset);

	// Add a component struct to this builder
	UE_API void AddComponentStruct(const UScriptStruct* InStruct);

	// Get the number of stacks from this graph
	int32 GetNumStacks() const
	{
		return Stacks.Num();
	}

	// Check whether this method is empty or not
	bool IsValid() const
	{
		return Stacks.Num() > 0;
	}

	// Reset back to empty state
	void Reset()
	{
		Stacks.Empty();
		VariableStructs.Empty();
		ComponentStructs.Empty();
		ReferencedVariableRigVMAssets.Empty();
		ReferencedVariableAssets.Empty();
	}

private:
	friend UE::UAF::FAnimGraphFactory;
	friend FAnimationAnimNextRuntimeTest_SimpleGraphBuilder;
	friend FAnimNextGraphInstanceTaskInstanceData;	// For old data upgrade
	friend UE::UAF::Editor::FAnimNextFactoryParamsDetails;
	friend FAnimNextFactoryParams;
	friend struct FUAFAssetDataTests;

	// Internal helpers
	bool Validate() const;
	bool ValidateTraitStruct(int32 InStackIndex, int32 InTraitIndex, TConstStructView<FAnimNextTraitSharedData> InStruct) const;
	bool ValidateLink(int32 InOutputStackIndex, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputChildName) const;
	bool ValidateVariableMapping(FAnimNextVariableReference InVariable, int32 InInputStackIndex, int32 InInputTraitIndex, FName InInputPropertyName) const;
	void AutoMapTraitHelper(int32 InStackIndex, const UScriptStruct* TraitStruct);

	// FAnimNextAnimGraphBuilder interface
	UE_API virtual bool Build(UE::UAF::FAnimGraphBuilderContext& InContext) const override;
	UE_API virtual uint64 RecalculateKey() const override;

	// The trait stacks we will use to generate our data
	UPROPERTY(EditAnywhere, Category = Parameters)
	TArray<FAnimNextSimpleAnimGraphBuilderTraitStackDesc> Stacks;

	// All variable structs that form the public interface
	UPROPERTY()
	TArray<TObjectPtr<const UScriptStruct>> VariableStructs;

	// The structs we used to generate the graph's components
	UPROPERTY()
	TArray<TObjectPtr<const UScriptStruct>> ComponentStructs;

	// All the IRigVMRuntimeAssetInterface whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedVariableRigVMAssets;

	// All the assets whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TObjectPtr<const UUAFRigVMAsset>> ReferencedVariableAssets;
};

#undef UE_API