// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphTaskContext.h"
#include "EngineDefines.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Templates/NonNullPointer.h"
#include "Templates/SharedPointer.h"

#define UE_API UAFANIMGRAPH_API

struct FUAFWeakSystemReference;
class UUAFAnimGraph;
struct FAnimNextGraphInstance;
class FReferenceCollector;

namespace UE::UAF
{
struct FTraitBinding;
struct FExecutionContext;
struct FWeakTraitPtr;
struct FWeakAnimGraphReference;
}

namespace UE::UAF
{

// An owning reference to a UAF graph.
// Provides a way for external systems to own an anim graph, but not have to fully allocate the graph before use
struct FAnimGraphReference
{
	// Default-constructable only because it needs to be embedded in UObjects/UStructs
	FAnimGraphReference() = default;

	// Construct an owning handle to a graph instance.
	// Constructs a FAnimNextGraphInstance for use by worker threads and minimally pre-allocates it
	// Can be called on any thread
	// @param    InAnimGraph        The graph asset to create this instance from
	// @param    InSystemReference  The system that the graph will be used with. Graphs can only be used within a single system
	// @param    InFactoryParams    The factory params to construct the instance with
	UE_API explicit FAnimGraphReference(TNonNullPtr<const UUAFAnimGraph> InAnimGraph, const FUAFWeakSystemReference& InSystemReference, FAnimNextFactoryParams&& InFactoryParams = FAnimNextFactoryParams());

	// Make a weak reference from this owning reference
	UE_API FWeakAnimGraphReference ToWeakRef() const;

	// Reset this handle to invalid - gives up ownership
	// Can be called on any thread
	UE_API void Reset();

	// Check if this handle is valid
	// Can be called on any thread.
	UE_API bool IsValid() const;

	// Get the anim graph that this handle references
	// Can be called on any thread.
	[[nodiscard]] UE_API const UUAFAnimGraph* GetAnimGraph() const;

	// Queue task to execute on the instance
	// Can be called on any thread.
	// @param	InTaskFunction		The function to run
	UE_API void QueueTask(FUniqueAnimGraphTask&& InTaskFunction) const;

	// Sets the debug name for the instance, used for tracing etc.
	UE_API void SetDebugName(FName InName) const;

	// GC support
	UE_API void AddReferencedObjects(FReferenceCollector& InCollector) const;

private:
	// Gets or allocate & binds the graph instance
	// Fully allocates the graph if it is only preallocated
	// Can be called on any thread.
	// @param    InBinding    The current binding (this must be called during a graph execution)
	[[nodiscard]] UE_API TSharedPtr<FAnimNextGraphInstance> GetOrAllocate(FExecutionContext& InContext, const FTraitBinding& InBinding) const;

private:
	friend FWeakAnimGraphReference;
	
	// Pointer to the instance
	TSharedPtr<FAnimNextGraphInstance> Ptr;
};

}

#undef UE_API