// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstanceTaskContext.h"
#include "Templates/Function.h"

#define UE_API UAFANIMGRAPH_API 

struct FAnimNextGraphInstance;

namespace UE::UAF
{
struct FAnimGraphReference;
struct FWeakAnimGraphReference;
}

namespace UE::UAF
{

// Context struct passed to graph instance tasks, allowing setting of variables etc.
struct FAnimGraphTaskContext : public FInstanceTaskContext
{
public:
	// Get the graph instance
	UE_API FAnimNextGraphInstance& GetAnimGraphInstance() const;

private:
	friend FAnimNextGraphInstance;
	friend UE::UAF::FAnimGraphReference;
	friend UE::UAF::FWeakAnimGraphReference;

	UE_API FAnimGraphTaskContext(FAnimNextGraphInstance& InGraphInstance);
};

using FUniqueAnimGraphTask = ::TUniqueFunction<void(const FAnimGraphTaskContext& InContext)>;

}

#undef UE_API