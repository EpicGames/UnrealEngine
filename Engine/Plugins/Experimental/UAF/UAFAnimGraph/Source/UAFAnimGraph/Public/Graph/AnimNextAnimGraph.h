// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/StructView.h"
#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitPtr.h"
#include "AnimNextAnimGraph.generated.h"

#define UE_API UAFANIMGRAPH_API

class FReferenceCollector;

struct FAnimNextGraphInstance;
struct FRigUnit_AnimNextGraphEvaluator;
struct FRigUnit_AnimNextRunAnimationGraph_v1;
struct FRigUnit_AnimNextRunAnimationGraph_v2;
struct FRigUnit_UAFRunAsset;
class UUAFAnimGraph;
class FRigVMTraitScope;
struct FRigVMExtendedExecuteContext;
struct FAnimNextExecuteContext;
struct FAnimNextModuleAnimGraphComponent;
struct FStructView;
struct FAnimNextModuleInjectionComponent;
struct FRigUnit_GetPostProcessAnimation;

namespace UE::UAF
{
	struct IEvaluationModifier;
	struct FExecutionContext;
	struct FGraphInstanceComponent;
	struct FBlendStackCoreTrait;
	struct FInjectionSiteTrait;
	struct FInjectionInfo;
}

namespace UE::UAF::Editor
{
	class FAnimNextGraphDetails;
}

namespace UE::UAF::AnimGraph
{
	class FAnimNextAnimGraphModule;
}

// Injection data for an AnimNext Graph 
USTRUCT(BlueprintType)
struct FAnimNextGraphInjectionData
{
	GENERATED_BODY()

	bool HasValidEvaluationModifier() const { return EvaluationModifier.IsValid(); }
private:
	// Evaluation modifier to allow IEvaluate pass to be controlled
	TWeakPtr<UE::UAF::IEvaluationModifier> EvaluationModifier;

	// Serial number for the last injection that was made to this struct
	uint32 InjectionSerialNumber = 0;

	friend UE::UAF::FInjectionSiteTrait;
	friend FAnimNextModuleInjectionComponent;
};

// Represents an instance of an AnimNext graph, either an asset or some externally-provided graph instance.
// The asset can be set to refer directly to a graph asset, or it can be used via a 'factory' to generate an appropriate graph to run.
// When public, represents an 'injection site' that can be used to parameterize graph execution externally. 
//~ This struct uses UE reflection because we wish for the GC to keep the graph
//~ alive while we own a reference to it (specifically when running an animation graph).
//~ It is not intended to be serialized on disk with a live instance.
USTRUCT(BlueprintType, DisplayName="UAF Graph")
struct FAnimNextAnimGraph
{
	GENERATED_BODY()

	// Creates an empty graph instance that doesn't reference anything
	FAnimNextAnimGraph() = default;

	// The asset to run as an animation graph
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Graph, meta=(GetAllowedClasses="/Script/UAFAnimGraph.AnimNextAnimGraphSettings:GetAllowedAssetClasses", AllowPrivateAccess))
	TObjectPtr<const UObject> Asset;

private:
	// Check if this graph is 'equal' when applied to an injection site (i.e. whether a change should trigger a re-injection)
	UE_API bool IsEqualForInjectionSiteChange(const FAnimNextAnimGraph& InOther) const;

	// Injection data used to override the supplied graph
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=Graph, meta=(AllowPrivateAccess))
	FAnimNextGraphInjectionData InjectionData;

	// The host graph to use to run the asset. Applies only to top-level graphs being run in modules.
	// We use a host graph to be able to blend between graph sources at the top level of a module when a RunGraph's input graph changes.
	// If this is unset, then the project's default host graph will be used.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Graph, AdvancedDisplay,  meta=(AllowPrivateAccess))
	TObjectPtr<UUAFAnimGraph> HostGraph;

	friend FRigUnit_AnimNextRunAnimationGraph_v1;	// We evaluate the instance
	friend FRigUnit_AnimNextRunAnimationGraph_v2;	// We evaluate the instance
	friend FRigUnit_UAFRunAsset;					// We evaluate the instance
	friend UE::UAF::FBlendStackCoreTrait;		// Requires access to take a reference to external graphs
	friend UE::UAF::FInjectionSiteTrait;		// Requires access to take a reference to external graphs
	friend UE::UAF::FInjectionInfo;
	friend FAnimNextModuleInjectionComponent;
	friend UE::UAF::Editor::FAnimNextGraphDetails;
	friend FRigUnit_GetPostProcessAnimation;
	friend UE::UAF::AnimGraph::FAnimNextAnimGraphModule;
};

#undef UE_API
