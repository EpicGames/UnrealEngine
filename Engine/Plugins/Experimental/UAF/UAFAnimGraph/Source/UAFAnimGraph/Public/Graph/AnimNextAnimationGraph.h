// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/AnimNextGraphEntryPoint.h"
#include "Variables/AnimNextSharedVariables.h"
#include "RigVMCore/RigVM.h"
#include "TraitCore/TraitPtr.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/EntryPointHandle.h"
#include "RigVMHost.h"
#include "AnimNextGraphEvaluatorExecuteDefinition.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Module/UAFWeakSystemReference.h"

#include "AnimNextAnimationGraph.generated.h"

#define UE_API UAFANIMGRAPH_API

class UEdGraph;
class UUAFSystem;
class UAnimGraphNode_AnimNextGraph;
struct FAnimNode_AnimNextGraph;
struct FRigUnit_AnimNextGraphEvaluator;
struct FAnimNextGraphInstance;
struct FAnimNextScheduleGraphTask;
struct FAnimNextEditorParam;
struct FAnimNextParam;
struct FAnimNextModuleInstance;
struct FAnimNextModuleAnimGraphComponent;

namespace UE::UAF
{
	struct FContext;
	struct FExecutionContext;
	class FAnimNextModuleImpl;
	struct FTestUtils;
	struct FParametersProxy;
	struct FPlayAnimSlotTrait;
	struct FBlendStackCoreTrait;
	struct FVariableOverrides;
	struct FAnimGraphFactory;
	struct FAnimGraphBuilderContext;
	struct FVariableOverridesCollection;
}

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

namespace UE::UAF::Editor
{
	class FModuleEditor;
	class FVariableCustomization;
}

namespace UE::UAF::AnimGraph
{
	class FAnimNextAnimGraphModule;
}

namespace UE::UAF::Graph
{
	extern UAFANIMGRAPH_API const FName EntryPointName;
	extern UAFANIMGRAPH_API const FName ResultName;
}

namespace UE::UAF
{
	// Parameters to AllocateInstance
	struct FGraphAllocationParams
	{
		// The system to use as our outermost host
		FUAFWeakSystemReference SystemReference;
		// The hosting/parent graph instance
		TWeakPtr<FAnimNextGraphInstance> ParentGraphInstance;
		// The entry point to use (deprecated)
		FName EntryPoint = NAME_None;
		// Any variable overrides to apply to this graph
		TSharedPtr<const UE::UAF::FVariableOverridesCollection> Overrides;
	};
}

// A user-created collection of animation logic & data
UCLASS(MinimalAPI, BlueprintType)
class UUAFAnimGraph : public UUAFSharedVariables
{
	GENERATED_BODY()

public:
	UE_API UUAFAnimGraph(const FObjectInitializer& ObjectInitializer);

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void BeginDestroy() override;
#endif
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#endif

	// Pre-allocates an instance of the graph
	// @param	InParams		Parameters to pre-allocate the graph with
	// @param	InFactoryParams	Factory parameters to apply to the graph when allocated
	// @return the pre-allocated instance.
	UE_API TSharedPtr<FAnimNextGraphInstance> PreAllocateInstance(const UE::UAF::FGraphAllocationParams& InParams = UE::UAF::FGraphAllocationParams(), FAnimNextFactoryParams&& InFactoryParams = FAnimNextFactoryParams()) const;

	// Allocates an instance of the graph according to the supplied parameters
	// @param	InParams		Parameters to allocate the graph with
	// @param	InFactoryParams	Factory parameters to apply to the graph when allocated
	// @return the allocated instance. This can be invalid if the entry point was not found or the graph's root trait is invalid.
	UE_API TSharedPtr<FAnimNextGraphInstance> AllocateInstance(const UE::UAF::FGraphAllocationParams& InParams = UE::UAF::FGraphAllocationParams(), FAnimNextFactoryParams&& InFactoryParams = FAnimNextFactoryParams()) const;

private:
	// Loads the graph data from the provided archive buffer and returns true on success, false otherwise
	UE_API bool LoadFromArchiveBuffer(const TArray<uint8>& SharedDataArchiveBuffer);

#if WITH_EDITOR
	void OnPreForceDeleteObjects(const TArray<UObject*>& ObjectsToDelete);
#endif

#if WITH_EDITORONLY_DATA
	// During graph compilation, if we have existing graph instances, we freeze them by releasing their memory before thawing them
	// Freezing is a partial release of resources that retains the necessary information to re-create things safely
	UE_API void FreezeGraphInstances();

	// During graph compilation, once compilation is done we thaw existing graph instances to reallocate their memory
	UE_API void ThawGraphInstances();
#endif
	
	friend class UUAFAnimGraphFactory;
	friend class UUAFAnimGraph_EditorData;
	friend class UAnimNextVariableEntry;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend class UE::UAF::Editor::FModuleEditor;
	friend struct UE::UAF::FTestUtils;
	friend FAnimNextGraphInstance;
	friend class UAnimGraphNode_AnimNextGraph;
	friend UE::UAF::FExecutionContext;
	friend struct FAnimNextScheduleGraphTask;
	friend UE::UAF::AnimGraph::FAnimNextAnimGraphModule;
	friend class UE::UAF::Editor::FVariableCustomization;
	friend struct UE::UAF::FParametersProxy;
	friend struct UE::UAF::FPlayAnimSlotTrait;
	friend struct UE::UAF::FBlendStackCoreTrait;
	friend struct FAnimNextModuleAnimGraphComponent;
	friend UE::UAF::FAnimGraphBuilderContext;

protected:
#if WITH_EDITORONLY_DATA
	mutable FCriticalSection GraphInstancesLock;

	// This is a list of live graph instances that have been allocated, used in the editor to reset instances when we re-compile/live edit
	mutable TSet<FAnimNextGraphInstance*> GraphInstances;
#endif

	// This is the execute method definition used by a graph to evaluate latent pins
	UPROPERTY()
	FAnimNextGraphEvaluatorExecuteDefinition ExecuteDefinition;

	// Data for each entry point in this graph
	UPROPERTY()
	TArray<FAnimNextGraphEntryPoint> EntryPoints;

	// This is a resolved handle to the root trait in our graph, for each entry point 
	TMap<FName, FAnimNextTraitHandle> ResolvedRootTraitHandles;

	// This is an index into EntryPoints, for each entry point
	TMap<FName, int32> ResolvedEntryPoints;

	// This is the graph shared data used by the trait system, the output of FTraitReader
	// We de-serialize manually into this buffer from the archive buffer, this is never saved on disk
	TArray<uint8> SharedDataBuffer;

	// This is a list of all referenced UObjects in the graph shared data
	// We collect all the references here to make it quick and easy for the GC to query them
	// It means that object references in the graph shared data are not visited at runtime by the GC (they are immutable)
	// The shared data serialization archive stores indices to these to perform UObject serialization
	UPROPERTY()
	TArray<TObjectPtr<UObject>> GraphReferencedObjects;

	// This is a list of all referenced soft objects in the graph shared data
	// Used to serialize the soft objects correctly as we dont use FArchiveUObject
	UPROPERTY()
	TArray<FSoftObjectPath> GraphReferencedSoftObjects;

	// The entry point that this graph defaults to using
	UPROPERTY(EditAnywhere, Category = "Graph")
	FName DefaultEntryPoint = TEXT("Root");

	// Default state for this graph
	UPROPERTY()
	FAnimNextGraphState DefaultState;

#if WITH_EDITORONLY_DATA
	// This buffer holds the output of the FTraitWriter post compilation
	// We serialize it manually and it is discarded at runtime
	TArray<uint8> SharedDataArchiveBuffer;
#endif
};

#undef UE_API
