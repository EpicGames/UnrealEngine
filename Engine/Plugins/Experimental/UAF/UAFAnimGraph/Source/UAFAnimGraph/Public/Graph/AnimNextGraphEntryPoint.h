// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/AnimNextGraphState.h"
#include "TraitCore/EntryPointHandle.h"
#include "AnimNextGraphEntryPoint.generated.h"

USTRUCT()
struct FAnimNextGraphEntryPoint
{
	GENERATED_BODY()

	// The name of the entry point
	UPROPERTY()
	FName EntryPointName;

	// This is a handle to the root trait for a graph
	UPROPERTY()
	FAnimNextEntryPointHandle RootTraitHandle;
};

