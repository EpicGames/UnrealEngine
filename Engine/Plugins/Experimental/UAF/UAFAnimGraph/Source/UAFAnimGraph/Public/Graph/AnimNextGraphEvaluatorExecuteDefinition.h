// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextGraphEvaluatorExecuteDefinition.generated.h"

// Represents an argument to a FRigUnit_AnimNextGraphEvaluator entry point
USTRUCT()
struct FAnimNextGraphEvaluatorExecuteArgument
{
	GENERATED_BODY();

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString CPPType;
};

// Represents an entry point of FRigUnit_AnimNextGraphEvaluator
USTRUCT()
struct FAnimNextGraphEvaluatorExecuteDefinition
{
	GENERATED_BODY();

	UPROPERTY()
	uint32 Hash = 0;

	UPROPERTY()
	FString MethodName;

	UPROPERTY()
	TArray<FAnimNextGraphEvaluatorExecuteArgument> Arguments;
};