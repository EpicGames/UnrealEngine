// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphTaskContext.h"
#include "InstanceTask.h"
#include "UAFAssetInstance.h"
#include "Containers/TransactionallySafeMpscQueue.h"
#include "HAL/CriticalSection.h"
#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitEventList.h"
#include "TraitCore/TraitPtr.h"
#include "RigVMCore/RigVMExecuteContext.h"

#include "AnimNextGraphInstance.generated.h"

#define UE_API UAFANIMGRAPH_API

class FReferenceCollector;

struct FRigUnit_AnimNextGraphEvaluator;
class UUAFAnimGraph;
class UUAFSystem;
struct FAnimNextModuleInstance;
class FRigVMTraitScope;
struct FRigVMExtendedExecuteContext;
struct FAnimNextExecuteContext;
struct FAnimNextModuleInjectionComponent;
struct FRigUnit_AnimNextRunAnimationGraph_v1;
struct FRigUnit_AnimNextRunAnimationGraph_v2;
struct FAnimNextStateTreeRigVMConditionBase;
struct FAnimNextStateTreeRigVMTaskBase;
struct FAnimNextFactoryParams;
struct FUAFWeakSystemReference;

namespace UE::UAF
{
	struct FTraitBinding;
	struct FExecutionContext;
	struct FGraphInstanceComponent;
	struct FLatentPropertyHandle;
	struct FTraitStackBinding;
	struct FPlayAnimSlotTrait;
	struct FBlendStackCoreTrait;
	struct FInjectionRequest;
	struct FInjectionSiteTrait;
	struct FStateTreeTrait;
	struct FCachedBindingInfo;
	struct FInjectionUtils;
	struct FBlendSpacePlayerTrait;
	struct FAnimGraphReference;
	struct FWeakAnimGraphReference;
}

// Represents an instance of an AnimNext graph
// This struct uses UE reflection because we wish for the GC to keep the graph
// alive while we own a reference to it. It is not intended to be serialized on disk with a live instance.
USTRUCT()
struct FAnimNextGraphInstance : public FUAFAssetInstance
#if CPP
	, public TSharedFromThis<FAnimNextGraphInstance>
#endif
{
	GENERATED_BODY()

	// Creates an empty graph instance that doesn't reference anything
	UE_API FAnimNextGraphInstance();

	// No copying, no moving
	FAnimNextGraphInstance(const FAnimNextGraphInstance&) = delete;
	FAnimNextGraphInstance& operator=(const FAnimNextGraphInstance&) = delete;

	// If the graph instance is allocated, we release it during destruction
	UE_API ~FAnimNextGraphInstance();

	// Releases the graph instance and frees all corresponding memory
	UE_API void Release();

	// Returns true if we have a live graph instance, false otherwise
	UE_API bool IsValid() const;

	// Returns the animation graph used by this instance or nullptr if the instance is invalid
	UE_API const UUAFAnimGraph* GetAnimationGraph() const;

	// Returns the entry point in Graph that this instance corresponds to 
	UE_API FName GetEntryPoint() const;
	
	// Returns a weak handle to the root trait instance
	UE_API UE::UAF::FWeakTraitPtr GetGraphRootPtr() const;

	// Returns the module instance that owns us or nullptr if we are invalid
	UE_API FAnimNextModuleInstance* GetModuleInstance() const;

	// Returns the system instance that owns us as a weak reference. Can be invalid if we are not owned by a system.
	UE_API FUAFWeakSystemReference GetModuleInstanceReference() const;

	// Returns the parent graph instance that owns us or nullptr for the root graph instance or if we are invalid
	UE_API FAnimNextGraphInstance* GetParentGraphInstance() const;

	// Returns the root graph instance that owns us and the components or nullptr if we are invalid
	UE_API FAnimNextGraphInstance* GetRootGraphInstance() const;

	// Check to see if this instance data matches the provided animation graph
	UE_API bool UsesAnimationGraph(const UUAFAnimGraph* InAnimationGraph) const;

	// Check to see if this instance data matches the provided graph entry point
	UE_API bool UsesEntryPoint(FName InEntryPoint) const;
	
	// Returns whether or not this graph instance is the root graph instance or false otherwise
	UE_API bool IsRoot() const;
	
	// Returns whether or not this graph instance has updated at least once
	UE_API bool HasUpdated() const;

	// Called each time the graph updates to mark the instance as updated
	UE_API void MarkAsUpdated();

private:
	// Executes a list of latent RigVM pins and writes the result into the destination pointer (latent handle offsets are using the destination as base)
	// When frozen, latent handles that can freeze are skipped, all others will execute
	UE_API void ExecuteLatentPins(const TConstArrayView<UE::UAF::FLatentPropertyHandle>& LatentHandles, void* DestinationBasePtr, float DeltaTime, bool bIsFrozen, bool bJustBecameRelevant);

	// When a group of latent pins do not require RigVM to operate, this copies the values directly from variables rather than calling ExecuteLatentPins.
	// When frozen, latent handles that can freeze are skipped, all others will execute
	void CopyVariablesToLatentPins(const TConstArrayView<UE::UAF::FLatentPropertyHandle>& LatentHandles, void* DestinationBasePtr, bool bIsFrozen, bool bJustBecameRelevant);

	// Performs allocation of internal variable data structures
	void AllocateVariablesInternal(const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides, TWeakPtr<FAnimNextGraphInstance> InParentGraphInstance);

	// Performs allocation of graph root ptr
	void AllocateGraphInternal();

#if WITH_EDITORONLY_DATA
	// During graph compilation, if we have existing graph instances, we freeze them by releasing their memory before thawing them
	// Freezing is a partial release of resources that retains the necessary information to re-create things safely
	UE_API void Freeze();

	// During graph compilation, once compilation is done we thaw existing graph instances to reallocate their memory
	UE_API void Thaw();
#endif

	// Registers the provided trait with the GC system
	// Once registered, IGarbageCollection::AddReferencedObjects will be called on it during GC
	void RegisterWithGC(const UE::UAF::FWeakTraitPtr& InTraitPtr);
	
	// Unregisters the provided trait from the GC system
	void UnregisterWithGC(const UE::UAF::FWeakTraitPtr& InTraitPtr);

	// Runs a task or queues it until AllocateVariablesInternal
	void RunOrQueueTask(UE::UAF::FUniqueAnimGraphTask&& InTaskFunction);

private:
	// GC handler. This is nested to avoid having to bubble the polymorphism it introduces up to super-structs
	struct FGCHandler : public FGCObject
	{
		explicit FGCHandler(FAnimNextGraphInstance& InOwner)
			: Owner(InOwner)
		{}

		// FGCObject implementation
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;

		// Owner instance
		FAnimNextGraphInstance& Owner;

		// List of trait handles that contain UObject references and implement IGarbageCollection
		TArray<UE::UAF::FWeakTraitPtr> TraitsWithReferences;
	} GCHandler;

	// The entry point in Graph that this instance corresponds to 
	FName EntryPoint;

	// Hard reference to the graph instance data, we own it
	UE::UAF::FTraitPtr GraphInstancePtr;

	// The module instance that owns the root, us and the components
	TWeakPtr<FAnimNextModuleInstance> ModuleInstance;

	// The root graph instance that owns us and the components
	FAnimNextGraphInstance* RootGraphInstance = nullptr;

	enum EAllocationState
	{
		// Default-initialized graph instance
		Empty,

		// Has a reference to its graph, but has no internal other structures allocated
		PreAllocated,

		// Has internal variables & components allocated
		VariablesAllocated,

		// Has all internal structures allocated
		Allocated,
	} AllocationState = EAllocationState::Empty;

	// Whether or not this graph has updated once
	bool bHasUpdatedOnce : 1 = false;

	friend UUAFAnimGraph;			// The graph is the one that allocates instances
	friend FRigUnit_AnimNextGraphEvaluator;	// We evaluate the instance
	friend UE::UAF::FExecutionContext;
	friend UE::UAF::FTraitStackBinding;
	friend UE::UAF::FPlayAnimSlotTrait;
	friend UE::UAF::FBlendStackCoreTrait;
	friend UE::UAF::FInjectionRequest;
	friend UE::UAF::FInjectionSiteTrait;
	friend UE::UAF::FStateTreeTrait;	// Temp - remove: Needs to access mutable variables to copy-in data OnBecomeRelevant
	friend FAnimNextModuleInjectionComponent;
	friend FRigUnit_AnimNextRunAnimationGraph_v1;
	friend FRigUnit_AnimNextRunAnimationGraph_v2;
	friend UE::UAF::FInjectionUtils;
	friend FAnimNextStateTreeRigVMConditionBase;	// Temp - remove: Needs to access mutable variables to copy-in data for function shim
	friend FAnimNextStateTreeRigVMTaskBase;	// Temp - remove: Needs to access mutable variables to copy-in data for function shim
	friend UE::UAF::FBlendSpacePlayerTrait;
	friend UE::UAF::FAnimGraphReference;
	friend UE::UAF::FWeakAnimGraphReference;
};

template<>
struct TStructOpsTypeTraits<FAnimNextGraphInstance> : public TStructOpsTypeTraitsBase2<FAnimNextGraphInstance>
{
	enum
	{
		WithCopy = false,
	};
};

#undef UE_API
