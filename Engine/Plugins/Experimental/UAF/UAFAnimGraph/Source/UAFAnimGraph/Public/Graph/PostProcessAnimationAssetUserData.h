// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"

#include "PostProcessAnimationAssetUserData.generated.h"

/** Asset user data used to store post-process animation data for a skeletal mesh asset. */
UCLASS(MinimalAPI, meta = (DisplayName = "Post-process Animation User Data"))
class UPostProcessAnimationUserAssetData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Animation)
	TObjectPtr<UObject> AnimationAsset;
};