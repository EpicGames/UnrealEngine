// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EvaluationVM/EvaluationProgram.h"
#include "RewindDebugger/UAFTrace.h"

#if UAF_TRACE_ENABLED

struct FAnimNextGraphInstance;

namespace UE::UAF
{
	struct FEvaluationProgram;
	
	UAFANIMGRAPH_API void TraceGraphInstances(const FAnimNextGraphInstance& RootGraph);
	UAFANIMGRAPH_API void TraceEvaluationProgram(const FEvaluationProgram& Program, const FAnimNextGraphInstance& RootGraph);
	UAFANIMGRAPH_API void TraceEvaluationProgram(const FEvaluationProgram& Program, const FUAFAssetInstance& AssetInstance, const UObject* OuterObject);
}

#define TRACE_UAF_GRAPHINSTANCES(RootGraph) UE::UAF::TraceGraphInstances(RootGraph);
#define TRACE_UAF_EVALUATIONPROGRAM(Program, RootGraph) UE::UAF::TraceEvaluationProgram(Program, RootGraph);
#define TRACE_UAF_EVALUATIONPROGRAM_WITHOWNER(Program, AssetInstance, Object) UE::UAF::TraceEvaluationProgram(Program, AssetInstance, Object);

#else

#define TRACE_UAF_GRAPHINSTANCES(RootGraph)
#define TRACE_UAF_EVALUATIONPROGRAM(Program, RootGraph)
#define TRACE_UAF_EVALUATIONPROGRAM_WITHOWNER(Program, AssetInstance, Object)

#endif
