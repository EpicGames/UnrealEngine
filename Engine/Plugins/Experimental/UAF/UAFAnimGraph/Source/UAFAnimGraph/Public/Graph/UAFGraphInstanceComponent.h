// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstanceComponent.h"
#include "UAFGraphInstanceComponent.generated.h"

#define UE_API UAFANIMGRAPH_API

struct FAnimNextTraitEvent;
struct FAnimNextGraphInstance;

namespace UE::UAF
{
	struct FExecutionContext;
}

/**
 * FGraphInstanceComponent
 *
 * A graph instance component is attached and owned by a graph instance.
 * It persists as long as it is needed.
 */
USTRUCT(meta=(Hidden, Abstract))
struct FUAFGraphInstanceComponent : public FUAFAssetInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	using ContainerType = FAnimNextGraphInstance;

	FUAFGraphInstanceComponent() = default;

	virtual ~FUAFGraphInstanceComponent() override = default;

	// Returns the owning graph instance this component lives on
	UE_API FAnimNextGraphInstance& GetGraphInstance();

	// Returns the owning graph instance this component lives on
	UE_API const FAnimNextGraphInstance& GetGraphInstance() const;
	
	// Called before the update traversal begins, before any node has been visited
	// Note that PreUpdate won't be called if a component is created during the update traversal until the next update
	// The execution context provided is bound to the graph root and can be bound to anything the component wishes
	virtual void PreUpdate(UE::UAF::FExecutionContext& Context) {}

	// Called after the update traversal completes, after every node has been visited
	// The execution context provided is bound to the graph root and can be bound to anything the component wishes
	virtual void PostUpdate(UE::UAF::FExecutionContext& Context) {}

	// Called before PreUpdate with input events and before PostUpdate with output events
	virtual void OnTraitEvent(UE::UAF::FExecutionContext& Context, FAnimNextTraitEvent& Event) {}
};

#undef UE_API