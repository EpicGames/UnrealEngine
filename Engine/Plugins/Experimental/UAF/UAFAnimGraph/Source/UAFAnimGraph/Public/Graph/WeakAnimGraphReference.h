// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphTaskContext.h"
#include "EngineDefines.h"
#include "InstanceTask.h"
#include "Templates/SharedPointer.h"

#define UE_API UAFANIMGRAPH_API

class UUAFAnimGraph;
struct FAnimNextGraphInstance;
class FReferenceCollector;

namespace UE::UAF
{
struct FTraitBinding;
struct FExecutionContext;
struct FWeakTraitPtr;
struct FAnimGraphReference;
struct FInputValueTrait;
struct FLayerStackTrait;
}

namespace UE::UAF
{

// A non-owning reference to a UAF graph.
struct FWeakAnimGraphReference
{
	// Default-constructable only because it needs to be embedded in UObjects/UStructs
	FWeakAnimGraphReference() = default;

	// Construct a non-owning reference to a graph instance.
	// Can be called on any thread
	UE_API explicit FWeakAnimGraphReference(const FAnimGraphReference& InAnimGraphReference);

	// Construct a non-owning reference to a graph instance.
	// Can be called on any thread
	UE_API explicit FWeakAnimGraphReference(const TSharedPtr<FAnimNextGraphInstance>& InAnimGraphInstance);

	// Reset this handle to invalid - gives up ownership
	// Can be called on any thread
	UE_API void Reset();

	// Check if this handle is valid
	// Can be called on any thread.
	UE_API bool IsValid() const;

	// Get the anim graph that this handle references
	// Can be called on any thread.
	[[nodiscard]] UE_API const UUAFAnimGraph* GetAnimGraph() const;

	// Queue task to execute on the instance
	// Can be called on any thread.
	// @param	InTaskFunction		The function to run
	UE_API void QueueTask(FUniqueAnimGraphTask&& InTaskFunction) const;

	// Sets the debug name for the instance, used for tracing etc.
	UE_API void SetDebugName(FName InName) const;

	// GC support
	UE_API void AddReferencedObjects(FReferenceCollector& InCollector) const;

private:
	friend FLayerStackTrait;
	friend FInputValueTrait;

	// Gets or allocate & binds the graph instance
	// Fully allocates the graph if it is only preallocated
	// Can be called on any thread.
	// @param    InBinding    The current binding (this must be called during a graph execution)
	[[nodiscard]] UE_API TSharedPtr<FAnimNextGraphInstance> GetOrAllocate(const FExecutionContext& InContext, const FTraitBinding& InBinding) const;

private:
	// Pointer to the instance
	TWeakPtr<FAnimNextGraphInstance> Ptr;
};

}

#undef UE_API