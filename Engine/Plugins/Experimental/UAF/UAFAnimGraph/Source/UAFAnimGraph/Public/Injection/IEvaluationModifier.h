// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::UAF
{
	struct FEvaluateTraversalContext;
};

namespace UE::UAF
{
	// Override point allowing injection sites to take over evaluation and inject their own tasks
	struct IEvaluationModifier
	{
		virtual ~IEvaluationModifier() = default;

		// Called before an injection site trait's children are evaluated
		virtual void PreEvaluate(FEvaluateTraversalContext& Context) const = 0;

		// Called after an injection site trait's children are evaluated
		virtual void PostEvaluate(FEvaluateTraversalContext& Context) const = 0;
	};
}
