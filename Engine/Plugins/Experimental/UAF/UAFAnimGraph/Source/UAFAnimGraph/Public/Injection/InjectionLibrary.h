// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "InjectionLibrary.generated.h"

class UUAFComponent;

UCLASS()
class UUAFInjectionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Return true if the running UAF System has an active montage in the given slot. A UAnimMontage that is playing in the slot and blending out is not determined to be "active".
	UFUNCTION(BlueprintPure, Category = "UAF|Injection", meta=(ScriptMethod))
	static UAFANIMGRAPH_API bool IsSlotActive(const UUAFComponent* UAFComponent, FName SlotNodeName);
};


