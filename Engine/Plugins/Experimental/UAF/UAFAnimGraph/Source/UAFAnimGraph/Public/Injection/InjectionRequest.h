// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AlphaBlend.h"
#include "Factory/AnimNextFactoryParams.h"
#include "UAFAssetInstance.h"
#include "InstanceTask.h"
#include "TraitCore/TraitEvent.h"
#include "TraitInterfaces/ITimeline.h"
#include "Injection/InjectionStatus.h"
#include "Injection/InjectionSite.h"
#include "Containers/SpscQueue.h"
#include "Module/UAFWeakSystemReference.h"

#include "InjectionRequest.generated.h"

struct FUAFWeakSystemReference;
class FReferenceCollector;
struct FAnimNextModuleInjectionComponent;
struct FAnimNextGraphInstance;
class UInjectionCallbackProxy;
class UWorld;

namespace UE::UAF
{
	struct FInstanceTaskContext;
	struct IEvaluationModifier;
	struct FInjection_InjectEvent;
	struct FInjection_StatusUpdateEvent;
	struct FInjection_TimelineUpdateEvent;
	struct FInjectionUtils;
	struct FPlayAnimSlotTrait;
	struct FInjectionSiteTrait;
	struct FInjectionRequest;
	struct FInjectionRequestTracker;
}

UENUM()
enum class EAnimNextInjectionBlendMode : uint8
{
	// Uses standard weight based blend
	Standard,

	// Uses inertialization. Requires an inertialization trait somewhere earlier in the graph.
	Inertialization,
};

/**
 * Injection Blend Settings
 *
 * Encapsulates the blend settings used by injection requests.
 */
USTRUCT(BlueprintType)
struct FAnimNextInjectionBlendSettings
{
	GENERATED_BODY()

	/** Blend Profile to use for this blend */
	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta = (DisplayAfter = "Blend"))
	//TObjectPtr<UBlendProfile> BlendProfile;

	/** AlphaBlend options (time, curve, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta = (DisplayAfter = "BlendMode"))
	FAlphaBlendArgs Blend;

	/** Type of blend mode (Standard vs Inertial) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend")
	EAnimNextInjectionBlendMode BlendMode = EAnimNextInjectionBlendMode::Standard;
};

// What to do when performing an injection
UENUM()
enum class EAnimNextInjectionType : uint8
{
	// Inject an object to be instantiated via factory at the injection site. Optionally this can bind to an external module.
	InjectObject,

	// Apply an evaluation modifier at the specified injection site. Does not affect the currently-provided object.
	EvaluationModifier,
};

// Lifetime behavior for injection requests
UENUM()
enum class EAnimNextInjectionLifetimeType : uint8
{
	// Automatically un-inject this request when the timeline of the injection request expires.
	// Looping/infinite timelines will never auto-expire and must be uninjected manually.
	Auto,

	// Injection request persists and must be un-injected manually.
	// Blending out to the source animation at the injection site does not occur.
	// Useful when you want to just push a chain of animations to an injection site, never seeing the source/passthrough pose.
	ForcePersistent,
};

/**
 * Injection Request Arguments
 *
 * Encapsulates the parameters required to initiate an injection request.
 */
USTRUCT()
struct FAnimNextInjectionRequestArgs
{
	GENERATED_BODY()

private:
	friend UE::UAF::FInjectionRequest;
	friend UE::UAF::FInjectionSiteTrait;
	friend UE::UAF::FPlayAnimSlotTrait;
	friend UE::UAF::FInjectionRequestTracker;
	friend UE::UAF::FInjectionUtils;
	friend FAnimNextModuleInjectionComponent;
	friend UInjectionCallbackProxy;

	// The injection site to target with this request.
	UPROPERTY()
	FAnimNextInjectionSite Site;

	// The blend settings to use when blending in
	UPROPERTY()
	FAnimNextInjectionBlendSettings BlendInSettings;

	// The blend settings to use when blending out (if not interrupted)
	UPROPERTY()
	FAnimNextInjectionBlendSettings BlendOutSettings;

	// Object to inject. The animation graph to be instantiated for this request will be chosen via a factory
	UPROPERTY()
	TObjectPtr<UObject> Object;

	// Evaluation modifier to apply at the injection site. Shared ownership, references can persist in worker threads until un-injected.
	TSharedPtr<UE::UAF::IEvaluationModifier> EvaluationModifier;

	UPROPERTY()
	EAnimNextInjectionType Type = EAnimNextInjectionType::InjectObject;

	// Lifetime behavior for the request
	UPROPERTY()
	EAnimNextInjectionLifetimeType LifetimeType = EAnimNextInjectionLifetimeType::Auto;

	// Whether or not the request should track the timeline progress
	UPROPERTY()
	bool bTrackTimelineProgress = false;

	// Set of assets & structs that describe the required variables for factory-generating injected graphs
	UPROPERTY()
	FAnimNextFactoryParams FactoryParams;
};

namespace UE::UAF
{
	// Create a namespaced aliases to simplify usage
	using EInjectionBlendMode = EAnimNextInjectionBlendMode;
	using FInjectionBlendSettings = FAnimNextInjectionBlendSettings;
	using FInjectionRequestArgs = FAnimNextInjectionRequestArgs;

	struct FInjectionRequest;

	DECLARE_DELEGATE_OneParam(FAnimNextOnInjectionStarted, const FInjectionRequest&)
	DECLARE_DELEGATE_OneParam(FAnimNextOnInjectionCompleted, const FInjectionRequest&)
	DECLARE_DELEGATE_OneParam(FAnimNextOnInjectionInterrupted, const FInjectionRequest&)
	DECLARE_DELEGATE_OneParam(FAnimNextOnInjectionBlendingOut, const FInjectionRequest&)

	// Delegates called for various lifetime events
	struct FInjectionLifetimeEvents
	{
		// Callback called when the request starts playing (status transitions from pending to playing)
		FAnimNextOnInjectionStarted OnStarted;

		// Callback called when the request completes (status transitions from playing to completed)
		FAnimNextOnInjectionCompleted OnCompleted;

		// Callback called when the request is interrupted (either by calling Stop on it or by another request)
		FAnimNextOnInjectionInterrupted OnInterrupted;

		// Callback called when the request starts blending out (if it wasn't interrupted)
		FAnimNextOnInjectionBlendingOut OnBlendingOut;
	};

	/**
	 * Injection Request
	 *
	 * Instances of this class represent individual requests to the injection system.
	 * They are allocated as shared pointers and ownership is split between gameplay (until
	 * it no longer cares about a particular request) and the injection site that hosts it
	 * (until the request completes).
	 * 
	 * Use MakeInjectionRequest(...) to construct instances of this type.
	 */
	struct FInjectionRequest : public TSharedFromThis<FInjectionRequest, ESPMode::ThreadSafe>
	{
		// Returns the arguments this request is using
		UAFANIMGRAPH_API const FInjectionRequestArgs& GetArgs() const;

		// Returns the lifetime delegates this request is using
		UAFANIMGRAPH_API const FInjectionLifetimeEvents& GetLifetimeEvents() const;

		// Returns the request status
		UAFANIMGRAPH_API EInjectionStatus GetStatus() const;

		// Returns the current timeline state (make sure to enable FInjectionRequestArgs::bTrackTimelineProgress to use this)
		UAFANIMGRAPH_API const FTimelineState& GetTimelineState() const;

		// Returns whether or not this request has expired
		UAFANIMGRAPH_API bool HasExpired() const;

		// Returns whether or not this request has completed (might have been interrupted)
		UAFANIMGRAPH_API bool HasCompleted() const;

		// Returns whether or not this request is playing (might be blending out or interrupted)
		UAFANIMGRAPH_API bool IsPlaying() const;

		// Returns whether or not this request is blending out
		UAFANIMGRAPH_API bool IsBlendingOut() const;

		// Returns whether or not this request was interrupted (by Stop or by another request)
		UAFANIMGRAPH_API bool WasInterrupted() const;

		UAFANIMGRAPH_API void ExternalAddReferencedObjects(FReferenceCollector& Collector);

		// Set a variable on the running injected instance.
		// @param    InVariable         The variable to set
		// @param    InValue            The value to set the variable to
		template<typename ValueType>
		void SetVariable(const FAnimNextVariableReference& InVariable, const ValueType& InValue)
		{
			QueueTask([InVariable, InValue](const FInstanceTaskContext& InContext)
			{
				InContext.SetVariable(InVariable, InValue);
			});
		}

		// Access a variable on the running injected instance for modification. Can avoid excess copies for larger structures/arrays.
		// @param    InVariable         The variable to set
		// @param    InFunction         The function used to modify the variable. Note: called back at some later time on a worker thread
		template<typename ValueType>
		void AccessVariable(const FAnimNextVariableReference& InVariable, TUniqueFunction<void(ValueType&)> InFunction)
		{
			QueueTask([InVariable, Function = MoveTemp(InFunction)](const FInstanceTaskContext& InContext)
			{
				InContext.AccessVariable(InVariable, Function);
			});
		}

		// Access the memory of the shared variable struct directly on the running injected instance.
		// @param	InFunction			Function called with a reference to the variable's struct. Note: called back at some later time on a worker thread
		template<typename StructType>
		void AccessVariablesStruct(TUniqueFunction<void(StructType&)> InFunction) const
		{
			QueueTask([Function = MoveTemp(InFunction)](const FInstanceTaskContext& InContext)
			{
				InContext.AccessVariablesStruct(Function);
			});
		}

		// Queues a task to run on the injected instance prior to execution. Use this to set variables etc.
		UAFANIMGRAPH_API void QueueTask(FUniqueInstanceTask&& InTask);

	private:
		// Sends this request to the specified host and it will attempt to play with the requested arguments
		bool Inject(FInjectionRequestArgs&& InRequestArgs, FInjectionLifetimeEvents&& InLifetimeEvents, UObject* InHost, FUAFWeakSystemReference InReference);

		// Interrupts this request and request that we transition to the source input on the playing injection site
		void Uninject();

		// Returns the arguments this request is using
		UAFANIMGRAPH_API FInjectionRequestArgs& GetMutableArgs();

		// Returns the lifetime delegates this request is using
		UAFANIMGRAPH_API FInjectionLifetimeEvents& GetMutableLifetimeEvents();

		void OnStatusUpdate(EInjectionStatus NewStatus);

		void OnTimelineUpdate(const FTimelineState& NewTimelineState);

		// GC API
		void AddReferencedObjects(FReferenceCollector& Collector);

		// Validate this set of args is set up correctly for injection
		static bool ValidateArgs(const FInjectionRequestArgs& InRequestArgs);

		// Flush any pending tasks, with the initialize task on the request args getting run once first
		void FlushTasks(const FInstanceTaskContext& InContext);

		// The request arguments
		FInjectionRequestArgs RequestArgs;

		// Callbacks for lifetime events 
		FInjectionLifetimeEvents LifetimeEvents;

		// The object we are playing on
		TWeakObjectPtr<UObject> WeakHost;

		// Reference to the system instance
		FUAFWeakSystemReference Reference;

		// The injection event if we have injected already
		TWeakPtr<FAnimNextTraitEvent> InjectionEvent;

		// Queue of tasks to be executed next time this injection updates
		TSpscQueue<FUniqueInstanceTask> TaskQueue;

		// The current request status
		EInjectionStatus Status = EInjectionStatus::None;

		// The current timeline state
		FTimelineState TimelineState;

		friend FInjection_InjectEvent;
		friend FInjection_StatusUpdateEvent;
		friend FInjection_TimelineUpdateEvent;
		friend FInjectionUtils;
		friend FPlayAnimSlotTrait;
		friend FInjectionSiteTrait;
		friend ::FAnimNextModuleInjectionComponent;
	};

	// Create a shared pointer alias for injection requests
	using FInjectionRequestPtr = TSharedPtr<FInjectionRequest, ESPMode::ThreadSafe>;

	// Constructs a injection request object
	inline FInjectionRequestPtr MakeInjectionRequest()
	{
		return MakeShared<FInjectionRequest>();
	}
}
