// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "Variables/AnimNextVariableReference.h"

#include "InjectionSite.generated.h"

// Specifies the desired injection site for an injection request.
// Optionally allows for fallback to the Default Injection Site specified by the module
//     if DesiredSiteName is none, or does not exist in the graph
USTRUCT()
struct FAnimNextInjectionSite
{
	GENERATED_BODY()

	FAnimNextInjectionSite()
		: bUseModuleFallback(true)
	{
	}

	explicit FAnimNextInjectionSite(const FAnimNextVariableReference& InDesiredSite)
		: DesiredSite(InDesiredSite)
		, bUseModuleFallback(!InDesiredSite.IsNone())
	{
	}

#if WITH_EDITORONLY_DATA
	// The name of the site to inject into
	UPROPERTY()
	FName DesiredSiteName_DEPRECATED;
#endif

	UPROPERTY()
	FAnimNextVariableReference DesiredSite;

	// Flag specifying whether the request can (or should) fallback to the Default Injection Site
	// specified in the module if DesiredSiteName was not found
	UPROPERTY()
	uint8 bUseModuleFallback : 1;

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
#endif
};

template<>
struct TStructOpsTypeTraits<FAnimNextInjectionSite> : public TStructOpsTypeTraitsBase2<FAnimNextInjectionSite>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
#endif
};

namespace UE::UAF
{
	using FInjectionSite = FAnimNextInjectionSite;
}