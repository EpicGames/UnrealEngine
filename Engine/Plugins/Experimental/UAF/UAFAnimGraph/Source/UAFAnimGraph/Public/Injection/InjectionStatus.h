// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InjectionStatus.generated.h"

/**
 * Injection Request Status
 */
UENUM()
enum class EAnimNextInjectionStatus : uint8
{
	//////////////////////////////////////////////////////////////////////////
	// Primary status is held in the bottom 5 bits
	// These are mutually exclusive

	// No current status
	None			= 0x00,

	// Animation request has been queue but hasn't been processed yet
	Pending			= 0x01,

	// Animation is playing (may be blending in or out, may be interrupted)
	Playing			= 0x02,

	// Animation has finished playing (may have the interrupted flag to signal that it didn't stop on its own)
	Completed		= 0x04,

	// Animation request expired without playing
	Expired			= 0x10,


	//////////////////////////////////////////////////////////////////////////
	// Additive status is held in the top 3 bits

	// Animation was interrupted by another animation request (or a stop request)
	Interrupted		= 0x40,

	// Animation is blending out
	BlendingOut		= 0x80,
};

ENUM_CLASS_FLAGS(EAnimNextInjectionStatus);

namespace UE::UAF
{
	// Create a namespaced alias to simplify usage
	using EInjectionStatus = EAnimNextInjectionStatus;
}
