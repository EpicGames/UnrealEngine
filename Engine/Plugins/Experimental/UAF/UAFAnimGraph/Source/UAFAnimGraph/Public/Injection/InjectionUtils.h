// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InjectionRequest.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Factory/AnimAssetParams.h"
#include "Injection/InjectionSite.h"

struct FUAFAssetInstance;
class UUAFComponent;

namespace UE::UAF
{
	struct IEvaluationModifier;
}

namespace UE::UAF
{
	// Optional arguments to PlayAnim
	struct FPlayAnimArgs
	{
		FAnimNextFactoryParams FactoryParams;
		float PlayRate = 1.0f;
		float StartPosition = 0.0f;
		EAnimAssetLoopMode LoopMode = EAnimAssetLoopMode::Auto;
		EAnimNextInjectionLifetimeType LifetimeType = EAnimNextInjectionLifetimeType::Auto;
	};

	// A collection of wrapper functions for common injection operations
	struct FInjectionUtils
	{
		// Injects an animation sequence into a running AnimNext module
		// @param InComponent          The UAnimNextComponent that hosts the module being played on
		// @param InHost               The object that hosts the module (e.g. UAnimNextComponent) being played on
		// @param InSystemReference    The system reference being injected into
		// @param InSite               The injection site to 'play' the animation sequence on
		// @param InAnimSequence       The animation sequence to play
		// @param InArgs               Optional args to use
		// @param InBlendInSettings    The blend in settings
		// @param InBlendOutSettings   The blend in settings
		// @param InLifetimeEvents     Delegates called for various request lifetime events
		// @return the new injection request for tracking, or an invalid request if injection failed. This can be discarded for fire-and-forget style behavior.
		UAFANIMGRAPH_API static FInjectionRequestPtr PlayAnim(
			UUAFComponent* InComponent,
			const FInjectionSite& InSite,
			UAnimSequence* InAnimSequence,
			FPlayAnimArgs&& InArgs = FPlayAnimArgs(),
			const FInjectionBlendSettings& InBlendInSettings = FInjectionBlendSettings(),
			const FInjectionBlendSettings& InBlendOutSettings = FInjectionBlendSettings(),
			FInjectionLifetimeEvents&& InLifetimeEvents = FInjectionLifetimeEvents());

		// @see PlayAnim
		UAFANIMGRAPH_API static FInjectionRequestPtr PlayAnimHandle(
			UObject* InHost,
			FUAFWeakSystemReference InSystemReference,
			const FInjectionSite& InSite,
			UAnimSequence* InAnimSequence,
			FPlayAnimArgs&& InArgs = FPlayAnimArgs(),
			const FInjectionBlendSettings& InBlendInSettings = FInjectionBlendSettings(),
			const FInjectionBlendSettings& InBlendOutSettings = FInjectionBlendSettings(),
			FInjectionLifetimeEvents&& InLifetimeEvents = FInjectionLifetimeEvents());

		// Injects an asset into a running AnimNext module
		// @param InComponent          The UAnimNextComponent that hosts the module being played on
		// @param InHost               The object that hosts the module (e.g. UAnimNextComponent) being played on
		// @param InSystemReference    The system reference being injected into
		// @param InSite               The injection site to 'play' the asset on
		// @param InAsset              The asset to 'play'
		// @param InFactoryParams      The params used to manufacture a graph to run the injected object 
		// @param InBlendInSettings    The blend in settings
		// @param InBlendOutSettings   The blend in settings
		// @param InLifetimeEvents     Delegates called for various request lifetime events
		// @return the new injection request for tracking, or an invalid request if injection failed. This can be discarded for fire-and-forget style behavior.
		UAFANIMGRAPH_API static FInjectionRequestPtr InjectAsset(
			UUAFComponent* InComponent,
			const FInjectionSite& InSite,
			UObject* InAsset,
			FAnimNextFactoryParams&& InFactoryParams = FAnimNextFactoryParams(),
			const FInjectionBlendSettings& InBlendInSettings = FInjectionBlendSettings(),
			const FInjectionBlendSettings& InBlendOutSettings = FInjectionBlendSettings(),
			FInjectionLifetimeEvents&& InLifetimeEvents = FInjectionLifetimeEvents());

		// @see InjectAsset
		UAFANIMGRAPH_API static FInjectionRequestPtr InjectAsset(
			UObject* InHost,
			FUAFWeakSystemReference InSystemReference,
			const FInjectionSite& InSite,
			UObject* InAsset,
			FAnimNextFactoryParams&& InFactoryParams = FAnimNextFactoryParams(),
			const FInjectionBlendSettings& InBlendInSettings = FInjectionBlendSettings(),
			const FInjectionBlendSettings& InBlendOutSettings = FInjectionBlendSettings(),
			FInjectionLifetimeEvents&& InLifetimeEvents = FInjectionLifetimeEvents());

		// Inject an evaluation modifier into a running AnimNext module to control animation evaluation at a low level.
		// Does not require that injection sites are publicly visible.
		// @param InComponent          The UAnimNextComponent that hosts the module being injected into
		// @param InHost               The object that hosts the module (e.g. UAnimNextComponent) being injected into
		// @param InSystemReference    The system reference being injected into
		// @param InEvaluationModifier The evaluation modifier to inject
		// @param InSite               The injection site to use. If this is NAME_None, then the first site will be used.
		// @return the new injection request for tracking, or an invalid request if injection failed. This can be discarded for fire-and-forget style behavior.
		UAFANIMGRAPH_API static FInjectionRequestPtr InjectEvaluationModifier(
			UUAFComponent* InComponent,
			const TSharedRef<IEvaluationModifier>& InEvaluationModifier,
			const FInjectionSite& InSite = FInjectionSite());

		// @see InjectEvaluationModifier
		UAFANIMGRAPH_API static FInjectionRequestPtr InjectEvaluationModifier(
			UObject* InHost,
			FUAFWeakSystemReference InSystemReference,
			const TSharedRef<IEvaluationModifier>& InEvaluationModifier,
			const FInjectionSite& InSite = FInjectionSite());

		// Inject into a running AnimNext module (raw custom args)
		// @param InComponent          The UAnimNextComponent that hosts the module being injected into
		// @param InHost               The object that hosts the module (e.g. UAnimNextComponent) being injected into
		// @param InSystemReference    The system reference being injected into
		// @param InArgs               Arguments to the injection request
		// @param InLifetimeEvents     Delegates called for various request lifetime events
		// @return the new injection request for tracking, or an invalid request if injection failed. This can be discarded for fire-and-forget style behavior.
		UAFANIMGRAPH_API static FInjectionRequestPtr Inject(
			UUAFComponent* InComponent,
			FInjectionRequestArgs&& InArgs,
			FInjectionLifetimeEvents&& InLifetimeEvents = FInjectionLifetimeEvents());

		// @see Inject
		UAFANIMGRAPH_API static FInjectionRequestPtr Inject(
			UObject* InHost,
			FUAFWeakSystemReference InSystemReference,
			FInjectionRequestArgs&& InArgs,
			FInjectionLifetimeEvents&& InLifetimeEvents = FInjectionLifetimeEvents());

		// Uninject a previously injected request
		// @param InInjectionRequest   The injection request to uninject
		UAFANIMGRAPH_API static void Uninject(FInjectionRequestPtr InInjectionRequest);
	};
}
