// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextRunAnimationGraph_v2.h"
#include "Graph/RigUnit_UAFRunAsset.h"
#include "Script/UAFScriptComponent.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAFSimplePrePhysicsGraphComponent.generated.h"

#define UE_API UAFANIMGRAPH_API

class USkeletalMeshComponent;
struct FAnimNextAnimGraph;

// Script component that can run a prephysics event to run a graph
USTRUCT()
struct FUAFSimplePrePhysicsGraphComponent : public FUAFScriptComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

private:
	static void PrePhysics(TStructView<FUAFScriptComponent> InScriptComponent, TConstStructView<FUAFScriptContextData> InContextData);

	// FUAFScriptComponent interface
	UE_API virtual void CallEventByName(TConstStructView<FUAFScriptContextData> InContextData) override;
	UE_API virtual TConstArrayView<UE::UAF::FScriptEventInfo> GetScriptEvents() override;

	// FUAFAssetInstanceComponent interface
	UE_API virtual void OnBindToInstance() override;

	// Output value
	UPROPERTY(Transient)
	FUAFValueBundle Output;

	// Work data for the 'run asset node'
	UPROPERTY(Transient)
	FUAFRunAssetWorkData WorkData;

	// Reference to injected graph data in variables
	FAnimNextAnimGraph* Graph = nullptr;
};

#undef UE_API