// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/NodeHandle.h"

#include "EntryPointHandle.generated.h"

/**
 * Entry Point Handle
 * An entry point handle is equivalent to a trait handle but it will not resolve automatically on load.
 * As such, it is safe to use outside of an AnimNext graph.
 * They must be manually resolved through FTraitReader.
 *
 * Internally, it contains a node handle (as a node ID) and a trait index.
 *
 * @see FAnimNextTraitHandle, FTraitReader
 */
USTRUCT(BlueprintType)
struct FAnimNextEntryPointHandle final
{
	GENERATED_BODY()

	// Creates an invalid entry point handle
	constexpr FAnimNextEntryPointHandle() noexcept
		: PackedTraitIndexAndNodeHandle(UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE)
	{}

	// Creates an entry point handle pointing to the first trait of the specified node
	explicit FAnimNextEntryPointHandle(UE::UAF::FNodeHandle NodeHandle)
		: PackedTraitIndexAndNodeHandle(NodeHandle.GetPackedValue() & NODE_HANDLE_MASK)
	{
		check(!NodeHandle.IsValid() || NodeHandle.IsNodeID());
	}

	// Creates an entry point handle pointing to the specified trait on the specified node
	FAnimNextEntryPointHandle(UE::UAF::FNodeHandle NodeHandle, uint32 TraitIndex)
		: PackedTraitIndexAndNodeHandle((NodeHandle.GetPackedValue() & NODE_HANDLE_MASK) | (TraitIndex << 24))
	{
		check(!NodeHandle.IsValid() || NodeHandle.IsNodeID());
		check(TraitIndex <= MAX_uint8);	// Make sure we don't truncate
	}

	// Returns true if this entry point handle is valid, false otherwise
	constexpr bool IsValid() const noexcept { return GetNodeHandle().IsValid(); }

	// Returns the trait index
	constexpr uint32 GetTraitIndex() const noexcept { return PackedTraitIndexAndNodeHandle >> TRAIT_INDEX_SHIFT; }

	// Returns a handle to the node referenced (its node ID)
	constexpr UE::UAF::FNodeHandle GetNodeHandle() const noexcept { return UE::UAF::FNodeHandle::FromPackedValue(PackedTraitIndexAndNodeHandle); }

private:
	// Bottom 24 bits are used by the node handle while the top 8 bits by the trait index
	static constexpr uint32 TRAIT_INDEX_SHIFT = 24;
	static constexpr uint32 NODE_HANDLE_MASK = ~0u >> (32 - TRAIT_INDEX_SHIFT);

	UPROPERTY()
	uint32		PackedTraitIndexAndNodeHandle;
};
