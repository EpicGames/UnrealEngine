// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitPtr.h"
#include "TraitCore/TraitStackBinding.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/IScopedTraitInterface.h"
#include "TraitCore/TraitInterfaceUID.h"
#include "TraitCore/LatentPropertyHandle.h"
#include "TraitCore/NodeHandle.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/UAFGraphInstanceComponent.h"
#include "EngineDefines.h"

#include <type_traits>

#define UE_API UAFANIMGRAPH_API

struct FAnimNextGraphInstance;
class FMemStack;
struct FRigVMDrawInterface;

namespace UE::UAF
{
	namespace Private
	{
		struct FScopedInterfaceEntry;
	}

	struct FNodeDescription;
	struct FNodeInstance;
	struct FNodeTemplateRegistry;
	struct FNodeTemplate;
	struct FTrait;
	struct FTraitRegistry;
	struct FTraitTemplate;

	/**
	 * Execution Context
	 * 
	 * The execution context aims to centralize the trait query API.
	 * It is meant to be bound to a graph instance and re-used by the nodes/traits within.
	 */
	struct FExecutionContext
	{
		// Creates an unbound execution context
		UE_API FExecutionContext();

		// Creates an execution context and binds it to the specified graph instance
		UE_API explicit FExecutionContext(FAnimNextGraphInstance& InGraphInstance);

		// Derived types can control various features
		virtual ~FExecutionContext() = default;

		// Owner object our module instance belongs to. Usually the Anim Next Component. This access is NOT thread-safe.
		UE_API const class UObject* GetHostObject() const;

#if UE_ENABLE_DEBUG_DRAWING
		// The world transform of our host. Usually the actor containing our Anim Next Component
		// @TODO: Not thread-safe and should be replaced ASAP by a subsystem so we can use this in scene graph/mass. Doesn't work for non-actors.
		UE_API const FTransform GetHostTransform() const;
#endif

		//////////////////////////////////////////////////////////////////////////
		// The following functions handle the context binding
		// In order to be used, the execution context must be bound to a valid root graph instance

		// Binds the execution context to the specified graph instance if it differs from the currently bound instance
		UE_API void BindTo(FAnimNextGraphInstance& InGraphInstance);

		// Binds the execution context to the graph instance that owns the specified trait if it differs from the currently bound instance
		UE_API void BindTo(const FWeakTraitPtr& TraitPtr);

		// Returns whether or not this execution context is bound to a graph instance
		UE_API bool IsBound() const;

		// Returns whether or not this execution context is bound to the specified graph instance
		// Returns true if the root graphs match
		UE_API bool IsBoundTo(const FAnimNextGraphInstance& InGraphInstance) const;

		//////////////////////////////////////////////////////////////////////////
		// The following functions allow creation of trait stack bindings

		// Returns a trait stack binding to the stack that contains the specified trait pointer.
		// Returns false if we failed to do so.
		UE_API bool GetStack(const FWeakTraitPtr& TraitPtr, FTraitStackBinding& OutStackBinding) const;



		//////////////////////////////////////////////////////////////////////////
		// The following functions allow for scope interface management and queries
		// Scoped interfaces live on a stack within the execution context
		// A parent node can publish a scoped interface that its children can query for

		// Pushed a scoped interface with the specified interface owned by the specified trait
		// Scoped interfaces automatically pop when the owned trait finishes its update
		template<class ScopedTraitInterface>
		void PushScopedInterface(const FTraitBinding& Binding);

		// Pushed an optional scoped interface with the specified interface owned by the specified trait
		// The scoped interface is only pushed if the condition is true
		// Scoped interfaces automatically pop when the owned trait finishes its update
		template<class ScopedTraitInterface>
		void PushOptionalScopedInterface(bool bCondition, const FTraitBinding& Binding);

		// Pops a scoped interface owned by the specified trait
		// An interface can only be popped if it lives at the top of the stack
		// Returns false if we failed to pop the specified interface
		// Failure can occur if the scoped interface is missing or not present on top of the stack
		template<class ScopedTraitInterface>
		bool PopScopedInterface(const FTraitBinding& Binding);

		// Pops all scoped interfaces owned by the trait stack that contains the specified trait
		// An interface can only be popped if it lives at the top of the stack
		// Returns true if any scoped interfaces were popped, false otherwise
		bool PopStackScopedInterfaces(const FTraitBinding& Binding);

		// Pops all scoped interfaces owned by the specified trait stack
		// An interface can only be popped if it lives at the top of the stack
		// Returns true if any scoped interfaces were popped, false otherwise
		UE_API bool PopStackScopedInterfaces(const FTraitStackBinding& StackBinding);

		// Queries the scoped interface stack starting at the top for a trait that implements the specified interface.
		// If no such trait exists, false is returned.
		template<class ScopedTraitInterface>
		bool GetScopedInterface(TTraitBinding<ScopedTraitInterface>& OutBinding) const;

		// Queries the scoped interface stack starting at the top for a trait that implements the specified interface.
		// When found, the provided callback will be called with the scoped interface binding.
		// Iteration will continue as long as the callback returns true.
		template<class ScopedTraitInterface>
		void ForEachScopedInterface(TFunctionRef<bool(TTraitBinding<ScopedTraitInterface>& Binding)> InFunction) const;

		// Returns whether or not any scoped interfaces have been pushed onto our stack
		bool HasScopedInterfaces() const { return ScopedInterfaceStackHead != nullptr; }



		//////////////////////////////////////////////////////////////////////////
		// The following functions handle node lifetime management

		// Allocates a new node instance from a trait handle using the specified graph instance
		// If the desired trait lives in the current parent, a weak handle to it will be returned
		UE_API FTraitPtr AllocateNodeInstance(FAnimNextGraphInstance& GraphInstance, FAnimNextTraitHandle ChildTraitHandle) const;

		// Allocates a new node instance from a trait handle
		// If the desired trait lives in the current parent, a weak handle to it will be returned
		FTraitPtr AllocateNodeInstance(const FTraitBinding& ParentBinding, FAnimNextTraitHandle ChildTraitHandle) const;

		// Allocates a new node instance from a trait handle
		// If the desired trait lives in the current parent, a weak handle to it will be returned
		UE_API FTraitPtr AllocateNodeInstance(const FWeakTraitPtr& ParentBinding, FAnimNextTraitHandle ChildTraitHandle) const;

		// Decrements the reference count of the provided node pointer and releases it if
		// there are no more references remaining, reseting the pointer in the process
		UE_API void ReleaseNodeInstance(FTraitPtr& NodePtr) const;



		//////////////////////////////////////////////////////////////////////////
		// The following functions handle graph instance components
		// Graph instance components live on the root graph instance and persist from
		// frame to frame

		// Returns a typed graph instance component, creating it lazily the first time it is queried
		template<class ComponentType>
		ComponentType& GetOrAddComponent() const;

		// Returns a typed graph instance component pointer if found or nullptr otherwise
		template<class ComponentType>
		ComponentType* TryGetComponent() const;

		// Iterates each of the graph instance components
		UE_API void ForEachComponent(TFunctionRef<bool(FUAFGraphInstanceComponent&)> InFunction) const;



		//////////////////////////////////////////////////////////////////////////
		// The following functions handle node lifetime management

		// Raises an input trait event
		// Behavior is traversal dependent, see the derived type for your current traversal (e.g. IUpdate)
		// Default behavior asserts
		UE_API virtual void RaiseInputTraitEvent(FAnimNextTraitEventPtr Event);

		// Raises an output trait event
		// Behavior is traversal dependent, see the derived type for your current traversal (e.g. IUpdate)
		// Default behavior asserts
		UE_API virtual void RaiseOutputTraitEvent(FAnimNextTraitEventPtr Event);



		//////////////////////////////////////////////////////////////////////////
		// Misc functions

		// Returns the bound root graph instance
		FAnimNextGraphInstance& GetRootGraphInstance() const { return *RootGraphInstance; }

		// Returns the skeletal mesh component associated to the graph execution, if any
		UE_API TWeakObjectPtr<const USkeletalMeshComponent> GetBindingObject() const;

		// Returns the local thread memstack
		FMemStack& GetMemStack() const { return MemStack; }

#if UE_ENABLE_DEBUG_DRAWING
		// Get the debug draw interface
		UE_API FRigVMDrawInterface* GetDebugDrawInterface() const;
#endif
		
		UE_API void Log(ELogVerbosity::Type Verbosity, const FString& Message, const FString& TraitName) const;

	private:
		// No copy or move
		FExecutionContext(const FExecutionContext&) = delete;
		FExecutionContext& operator=(const FExecutionContext&) = delete;

		UE_API void PushScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding);
		UE_API bool PopScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding);
		UE_API bool GetScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, FTraitBinding& OutBinding) const;
		UE_API void ForEachScopedInterfaceImpl(FTraitInterfaceUID InterfaceUID, TFunctionRef<bool(FTraitBinding& Binding)> InFunction) const;

		UE_API const FNodeDescription& GetNodeDescription(const FAnimNextGraphInstance& GraphInstance, FNodeHandle NodeHandle) const;
		UE_API const FNodeDescription& GetNodeDescription(const FNodeInstance& NodeInstance) const;
		UE_API const FNodeTemplate* GetNodeTemplate(const FNodeDescription& NodeDesc) const;
		UE_API const FTrait* GetTrait(const FTraitTemplate& TraitDesc) const;

	protected:
		// The memstack of the local thread
		FMemStack& MemStack;

	private:
		// Cached references to the registries we need
		const FNodeTemplateRegistry& NodeTemplateRegistry;
		const FTraitRegistry& TraitRegistry;

		// Root graph instance we are bound to
		FAnimNextGraphInstance* RootGraphInstance = nullptr;

		// The head pointer of the scoped interface stack
		Private::FScopedInterfaceEntry* ScopedInterfaceStackHead = nullptr;

		// The head pointer of the free scoped interface entry stack
		// Entries are allocated from the memstack and are re-used in LIFO since they'll
		// be warmer in the CPU cache
		Private::FScopedInterfaceEntry* FreeScopedInterfaceEntryStackHead = nullptr;



		friend struct FTraitStackBinding;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementations

	template<class ScopedTraitInterface>
	inline void FExecutionContext::PushScopedInterface(const FTraitBinding& Binding)
	{
		static_assert(std::is_base_of<IScopedTraitInterface, ScopedTraitInterface>::value, "ScopedTraitInterface type must derive from ITraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = ScopedTraitInterface::InterfaceUID;
		PushScopedInterfaceImpl(InterfaceUID, Binding);
	}

	template<class ScopedTraitInterface>
	inline void FExecutionContext::PushOptionalScopedInterface(bool bCondition, const FTraitBinding& Binding)
	{
		static_assert(std::is_base_of<IScopedTraitInterface, ScopedTraitInterface>::value, "ScopedTraitInterface type must derive from ITraitInterface");

		if (!bCondition)
		{
			return;	// Scoped interface not required
		}

		PushScopedInterface<ScopedTraitInterface>(Binding);
	}

	template<class ScopedTraitInterface>
	inline bool FExecutionContext::PopScopedInterface(const FTraitBinding& Binding)
	{
		static_assert(std::is_base_of<IScopedTraitInterface, ScopedTraitInterface>::value, "ScopedTraitInterface type must derive from ITraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = ScopedTraitInterface::InterfaceUID;
		return PopScopedInterfaceImpl(InterfaceUID, Binding);
	}

	inline bool FExecutionContext::PopStackScopedInterfaces(const FTraitBinding& Binding)
	{
		if (!Binding.IsValid())
		{
			return false;
		}

		return PopStackScopedInterfaces(*Binding.GetStack());
	}

	template<class ScopedTraitInterface>
	inline bool FExecutionContext::GetScopedInterface(TTraitBinding<ScopedTraitInterface>& OutBinding) const
	{
		static_assert(std::is_base_of<IScopedTraitInterface, ScopedTraitInterface>::value, "ScopedTraitInterface type must derive from IScopedTraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = ScopedTraitInterface::InterfaceUID;
		return GetScopedInterfaceImpl(InterfaceUID, OutBinding);
	}

	template<class ScopedTraitInterface>
	inline void FExecutionContext::ForEachScopedInterface(TFunctionRef<bool(TTraitBinding<ScopedTraitInterface>& Binding)> InFunction) const
	{
		static_assert(std::is_base_of<IScopedTraitInterface, ScopedTraitInterface>::value, "ScopedTraitInterface type must derive from IScopedTraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = ScopedTraitInterface::InterfaceUID;

		ForEachScopedInterfaceImpl(InterfaceUID,
			[&InFunction](FTraitBinding& Binding)
			{
				// This cast is safe because we already queried the interface into the binding
				return InFunction(static_cast<TTraitBinding<ScopedTraitInterface>&>(Binding));
			});
	}

	inline FTraitPtr FExecutionContext::AllocateNodeInstance(const FTraitBinding& ParentBinding, FAnimNextTraitHandle ChildTraitHandle) const
	{
		return AllocateNodeInstance(ParentBinding.GetTraitPtr(), ChildTraitHandle);
	}

	template<class ComponentType>
	ComponentType& FExecutionContext::GetOrAddComponent() const
	{
		static_assert(std::is_base_of<FUAFGraphInstanceComponent, ComponentType>::value, "ComponentType type must derive from FGraphInstanceComponent");

		return RootGraphInstance->GetOrAddComponent<ComponentType>();
	}

	template<class ComponentType>
	ComponentType* FExecutionContext::TryGetComponent() const
	{
		static_assert(std::is_base_of<FUAFGraphInstanceComponent, ComponentType>::value, "ComponentType type must derive from FGraphInstanceComponent");

		return RootGraphInstance->TryGetComponent<ComponentType>();
	}
}

#undef UE_API
