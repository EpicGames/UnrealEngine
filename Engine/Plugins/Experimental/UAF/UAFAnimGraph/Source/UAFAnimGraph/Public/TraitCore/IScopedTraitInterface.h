// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"

namespace UE::UAF
{
	/**
	 * IScopedTraitInterface
	 * 
	 * Base type for all scoped trait interfaces. Used for type safety.
	 */
	struct IScopedTraitInterface : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IScopedTraitInterface)
	};
}
