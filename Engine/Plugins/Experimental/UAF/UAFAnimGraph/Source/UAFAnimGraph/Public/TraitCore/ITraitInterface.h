// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitInterfaceUID.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct ITraitInterface;
}

// Helper macros

// In a trait interface struct declaration, this macro declares the necessary boilerplate we require
#define DECLARE_ANIM_TRAIT_INTERFACE(InterfaceName) \
	/* Globally unique UID for this interface */ \
	static constexpr UE::UAF::FTraitInterfaceUID InterfaceUID = UE::UAF::FTraitInterfaceUID::MakeUID(TEXT(#InterfaceName)); \
	virtual UE::UAF::FTraitInterfaceUID GetInterfaceUID() const override { return InterfaceUID; }

// Allows a trait interface to auto-register and unregister within the current execution scope
// The trait interface must be found in the current scope without a namespace qualification
#define AUTO_REGISTER_ANIM_TRAIT_INTERFACE(Interface) \
		UE::UAF::FTraitInterfaceStaticInitHook Interface##Hook(MakeShared<Interface>());


namespace UE::UAF
{
	struct FExecutionContext;	// Derived types will have functions that accept the execution context

	/**
	 * ITraitInterface
	 * 
	 * Base type for all trait interfaces. Used for type safety.
	 */
	struct ITraitInterface
	{
		virtual ~ITraitInterface() {}

		// The globally unique UID for this interface
		// Derived types will have their own InterfaceUID member that hides/aliases/shadows this one
		// @see DECLARE_ANIM_TRAIT_INTERFACE
		static constexpr FTraitInterfaceUID InterfaceUID = FTraitInterfaceUID::MakeUID(TEXT("ITraitInterface"));

		// Returns the globally unique UID for this interface
		virtual FTraitInterfaceUID GetInterfaceUID() const { return InterfaceUID; };

#if WITH_EDITOR
		// Internal interfaces are only displayed in the Traits Editor in the Advanced View
		virtual bool IsInternal() const { return false; }

		// Human readable interface names, in long and short format
		virtual const FText& GetDisplayName() const { ensure(false); return FText::GetEmpty(); }
		// Human readable interface names, in short format (ideally 3 or 4 letters, due to space restrictions in the editor)
		virtual const FText& GetDisplayShortName() const { ensure(false); return FText::GetEmpty(); }
#endif // WITH_EDITOR
	};


	/**
	 * Trait Stack Propagation
	 * 
	 * This enum can be used to signal whether an interface call should be forwarded to its parent
	 * on the trait stack or not.
	 */
	enum class ETraitStackPropagation
	{
		// Forward the call to our parent on the trait stack (if we have one)
		Continue,

		// Do not forward the call to our parent, execution stops
		Stop,
	};

	/**
	* FTraitInterfaceStaticInitHook
	*
	* Allows trait interfaces to automatically register/unregister within the current scope.
	* This can be used during static init.
	* See AUTO_REGISTER_ANIM_TRAIT_INTERFACE
	*/
	struct FTraitInterfaceStaticInitHook final
	{
		UE_API explicit FTraitInterfaceStaticInitHook(const TSharedPtr<UE::UAF::ITraitInterface>& InTraitInterface);
		UE_API ~FTraitInterfaceStaticInitHook();

	private:
		TSharedPtr<ITraitInterface> TraitInterface = nullptr;
	};
}

#undef UE_API
