// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/Archive.h"

namespace UE::UAF
{
	class FTraitReader;

	/**
	 * Latent Property Metadata
	 * 
	 * The metadata we serialize in the archive for each latent property.
	 */
	struct FLatentPropertyMetadata final
	{
		// The name of the latent property
		FName Name;

		// The index of the RigVM memory handle to evaluate to compute our latent value
		// An invalid index of ~0 means that the latent property has a hardcoded inline value
		uint16 RigVMIndex = MAX_uint16;

		// Whether or not the property supports freezing
		bool bCanFreeze = false;

		// Whether or not the property is a simple variable copy
		bool bUsesVariableCopy = false;
		
		// Whether or not the property evaluates only once on become relevant
		bool bOnBecomeRelevant = false;
	};
	

	// Serializes latent property metadata
	inline FArchive& operator<<(FArchive& Ar, FLatentPropertyMetadata& Metadata)
	{
		Ar << Metadata.Name;
		Ar << Metadata.RigVMIndex;
		Ar << Metadata.bCanFreeze;
		Ar << Metadata.bUsesVariableCopy;
		Ar << Metadata.bOnBecomeRelevant;
		return Ar;
	}

	/**
	 * Latent Properties Header
	 * Holds metadata about the stored latent properties
	 *
	 * @see FExecutionContext
	 */
	struct alignas(uint16) FLatentPropertiesHeader final
	{
		// Whether or not this base trait has any bound latent properties that can evaluate
		uint8 bHasValidLatentProperties : 1 = false;

		// Whether or not this base trait has all its latent properties support freezing
		uint8 bCanAllPropertiesFreeze : 1 = false;
		
		// Whether or not this base trait has all its latent properties only evaluate on first frame
		uint8 bAllLatentPropertiesOnBecomeRelevant : 1 = false;

		// Whether the latent properties for this trait are all simple variable accesses
		uint8 bAreAllPropertiesVariableAccesses : 1 = false;
	};

	/**
	 * Latent Property Handle
	 * A latent property handle represents:
	 *    - A 1-based index into the RigVM memory handles array during execution
	 *    - A flag that specifies if the latent property should always update or if it supports freezing
	 *    - A node instance local offset to the cached latent property value
	 * 
	 * A latent property can be in one of 3 states:
	 *    - Invalid (property is inline and not latent)
	 *    - Latent and Cached (property is latent and is cached at some offset)
	 *    - Cached Only (property is not latent but is cached at some offset)
	 * 
	 * The Cached Only state can occur if a base trait has duplicate entries within its property handles.
	 * Duplicate entries would have the same RigVM index but different offsets and thus evaluate it repeatedly.
	 * To avoid the waste of cycles and space, subsequent properties are marked as Cached Only with the same
	 * offset as the first property that is Latent and Cached (it will update the cached location).
	 *
	 * @see FExecutionContext
	 */
	struct FLatentPropertyHandle final
	{
		// Creates an invalid latent property handle
		constexpr FLatentPropertyHandle() noexcept = default;

		// Returns true if this latent property handle has a valid RigVM index, false otherwise
		constexpr bool IsIndexValid() const noexcept { return (PackedIndexCanFreezeValue & HANDLE_INDEX_MASK) != INVALID_HANDLE_VALUE; }

		// Returns true if this latent property handle has a valid property offset, false otherwise
		constexpr bool IsOffsetValid() const noexcept { return OffsetValue != INVALID_OFFSET_VALUE; }

		// Returns the latent property RigVM index represented by this handle if valid, -1 otherwise
		constexpr int32 GetLatentPropertyIndex() const noexcept { return IsIndexValid() ? (PackedIndexCanFreezeValue & HANDLE_INDEX_MASK) : INDEX_NONE; }

		// Returns the latent property offset represented by this handle if valid, 0 otherwise
		// This offset is relative to the start of the node instance data
		constexpr uint32 GetLatentPropertyOffset() const noexcept { return OffsetValue; }

		// Returns whether or not the latent property supports freezing during snapshots
		// Latent properties that do not support freezing always update in a snapshot
		constexpr bool CanFreeze() const noexcept { return IsIndexValid() ? !!(PackedIndexCanFreezeValue & HANDLE_CAN_FREEZE_MASK) : false; }
		
		// Returns whether or not the latent property evaluates only on become relevant
		constexpr bool OnBecomeRelevant() const noexcept { return IsIndexValid() ? !!(PackedIndexCanFreezeValue & HANDLE_ON_BECOME_RELEVANT_MASK) : false; }

	private:
		// Constructs a FLatentPropertyHandle instance
		FLatentPropertyHandle(uint16 InRigRMIndex, uint32 InPropertyOffset, bool bInCanFreeze, bool bInOnBecomeRelevant)
			: PackedIndexCanFreezeValue((InRigRMIndex & HANDLE_INDEX_MASK) | (uint16(bInCanFreeze) << HANDLE_CAN_FREEZE_SHIFT_OFFSET) | (uint16(bInOnBecomeRelevant) << HANDLE_ON_BECOME_RELEVANT_SHIFT_OFFSET))
			, OffsetValue(uint16(InPropertyOffset))
		{
			check(InRigRMIndex <= INVALID_HANDLE_VALUE || InRigRMIndex == MAX_uint16);	// Must fit on 15 bits when valid
			check(InPropertyOffset <= MAX_uint16);	// Must fit on 16 bits
		}

		// A mask for the RigVM index portion of the packed value
		static constexpr uint16 HANDLE_INDEX_MASK = 0x3FFF;

		// A mask for the CanFreeze metadata flag in the packed value
		static constexpr uint16 HANDLE_CAN_FREEZE_MASK = 0x8000;
		static constexpr uint16 HANDLE_ON_BECOME_RELEVANT_MASK = 0x4000;

		// A shift offset for the CanFreeze metadata flag in the packed value
		static constexpr uint16 HANDLE_CAN_FREEZE_SHIFT_OFFSET = 15;
		static constexpr uint16 HANDLE_ON_BECOME_RELEVANT_SHIFT_OFFSET = 14;

		// Latent property handles are a 15 bit index and a 1 bit flag to denote if the property can freeze, ~0 is invalid
		static constexpr uint16 INVALID_HANDLE_VALUE = 0x3FFF;

		// Latent property offsets are relative to the start of the node instance data, 0 is invalid
		static constexpr uint16 INVALID_OFFSET_VALUE = 0;

		// A packed RigVM index and metadata flag for whether or not this property can be frozen
		uint16		PackedIndexCanFreezeValue = INVALID_HANDLE_VALUE;

		// An offset relative to the start of the node instance where the latent property is cached
		uint16		OffsetValue = INVALID_OFFSET_VALUE;

		friend FArchive& operator<<(FArchive& Ar, FLatentPropertyHandle& Handle);
		friend FTraitReader;
	};

	static_assert(alignof(FLatentPropertiesHeader) == alignof(FLatentPropertyHandle), "Header and handle must have matching alignment");

	// Serializes a latent handle value
	inline FArchive& operator<<(FArchive& Ar, FLatentPropertyHandle& Handle)
	{
		Ar << Handle.PackedIndexCanFreezeValue;
		Ar << Handle.OffsetValue;
		return Ar;
	}
}
