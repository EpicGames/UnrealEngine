// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/NodeID.h"
#include "TraitCore/NodeTemplateRegistryHandle.h"

class FArchive;

namespace UE::UAF
{
	class FTraitReader;

	/**
	 * Node Description
	 * A node description represents a unique instance in the authored static graph.
	 * A node description may have any number of runtime instances in the dynamically executed graph.
	 * As such, a node description is read-only at runtime while a node instance is read/write.
	 * 
	 * A node description is followed in memory by the trait descriptions (their shared read-only data)
	 * that live within it. Trait descriptions include things like hard-coded/inline properties, pin links, etc.
	 * 
	 * A node description is itself an instance of a node template.
	 * 
	 * @see FNodeTemplate
	 */
	struct alignas(alignof(uint32)) FNodeDescription
	{
		// Largest allowed size for a node description and the shared data of its traits
		// We use unsigned 16 bit offsets within the NodeTemplate/TraitTemplate
		static constexpr uint32 MAXIMUM_NODE_SHARED_DATA_SIZE = 64 * 1024;

		FNodeDescription(FNodeID InNodeID, FNodeTemplateRegistryHandle InTemplateHandle)
			: NodeID(InNodeID)
			, TemplateHandle(InTemplateHandle)
			, NodeInstanceDataSize(0)
			, Padding0(0)
		{}

		// Returns the node UID, unique to the owning sub-graph
		FNodeID GetUID() const { return NodeID; }

		// Returns the handle of the node's template in the node template registry
		FNodeTemplateRegistryHandle GetTemplateHandle() const { return TemplateHandle; }

		// Returns the node instance data size factoring in any cached latent properties
		uint32 GetNodeInstanceDataSize() const { return NodeInstanceDataSize; }

		// Serializes this node description instance and the shared data of each trait that follows
		UAFANIMGRAPH_API void Serialize(FArchive& Ar);

	private:
		// Assigned during export/cook, unique to current sub-graph (16 bits)
		FNodeID							NodeID;

		// Offset of the node template within the global list (16 bits)
		FNodeTemplateRegistryHandle		TemplateHandle;

		// The node instance data size, includes latent properties (not serialized, @see FNodeTemplate::Finalize)
		uint16	NodeInstanceDataSize;

		uint16	Padding0;	// Unused

		friend FTraitReader;

		// This structures is the header for a node's shared data. The memory layout is as follows:
		// 
		// [FNodeDescription] for the header
		// [FAnimNextTraitSharedData] for trait 1 (base)
		// [FLatentPropertiesHeader][FLatentPropertyHandle][...] for the latent properties of trait 1
		// [FAnimNextTraitSharedData] for trait 2 (additive)
		// [FAnimNextTraitSharedData] for trait 3 (base)
		// [FLatentPropertiesHeader][FLatentPropertyHandle][...] for the latent properties of trait 2
		// [...]
		// 
		// Each node is thus followed by the trait shared data contiguously.
		// After each base trait's shared data, an optional list of latent property handles follows
		// before the next trait's shared data begins. This list contains all latent property handles
		// of the whole sub-stack (including the additive traits on top of the base).
		// 
		// Each trait contains a shared data structure that derives from FAnimNextTraitSharedData.
		// That derived structure is what is contained in the actual buffer. As such, sizes and offsets
		// vary as required. The [FTraitTemplate] contains the offsets that map here.
		// 
		// Optional padding is inserted as required by alignment constraints.
	};
}
