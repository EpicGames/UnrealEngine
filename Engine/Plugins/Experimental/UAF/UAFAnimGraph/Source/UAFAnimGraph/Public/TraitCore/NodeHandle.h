// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/NodeID.h"
#include "Serialization/Archive.h"

struct FAnimNextTraitHandle;
struct FAnimNextEntryPointHandle;

namespace UE::UAF
{
	/**
	 * Node Handle
	 * A node handle represents a reference to a specific node instance in the shared/read-only portion
	 * of a sub-graph. It points to a FNodeDescription when resolved.
	 * 
	 * Internally, it contains either a shared data offset into the parent graph (at runtime) or a FNodeID (during compilation and on disk).
	 * Shared offsets must be computed at runtime and as such we use the FNodeID to resolve into our final offset on load.
	 * We only use the bottom 24 bits. If the LSB is 0, the packed value represents a shared offset (due to alignment) but if the
	 * LSB is 1, the packed value represents a node ID shifted left by 1.
	 * 
	 * @see FNodeDescription, FNodeID
	 */
	struct FNodeHandle final
	{
		// Creates an invalid node handle
		constexpr FNodeHandle() noexcept
			: PackedSharedOffsetOrID(INVALID_NODE_HANDLE_RAW_VALUE)
		{}

		// Creates a node handle for the specified shared offset
		static constexpr FNodeHandle FromSharedOffset(uint32 SharedOffset) noexcept
		{
			if (SharedOffset > NODE_HANDLE_MASK)
			{
				return FNodeHandle();	// The shared offset is too large, the handle isn't valid
			}

			return FNodeHandle(SharedOffset);
		}

		// Creates a node handle from a node ID
		static constexpr FNodeHandle FromNodeID(FNodeID NodeID) noexcept
		{
			if (!NodeID.IsValid())
			{
				return FNodeHandle();	// The NodeID isn't valid, the handle isn't valid
			}

			return FNodeHandle((NodeID.GetRawValue() << NODE_ID_SHIFT) | NODE_ID_MARKER_BIT_MASK);
		}

		// Returns true if this node handle is valid, false otherwise
		constexpr bool IsValid() const noexcept { return (PackedSharedOffsetOrID & NODE_HANDLE_MASK) != INVALID_NODE_HANDLE_RAW_VALUE; }

		// Returns whether this node handle encodes a shared offset or not
		constexpr bool IsSharedOffset() const noexcept { return IsValid() && (PackedSharedOffsetOrID & NODE_ID_MARKER_BIT_MASK) == 0; }

		// Returns whether this node handle encodes a node ID or not
		constexpr bool IsNodeID() const noexcept { return IsValid() && (PackedSharedOffsetOrID & NODE_ID_MARKER_BIT_MASK) != 0; }

		// Returns the offset into the shared data relative to the root of the owning sub-graph
		uint32 GetSharedOffset() const { check(IsSharedOffset()); return PackedSharedOffsetOrID & NODE_HANDLE_MASK; }

		// Returns the node ID encoded by the handle
		FNodeID GetNodeID() const { check(IsNodeID()); return FNodeID::FromRawValue((PackedSharedOffsetOrID & NODE_HANDLE_MASK) >> NODE_ID_SHIFT); }

		// Equality and inequality tests
		constexpr bool operator==(FNodeHandle RHS) const noexcept { return (PackedSharedOffsetOrID & NODE_HANDLE_MASK) == (RHS.PackedSharedOffsetOrID & NODE_HANDLE_MASK); }
		constexpr bool operator!=(FNodeHandle RHS) const noexcept { return (PackedSharedOffsetOrID & NODE_HANDLE_MASK) != (RHS.PackedSharedOffsetOrID & NODE_HANDLE_MASK); }

	private:
		explicit constexpr FNodeHandle(uint32 PackedValue) noexcept
			: PackedSharedOffsetOrID(PackedValue)
		{}

		// Returns the packed raw value
		constexpr uint32 GetPackedValue() const { return PackedSharedOffsetOrID; }

		// Creates a node handle from a raw packed value
		static constexpr FNodeHandle FromPackedValue(uint32 PackedValue) noexcept { return FNodeHandle(PackedValue); }

		// We pack the shared offset on 24 bits, the top 8 bits are unused and they can have any value (zero by default)
		static constexpr uint32 INVALID_NODE_HANDLE_RAW_VALUE = ~0u >> (32 - 24);	// 0x00FFFFFF
		static constexpr uint32 NODE_HANDLE_MASK = INVALID_NODE_HANDLE_RAW_VALUE;
		static constexpr uint32 NODE_ID_MARKER_BIT_MASK = 0x01u;					// When valid, a LSB of 1 means we encode a node ID, 0 means a shared offset
		static constexpr uint32 NODE_ID_SHIFT = 1;									// Our LSB is reserved as a marker, the node ID is shifted left by 1

		uint32	PackedSharedOffsetOrID;

		friend FAnimNextTraitHandle;
		friend FAnimNextEntryPointHandle;
	};
}
