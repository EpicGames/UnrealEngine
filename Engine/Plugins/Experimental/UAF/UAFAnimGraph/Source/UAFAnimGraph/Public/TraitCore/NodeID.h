// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/Archive.h"

namespace UE::UAF
{
	/**
	  * Node ID
	  * 
	  * A node ID fits on 16 bits and is used to encode a node index.
	  * The zero ID is reserved as an invalid value, this means that the
	  * node index is 1-based.
	  * 
	  * Node IDs are assigned during compilation of an AnimNext graph to each node.
	  * The trait reader is then used to convert them into a shared offset within
	  * the graph.
	  * 
	  * @see FTraitReader
	  */
	struct FNodeID final
	{
		// Constructs an invalid FNodeID
		constexpr FNodeID() noexcept : ID(INVALID_NODE_ID) {}

		// Returns whether or not this ID is valid
		constexpr bool IsValid() const noexcept { return ID != INVALID_NODE_ID; }

		// Returns the node index represented by this node ID
		// If the node ID is invalid, the index returned is 0xFFFFFFFF
		constexpr uint32 GetNodeIndex() const noexcept { return static_cast<uint32>(ID) - 1; }

		// The first valid node ID
		static constexpr FNodeID GetFirstID() noexcept { return FNodeID(1); }

		// Returns the next node ID
		// If the current ID is invalid, the next is invalid as well
		// Note that this function does not handle wrapping if more than 65536 IDs are used
		// If wrapping occurs, the next ID returned will be invalid and all subsequent IDs will be invalid
		constexpr FNodeID GetNextID() const noexcept { return IsValid() ? FNodeID(ID + 1) : FNodeID(); }

		// Equality and inequality tests
		constexpr bool operator==(const FNodeID RHS) const noexcept { return ID == RHS.ID; }
		constexpr bool operator!=(const FNodeID RHS) const noexcept { return ID != RHS.ID; }

		// Serialization operator
		friend FArchive& operator<<(FArchive& Ar, FNodeID& Value)
		{
			Ar << Value.ID;
			return Ar;
		}

	private:
		// Constructs a FNodeID instance from a raw value
		explicit constexpr FNodeID(uint16 InID) noexcept : ID(InID) {}

		// Returns the packed raw value
		constexpr uint32 GetRawValue() const { return ID; }

		// Creates a node handle from a raw packed value
		static constexpr FNodeID FromRawValue(uint32 InID) noexcept { return FNodeID(static_cast<uint16>(InID)); }

		static constexpr uint16 INVALID_NODE_ID = 0;

		uint16 ID;

		friend struct FNodeHandle;
	};
}
