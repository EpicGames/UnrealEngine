// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitTemplate.h"
#include "TraitCore/NodeHandle.h"

struct FAnimNextGraphInstance;

namespace UE::UAF
{
	/**
	 * Node Instance
	 * A node instance represents allocated data for specific node.
	 *
	 * In order to access the trati instance data, the offsets need to be looked up in the node template.
	 * 
	 * @see FNodeTemplate, FTraitTemplate
	 */
	struct FNodeInstance
	{
		// Largest allowed size for a node instance and the instance data of its traits
		// We use unsigned 16 bit offsets within the NodeTemplate/TraitTemplate
		static constexpr uint32 MAXIMUM_NODE_INSTANCE_DATA_SIZE = 64 * 1024;

		// Returns whether the node instance is valid or not
		bool IsValid() const { return NodeHandle.IsValid(); }

		// Returns whether or not this node instance is owned by the specified graph instance
		bool IsOwnedBy(const FAnimNextGraphInstance& GraphInstance) const { return &Owner == &GraphInstance; }

		// Returns the graph instance that owns this node instance
		FAnimNextGraphInstance& GetOwner() const { return Owner; }

		// Returns a handle to the shared data for this node
		FNodeHandle GetNodeHandle() const { return NodeHandle; }

		// Returns the number of live references to this node instance, does not include weak handles
		uint32 GetReferenceCount() const { return ReferenceCount; }

	private:
		FNodeInstance(FAnimNextGraphInstance& InOwner, FNodeHandle InNodeHandle) : Owner(InOwner), ReferenceCount(0), NodeHandle(InNodeHandle) {}
		~FNodeInstance() { check(ReferenceCount == 0); }

		// Cannot move or copy node instances
		FNodeInstance(const FNodeInstance&) = delete;
		FNodeInstance& operator=(const FNodeInstance&) = delete;

		// Increments the reference count
		void AddReference() { ReferenceCount++; }

		// Decrements the reference count and returns true if any references remain
		bool RemoveReference() { check(ReferenceCount > 0); return ReferenceCount-- != 1; }

		FAnimNextGraphInstance&		Owner;					// The graph instance that owns this node instance

		uint32						ReferenceCount;			// how many non-weak FTraitPtr handles point to us, not thread safe
		FNodeHandle					NodeHandle;				// relative to root of sub-graph, should this be a pointer?

		friend struct FTraitPtr;
		friend struct FExecutionContext;
		friend struct FNodePtr;

		// This structures is the header for a node's instance data. The memory layout is as follows:
		// 
		// [FNodeInstance] for the header
		// [FTraitInstanceData] for trait 1
		// [FTraitInstanceData] for trait 2
		// [FTraitInstanceData] for trait 3
		// [...]
		// [LatentProperty] for some trait
		// [LatentProperty] for some trait
		// [...]
		// 
		// Each node is thus followed by the trait instance data contiguously.
		// Following the trait instance data, cached latent properties are present
		// 
		// Each trait contains an instance data structure that derives from FTraitInstanceData.
		// That derived structure is what is contained in the actual buffer. As such, sizes and offsets
		// vary as required. The [FTraitTemplate] contains the offsets that map here.
		// 
		// Optional padding is inserted as required by alignment constraints.
	};
}
