// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitTemplate.h"

#include <type_traits>

class FArchive;

namespace UE::UAF
{
	/**
	 * Node Template
	 * A node template represents a specific combination and ordering of traits on a node.
	 * For example, a sequence player + sync group trait + output caching trait
	 * 
	 * Multiple node descriptions can share the same template each with their own property/pin values.
	 * 
	 * Templates can exist in multiple sub-graphs on disk. On load, a single unique copy is retained in
	 * memory and shared between ALL graphs.
	 * 
	 * Node templates are created through the FNodeTemplateBuilder.
	 * 
	 * @see FNodeDescription, FNodeTemplateBuilder
	 */
	struct FNodeTemplate
	{
		// The largest size allowed for a node template
		// Node templates are just a lightweight descriptor, in theory they can have any size
		// We artificially specify a conservative upper bound
		static constexpr uint32 MAXIMUM_SIZE = 64 * 1024;

		// Returns the globally unique template identifier (a hash of all the trait UIDs present in this node)
		uint32 GetUID() const { return UID; }

		// Returns the size in bytes of this node template
		uint32 GetNodeTemplateSize() const { return sizeof(FNodeTemplate) + (NumTraits * sizeof(FTraitTemplate)); }

		// Returns the size in bytes of a node shared data
		uint32 GetNodeSharedDataSize() const { return NodeSharedDataSize; }

		// Returns the size in bytes of a node instance (how much space to allocate to hold a node instance of this template)
		uint32 GetNodeInstanceDataSize() const { return NodeInstanceDataSize; }

		// Returns the number of traits present in the node template
		uint32 GetNumTraits() const { return NumTraits; }

		// Returns the required alignment of this node
		uint32 GetMinAlignment() const { return MinAlignment; }

		// Returns whether the node template is valid or not
		bool IsValid() const { return NodeSharedDataSize != 0 && NodeInstanceDataSize != 0; }

		// Returns a pointer to the list of trait template descriptions
		FTraitTemplate* GetTraits() { return reinterpret_cast<FTraitTemplate*>(reinterpret_cast<uint8*>(this) + sizeof(FNodeTemplate)); }

		// Returns a pointer to the list of trait template descriptions
		const FTraitTemplate* GetTraits() const { return reinterpret_cast<const FTraitTemplate*>(reinterpret_cast<const uint8*>(this) + sizeof(FNodeTemplate)); }

		// Serializes this node template instance and each trait template that follows
		UAFANIMGRAPH_API void Serialize(FArchive& Ar);

	private:
		friend struct FNodeTemplateBuilder;

		FNodeTemplate(uint32 InUID, uint32 InNumTraits)
			: UID(InUID)
			, NodeSharedDataSize(0)
			, NodeInstanceDataSize(0)
			, NumTraits(static_cast<uint8>(InNumTraits))
		{}

		// This function performs the necessary fix-ups once all trait templates have been populated
		// Call this function whenever a new node template has been constructed before using it
		UAFANIMGRAPH_API void Finalize();

		uint32	UID;					// globally unique template identifier or hash

		uint16	NodeSharedDataSize;		// size in bytes of a node shared data (not serialized, @see FNodeTemplate::Finalize)
		uint16	NodeInstanceDataSize;	// size in bytes of a node instance, excludes optional latent properties (not serialized, @see FNodeTemplate::Finalize)

		uint8	NumTraits;
		uint8	MinAlignment;
		uint8	Padding[2];

		// TODO: We could use the padding (and extra space) here to cache which trait handles which interface
		// This would avoid the need to iterate on every trait to look it up. Perhaps only common interfaces could be cached.

		// Followed in memory by a list of [FTraitTemplate] instances
	};

	static_assert(std::is_trivially_copyable_v<FNodeTemplate>, "FNodeTemplate needs to be trivially copyable");
	static_assert(alignof(FNodeTemplate) == alignof(FTraitTemplate), "FNodeTemplate and FTraitTemplate must have the same alignment");
}
