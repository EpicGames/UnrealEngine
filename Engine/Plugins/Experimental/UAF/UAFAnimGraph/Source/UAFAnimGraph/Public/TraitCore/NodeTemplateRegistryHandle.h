// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	/**
	 * FNodeTemplateRegistryHandle
	 *
	 * Encapsulates a value used as a handle in the node template registry.
	 * When valid, it can be used to retrieve a pointer to the corresponding node template.
	 */
	struct FNodeTemplateRegistryHandle final
	{
		// Default constructed handles are invalid
		FNodeTemplateRegistryHandle() = default;

		// Returns whether or not this handle points to a valid node template
		bool IsValid() const { return TemplateOffset != 0; }

		// Returns the template offset for this handle when valid, otherwise INDEX_NONE
		int32 GetTemplateOffset() const { return IsValid() ? (TemplateOffset - 1) : INDEX_NONE; }

		// Compares for equality and inequality
		bool operator==(FNodeTemplateRegistryHandle RHS) const { return TemplateOffset == RHS.TemplateOffset; }
		bool operator!=(FNodeTemplateRegistryHandle RHS) const { return TemplateOffset != RHS.TemplateOffset; }

	private:
		explicit FNodeTemplateRegistryHandle(int16 TemplateOffset_)
			: TemplateOffset(TemplateOffset_)
		{}

		// Creates a handle based on a note template offset in the static buffer
		static FNodeTemplateRegistryHandle MakeHandle(int32 TemplateOffset_)
		{
			check(TemplateOffset_ >= 0 && TemplateOffset_ < ((1 << 16) - 2));
			return FNodeTemplateRegistryHandle(TemplateOffset_ + 1);
		}

		// When 0, the handle is invalid
		// Otherwise it is a 1-based offset in the registry's static buffer
		int16 TemplateOffset = 0;

		friend struct FNodeTemplateRegistry;
	};
}
