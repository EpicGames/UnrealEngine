// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMStructUpgradeInfo.h"

#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitHandle.h"			// Derived types are likely to refer to other traits as children
#include "TraitCore/TraitInstanceData.h"
#include "TraitCore/TraitMode.h"
#include "TraitCore/TraitPtr.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitCore/TraitUID.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/LatentPropertyHandle.h"

#include <type_traits>

#define UE_API UAFANIMGRAPH_API

class FArchive;
class URigVMController;
class URigVMPin;
struct FRigVMPinInfoArray;

#define ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_BASIC(TraitName, SuperTraitName) \
	using TraitSuper = SuperTraitName; \
	/* FTrait impl */ \
	static constexpr UE::UAF::FTraitUID TraitUID = UE::UAF::FTraitUID::MakeUID(TEXT(#TraitName)); \
	virtual UE::UAF::FTraitUID GetTraitUID() const override { return TraitUID; } \
	virtual FString GetTraitName() const override { return TEXT(#TraitName); }

#define ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INSTANCING_SUPPORT() \
	static const UE::UAF::FTraitMemoryLayout TraitMemoryDescription; \
	virtual UE::UAF::FTraitMemoryLayout GetTraitMemoryDescription() const override { return TraitMemoryDescription; } \
	virtual UScriptStruct* GetTraitSharedDataStruct() const override { return FSharedData::StaticStruct(); } \
	virtual void ConstructTraitInstance(const UE::UAF::FExecutionContext& Context, const UE::UAF::FTraitBinding& Binding) const override; \
	virtual void DestructTraitInstance(const UE::UAF::FExecutionContext& Context, const UE::UAF::FTraitBinding& Binding) const override;

#define ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INTERFACE_SUPPORT() \
	virtual const UE::UAF::ITraitInterface* GetTraitInterface(UE::UAF::FTraitInterfaceUID InterfaceUID) const override; \
	virtual TConstArrayView<UE::UAF::FTraitInterfaceUID> GetTraitInterfaces() const override; \
	virtual TConstArrayView<UE::UAF::FTraitInterfaceUID> GetTraitRequiredInterfaces() const override;

#define ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_EVENT_SUPPORT() \
	virtual UE::UAF::ETraitStackPropagation OnTraitEvent(UE::UAF::FExecutionContext& Context, UE::UAF::FTraitBinding& Binding, FAnimNextTraitEvent& Event) const override; \
	virtual TConstArrayView<UE::UAF::FTraitEventUID> GetTraitEvents() const override;

#define ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_LATENT_PROPERTY_SUPPORT() \
	virtual uint32 GetNumLatentTraitProperties() const override { return -FSharedData::GetLatentPropertyIndex(~(size_t)0); } \
	virtual UE::UAF::FTraitLatentPropertyMemoryLayout GetLatentPropertyMemoryLayout(const FAnimNextTraitSharedData& SharedData, FName PropertyName, uint32 PropertyIndex) const override; \
	virtual int32 GetLatentPropertyIndex(const FAnimNextTraitSharedData& InSharedData, FName PropertyName) const override;

// Helper macros
// In the trait class declaration, this macro declares the Super alias and base functions we override
// We use use FNV1A32 to hash entire TraitName, including 'F' part (Currently must be manually done). 
#define DECLARE_ANIM_TRAIT(TraitName, SuperTraitName) \
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_BASIC(TraitName, SuperTraitName) \
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INSTANCING_SUPPORT() \
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INTERFACE_SUPPORT() \
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_EVENT_SUPPORT() \
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_LATENT_PROPERTY_SUPPORT()

#define DECLARE_ABSTRACT_ANIM_TRAIT(TraitName, SuperTraitName) \
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_BASIC(TraitName, SuperTraitName)

// In the trait cpp, these three macros implement the base functionality
// 
// Usage is as follow:
// #define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
//		GeneratorMacro(IHierarchy) \
//		GeneratorMacro(IUpdate) \
//
// GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMyTrait, TRAIT_INTERFACE_ENUMERATOR)
// #undef TRAIT_INTERFACE_ENUMERATOR

// Implements various parts of FTrait
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT(TraitName) \
	const UE::UAF::FTraitMemoryLayout TraitName::TraitMemoryDescription = \
		UE::UAF::FTraitMemoryLayout{ \
			sizeof(TraitName), alignof(TraitName), \
			/* Use the effective size to ensure optimal packing for empty shared/instance data */ \
			UE::UAF::Private::TGetEffectiveSize<TraitName::FSharedData>::Value, alignof(TraitName::FSharedData), \
			UE::UAF::Private::TGetEffectiveSize<TraitName::FInstanceData>::Value, alignof(TraitName::FInstanceData) \
		}; \
	void TraitName::ConstructTraitInstance(const UE::UAF::FExecutionContext& Context, const UE::UAF::FTraitBinding& Binding) const \
	{ \
		static_assert(std::is_base_of<TraitSuper::FSharedData, TraitName::FSharedData>::value, "Trait shared data must derive from its Super's FSharedData"); \
		static_assert(std::is_base_of<TraitSuper::FInstanceData, TraitName::FInstanceData>::value, "Trait instance data must derive from Super's FInstanceData"); \
		/* Construct the base struct first */ \
		FInstanceData* Data = new(Binding.GetInstanceData<FInstanceData>()) FInstanceData(); \
		/* Then construct our latent properties, the Construct implementation below might need them */ \
		FSharedData::ConstructLatentProperties(Binding); \
		/* Construct our typed instance last */ \
		Data->Construct(Context, Binding); \
	} \
	void TraitName::DestructTraitInstance(const UE::UAF::FExecutionContext& Context, const UE::UAF::FTraitBinding& Binding) const \
	{ \
		/* Destruction is reverse order of construction above */ \
		FInstanceData* Data = Binding.GetInstanceData<FInstanceData>(); \
		Data->Destruct(Context, Binding); \
		FSharedData::DestructLatentProperties(Binding); \
		Data->~FInstanceData(); \
	}

// Implements GetLatentPropertyMemoryLayout()
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_LATENT_PROPERTY_MEMORY_LAYOUT(TraitName) \
	UE::UAF::FTraitLatentPropertyMemoryLayout TraitName::GetLatentPropertyMemoryLayout(const FAnimNextTraitSharedData& SharedData, FName PropertyName, uint32 PropertyIndex) const \
	{ \
		/* Thread safe cache initialization */ \
		static TArray<UE::UAF::FTraitLatentPropertyMemoryLayout> CachedLatentPropertyMemoryLayouts = [this](){ TArray<UE::UAF::FTraitLatentPropertyMemoryLayout> Result; Result.SetNum(GetNumLatentTraitProperties()); return Result; }(); \
		return GetLatentPropertyMemoryLayoutImpl(SharedData, PropertyName, PropertyIndex, CachedLatentPropertyMemoryLayouts); \
	}

#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_IS_PROPERTY_LATENT(TraitName) \
	int32 TraitName::GetLatentPropertyIndex(const FAnimNextTraitSharedData& InSharedData, FName PropertyName) const \
	{ \
		const UScriptStruct* SharedDataStruct = FSharedData::StaticStruct(); \
		const FProperty* Property = SharedDataStruct->FindPropertyByName(PropertyName); \
		if (!ensure(Property != nullptr)) \
		{ \
			/* Unknown property */ \
			return INDEX_NONE; \
		} \
		const int32 LatentPropertyIndex = FSharedData::GetLatentPropertyIndex(Property->GetOffset_ForInternal()); \
		if (LatentPropertyIndex <= 0) \
		{ \
			/* Property not part of latent enumerator macro */ \
			return INDEX_NONE; \
		} \
		return LatentPropertyIndex - 1; \
	}

// Helper that handles the GetTraitInterface() details for each interface specified by the generator macro
#define ANIM_NEXT_IMPL_GET_INTERFACE_IMPL_FOR_INTERFACE(InterfaceName) \
	if (InInterfaceUID == InterfaceName::InterfaceUID) \
	{ \
		return static_cast<const InterfaceName*>(this); \
	}

// Implements GetTraitInterface()
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACE(TraitName, InterfaceEnumeratorMacro) \
	const UE::UAF::ITraitInterface* TraitName::GetTraitInterface(UE::UAF::FTraitInterfaceUID InInterfaceUID) const \
	{ \
		InterfaceEnumeratorMacro(ANIM_NEXT_IMPL_GET_INTERFACE_IMPL_FOR_INTERFACE) \
		/* Forward to base implementation */ \
		return TraitSuper::GetTraitInterface(InInterfaceUID); \
	}

// Helper that handles the GetTraitInterfaces() details for each interface specified by the generator macro
#define ANIM_NEXT_IMPL_GET_INTERFACES_IMPL_FOR_INTERFACE(InterfaceName) InterfaceName::InterfaceUID,

// Implements GetTraitInterfaces()
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACES(TraitName, InterfaceEnumeratorMacro) \
	TConstArrayView<UE::UAF::FTraitInterfaceUID> TraitName::GetTraitInterfaces() const \
	{ \
		/* Thread safe cache initialization */ \
		static TArray<UE::UAF::FTraitInterfaceUID> CachedInterfaceList = FTrait::BuildTraitInterfaceList( \
			TraitSuper::GetTraitInterfaces(), \
			{ \
				InterfaceEnumeratorMacro(ANIM_NEXT_IMPL_GET_INTERFACES_IMPL_FOR_INTERFACE) \
			}); \
		return CachedInterfaceList; \
	}

// Implements GetTraitRequiredInterfaces()
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_REQUIRED_INTERFACES(TraitName, RequiredInterfaceEnumeratorMacro) \
	TConstArrayView<UE::UAF::FTraitInterfaceUID> TraitName::GetTraitRequiredInterfaces() const \
	{ \
		/* Thread safe cache initialization */ \
		static TArray<UE::UAF::FTraitInterfaceUID> CachedInterfaceList = FTrait::BuildTraitInterfaceList( \
			TraitSuper::GetTraitRequiredInterfaces(), \
			{ \
				RequiredInterfaceEnumeratorMacro(ANIM_NEXT_IMPL_GET_INTERFACES_IMPL_FOR_INTERFACE) \
			}); \
		return CachedInterfaceList; \
	}

namespace UE::UAF::Private
{
	// Helper to grab the event type from an event handler function signature
	template<typename HandlerEventType>
	struct EventHandlerTypeTrait;

	template<typename HandlerEventType>
	struct EventHandlerTypeTrait<ETraitStackPropagation(*)(FExecutionContext&, FTraitBinding&, HandlerEventType&)>
	{
		using EventType = std::remove_const_t<HandlerEventType>;
	};

	template<typename HandlerEventType>
	struct EventHandlerTypeTrait<ETraitStackPropagation(*)(const FExecutionContext&, FTraitBinding&, HandlerEventType&)>
	{
		using EventType = std::remove_const_t<HandlerEventType>;
	};

	template<typename BaseType, typename HandlerEventType>
	struct EventHandlerTypeTrait<ETraitStackPropagation(BaseType::*)(FExecutionContext&, FTraitBinding&, HandlerEventType&) const>
	{
		using EventType = std::remove_const_t<HandlerEventType>;
	};

	template<typename BaseType, typename HandlerEventType>
	struct EventHandlerTypeTrait<ETraitStackPropagation(BaseType::*)(const FExecutionContext&, FTraitBinding&, HandlerEventType&) const>
	{
		using EventType = std::remove_const_t<HandlerEventType>;
	};

	template<class BaseType>
	struct TPaddedDerivedType : public BaseType { uint8 Dummy; };

	// Helper to find the effective size of a struct
	// If a struct contains no members, its size as defined by C++ is 1 byte to allow for pointer arithmetic
	// However, if we don't use pointer arithmetic (e.g. have them in an array), then we can avoid this waste
	// by computing its effective size. The effective size of a struct is its size when you derive from it.
	// A struct that has no members ends up with an effective size of zero bytes.
	template<class StructType>
	struct TGetEffectiveSize
	{
		// The effective size of the specified struct
		static constexpr uint32 Value = sizeof(TPaddedDerivedType<StructType>) == sizeof(uint8) ? 0 : sizeof(StructType);
	};
}

// Helper that handles the OnTraitEvent() details for each event specified by the generator macro
#define ANIM_NEXT_IMPL_ON_TRAIT_EVENT_IMPL_FOR_EVENT(EventHandler) \
	if (EventUID == UE::UAF::Private::EventHandlerTypeTrait<decltype(&EventHandler)>::EventType::TypeUID) \
	{ \
		return EventHandler(Context, Binding, static_cast<UE::UAF::Private::EventHandlerTypeTrait<decltype(&EventHandler)>::EventType&>(Event)); \
	}

// Implements OnTraitEvent()
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_ON_TRAIT_EVENT(TraitName, EventEnumeratorMacro) \
	UE::UAF::ETraitStackPropagation TraitName::OnTraitEvent(UE::UAF::FExecutionContext& Context, UE::UAF::FTraitBinding& Binding, FAnimNextTraitEvent& Event) const \
	{ \
		const UE::UAF::FTraitEventUID EventUID = Event.GetTypeUID(); \
		EventEnumeratorMacro(ANIM_NEXT_IMPL_ON_TRAIT_EVENT_IMPL_FOR_EVENT) \
		/* Forward to base implementation */ \
		return TraitSuper::OnTraitEvent(Context, Binding, Event); \
	}

// Helper that handles the GetTraitEvents() details for each event specified by the generator macro
#define ANIM_NEXT_IMPL_GET_TRAIT_EVENTS_IMPL_FOR_EVENT(EventHandler) \
	UE::UAF::Private::EventHandlerTypeTrait<decltype(&EventHandler)>::EventType::TypeUID,

// Implements GetTraitEvents()
#define ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_TRAIT_EVENTS(TraitName, EventEnumeratorMacro) \
	TConstArrayView<UE::UAF::FTraitEventUID> TraitName::GetTraitEvents() const \
	{ \
		/* Thread safe cache initialization */ \
		static TArray<UE::UAF::FTraitEventUID> CachedEventList = FTrait::BuildTraitEventList( \
			TraitSuper::GetTraitEvents(), \
			{ \
				EventEnumeratorMacro(ANIM_NEXT_IMPL_GET_TRAIT_EVENTS_IMPL_FOR_EVENT) \
			}); \
		return CachedEventList; \
	}

// A dummy trait interface generator for traits that do not implement any interfaces
#define NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro)

// A dummy trait event generator for traits that do not handle any events
#define NULL_ANIM_TRAIT_EVENT_ENUMERATOR(GeneratorMacro)

/**
  * This macro defines the necessary boilerplate for implementing FTrait. See above for usage example.
  */
#define GENERATE_ANIM_TRAIT_IMPLEMENTATION(TraitName, InterfaceEnumeratorMacro, RequiredInterfaceEnumeratorMacro, EventEnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT(TraitName) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_LATENT_PROPERTY_MEMORY_LAYOUT(TraitName) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_IS_PROPERTY_LATENT(TraitName) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACE(TraitName, InterfaceEnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACES(TraitName, InterfaceEnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_REQUIRED_INTERFACES(TraitName, RequiredInterfaceEnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_ON_TRAIT_EVENT(TraitName, EventEnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_TRAIT_EVENTS(TraitName, EventEnumeratorMacro)

// Allows a trait to auto-register and unregister within the current execution scope
// The trait must be found in the current scope without a namespace qualification
#define AUTO_REGISTER_ANIM_TRAIT(TraitName) \
	UE::UAF::FTraitStaticInitHook TraitName##Hook( \
		[](void* DestPtr, UE::UAF::FTraitMemoryLayout& MemoryDesc) -> UE::UAF::FTrait* \
		{ \
			MemoryDesc = TraitName::TraitMemoryDescription; \
			return DestPtr != nullptr ? new(DestPtr) TraitName() : nullptr; \
		});

namespace UE::UAF
{
	struct FTrait;
	struct FTraitBinding;
	struct FTraitMemoryLayout;
	class FTraitReader;
	class FTraitWriter;
	struct FExecutionContext;

	// A function pointer to a shim to construct a trait into the desired memory location
	// When called with a nullptr DestPtr, the function returns nullptr and only populates the
	// memory description output argument. This allows the caller to determine how much space
	// to reserve and how to properly align it. This is similar in spirit to various Windows SDK functions.
	using TraitConstructorFunc = FTrait * (*)(void* DestPtr, FTraitMemoryLayout& MemoryDesc);

	/**
	 * FTraitMemoryLayout
	 * 
	 * Encapsulates size/alignment details for a trait.
	 */
	struct FTraitMemoryLayout
	{
		// The size in bytes of an instance of the trait class which derives from FTrait
		uint32 TraitSize = 0;

		// The alignment in bytes of an instance of the trait class which derives from FTrait
		uint32 TraitAlignment = 1;

		// The size in bytes of the shared data for the trait which derives from FAnimNextTraitSharedData
		uint32 SharedDataSize = 0;

		// The alignment in bytes of the shared data for the trait which derives from FAnimNextTraitSharedData
		uint32 SharedDataAlignment = 1;

		// The size in bytes of the instance data for the trait which derives from FTraitInstanceData
		uint32 InstanceDataSize = 0;

		// The alignment in bytes of the instance data for the trait which derives from FTraitInstanceData
		uint32 InstanceDataAlignment = 1;
	};

	/**
	 * FTraitLatentPropertyMemoryLayout
	 *
	 * Encapsulates size/alignment details for a latent property.
	 */
	struct FTraitLatentPropertyMemoryLayout
	{
		// The size in bytes of the latent property
		uint32 Size = 0;

		// The alignment in bytes of the latent property
		uint32 Alignment = 1;

		// The latent property index that matches the enumerator macro, @see FTrait::GetLatentPropertyIndex
		int32 LatentPropertyIndex = INDEX_NONE;
	};

	/**
	 * FTrait
	 * 
	 * Base class for all traits.
	 * A trait can implement any number of interfaces based on ITraitInterface.
	 * A trait may derive from another trait.
	 * A trait should implement GetInterface(..) and test against the interfaces that it supports.
	 * 
	 * Traits should NOT have any internal state, hence why all API functions are 'const'.
	 * The reason for this is that at runtime, a single instance of every trait exists.
	 * That single instance is used by all instances of a trait on a node and concurrently
	 * on all worker threads.
	 * 
	 * Traits can have shared read-only data that all instances of a graph can use (e.g. hard-coded properties).
	 * Shared data must derive from FAnimNextTraitSharedData.
	 * Traits can have instance data (e.g. blend weight).
	 * Instance data must derive from FTraitInstanceData.
	 */
	struct FTrait
	{
		virtual ~FTrait() {}

		// Empty shared/instance data types
		// Derived types must define an alias for these
		using FSharedData = FAnimNextTraitSharedData;			// Trait shared data must derive from FAnimNextTraitSharedData
		using FInstanceData = FTraitInstanceData;				// Trait instance data must derive from FTraitInstanceData

		// The globally unique UID for this trait
		// Derived types will have their own TraitUID member that hides/aliases/shadows this one
		// @see DECLARE_ANIM_TRAIT
		static constexpr FTraitUID TraitUID = FTraitUID::MakeUID(TEXT("FTrait"));

		// Returns the globally unique UID for this trait
		virtual FTraitUID GetTraitUID() const { return TraitUID; };

		// Returns the trait name
		virtual FString GetTraitName() const { return TEXT("FTrait"); }

		// Returns the memory requirements of the derived trait instance
		virtual FTraitMemoryLayout GetTraitMemoryDescription() const = 0;

		// Returns the UScriptStruct associated with the shared data for the trait
		virtual UScriptStruct* GetTraitSharedDataStruct() const { return FSharedData::StaticStruct(); }

		// Called when a new instance of the trait is created or destroyed
		// Derived types must override this and forward to the instance data constructor/destructor
		virtual void ConstructTraitInstance(const FExecutionContext& Context, const FTraitBinding& Binding) const = 0;
		virtual void DestructTraitInstance(const FExecutionContext& Context, const FTraitBinding& Binding) const = 0;

		// Returns the trait mode.
		virtual ETraitMode GetTraitMode() const = 0;

		// Returns a pointer to the specified interface if it is supported.
		// Derived types must override this.
		virtual const ITraitInterface* GetTraitInterface(FTraitInterfaceUID InterfaceUID) const
		{
			// TODO:
			// if/else sequence with static_casts to get the right v-table
			// could be implemented with two tables: one of UIDs, another with matching offsets to 'this'
			// we could scan the first table with SIMD, 4x UIDs at a time with 'cmpeq' to generate a mask
			// we can mode the mask into a general register, if non-zero, we have a match
			// using the mask, we can easily compute the UID offset in our 4x entry by counting leading/trailing zeroes
			// using the UID offset, we can load and apply the correct offset
			// may or may not be faster, but it shifts the burden from code cache to data cache and we can better control locality
			// we could store the tables contiguous with one another, offsets could be 16 bit or maybe 8 bit (multiple of pointer size)
			// we could store the tables contiguous with the tables of other traits for better cache locality
			// by using tables, it means the lookup code can live in a single place and remain hot
			// it means we can test 4x UIDs at a time, or interleave and test 8x or 16x
			// it means we can quickly early out if none of the interfaces match (common case?) since we don't need to test
			// all of them one by one
			// SIMD code path also opens the door for cheap bulk interface queries where we query up to 4x interface UIDs and
			// return 4x interface offsets (caller can generate pointers easily)

			// Base class doesn't implement any interfaces
			// Derived types must implement this
			return nullptr;
		}

		// Returns a list of interfaces that this trait supports
		virtual TConstArrayView<FTraitInterfaceUID> GetTraitInterfaces() const { return TConstArrayView<FTraitInterfaceUID>(); }

		// Returns a list of interfaces that this trait reqquires
		virtual TConstArrayView<FTraitInterfaceUID> GetTraitRequiredInterfaces() const { return TConstArrayView<FTraitInterfaceUID>(); }

		// Called when an event reaches an instance of this trait
		virtual ETraitStackPropagation OnTraitEvent(FExecutionContext& Context, FTraitBinding& Binding, FAnimNextTraitEvent& Event) const { return ETraitStackPropagation::Continue; }

		// Returns a list of events that this trait handles
		virtual TConstArrayView<FTraitEventUID> GetTraitEvents() const { return TConstArrayView<FTraitEventUID>(); }

		// The number of latent property properties in the shared data of this trait
		virtual uint32 GetNumLatentTraitProperties() const { return 0; }

		// Returns the memory layout of the specified latent property
		virtual FTraitLatentPropertyMemoryLayout GetLatentPropertyMemoryLayout(const FAnimNextTraitSharedData& SharedData, FName PropertyName, uint32 PropertyIndex) const { return FTraitLatentPropertyMemoryLayout(); }

		// Called to serialize trait shared data
		UE_API virtual void SerializeTraitSharedData(FArchive& Ar, FAnimNextTraitSharedData& SharedData) const;

#if WITH_EDITOR
		// Takes the editor properties as authored in the graph and converts them into an instance of the FAnimNextTraitSharedData
		// derived type using UE reflection.
		// Traits can override this function to control how editor only properties are coerced into the runtime shared data
		// instance.
		UE_API virtual void SaveTraitSharedData(const TFunction<FString(FName PropertyName)>& GetTraitProperty, FAnimNextTraitSharedData& OutSharedData) const;

		// Takes the trait shared data and the editor properties as authored in the graph and generates the latent property metadata using UE reflection.
		// Returns the number of latent property metadatas added to OutLatentPropertyHandles
		UE_API virtual uint32 GetLatentPropertyHandles(
			const FAnimNextTraitSharedData* InSharedData,
			TArray<FLatentPropertyMetadata>& OutLatentPropertyHandles,
			bool bFilterEditorOnly,
			const TFunction<uint16(FName PropertyName)>& GetTraitLatentPropertyIndex) const;

		// Makes the Trait Editor only display this Trait in Advanced view
		virtual bool IsHidden() const { return false; }

		// Whether or not this trait can be placed multiple times on a trait stack
		virtual bool MultipleInstanceSupport() const { return false; }

		// Enable Traits to generate TraitStack pins
		virtual void GetProgrammaticPins(FAnimNextTraitSharedData* InSharedData, URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, struct FRigVMPinInfoArray& OutPinArray) const {}
#endif

		// Takes the trait shared data and the editor properties as authored in the graph and generates the latent property metadata using UE reflection.
		// Returns the number of latent property metadatas added to OutLatentPropertyHandles
		uint32 GetVariableMappedLatentPropertyHandles(
			const FAnimNextTraitSharedData* InSharedData,
			TArray<FLatentPropertyMetadata>& OutLatentPropertyHandles,
			bool bFilterEditorOnly,
			const TFunction<uint16(FName PropertyName)>& GetTraitLatentPropertyIndex) const;

		// Returns whether or not the specified property has been marked as latent
		bool IsPropertyLatent(const FAnimNextTraitSharedData& InSharedData, FName PropertyName) const { return GetLatentPropertyIndex(InSharedData, PropertyName) != INDEX_NONE; }

		// Returns the latent property index or INDEX_NONE if the property isn't found or if it isn't latent
		// WARNING: Latent property handles are in the order defined by the enumerator macro within the shared data,
		// @see FAnimNextTraitSharedData::GetLatentPropertyIndex
		virtual int32 GetLatentPropertyIndex(const FAnimNextTraitSharedData& InSharedData, FName PropertyName) const { return INDEX_NONE; }

		// Get any upgrade info needed (e.g. to resolve deprecation)
		UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const;

	protected:
		// Implements GetLatentPropertyMemoryLayout() by allowing a map to be provided for caching purposes to speed up repeated queries
		UE_API FTraitLatentPropertyMemoryLayout GetLatentPropertyMemoryLayoutImpl(
			const FAnimNextTraitSharedData& SharedData,
			FName PropertyName,
			uint32 PropertyIndex,
			TArray<FTraitLatentPropertyMemoryLayout>& LatentPropertyMemoryLayouts) const;

		// Builds a list of interfaces with the provided super interfaces and current interfaces as an initializer list
		static UE_API TArray<FTraitInterfaceUID> BuildTraitInterfaceList(
			const TConstArrayView<FTraitInterfaceUID>& SuperInterfaces,
			std::initializer_list<FTraitInterfaceUID> InterfaceList);

		// Builds a list of events with the provided super events and current events as an initializer list
		static UE_API TArray<FTraitEventUID> BuildTraitEventList(
			const TConstArrayView<FTraitEventUID>& SuperEvents,
			std::initializer_list<FTraitEventUID> EventList);
	};

	// Base class for base traits that are standalone
	struct FBaseTrait : FTrait
	{
		DECLARE_ABSTRACT_ANIM_TRAIT(FBaseTrait, FTrait)

		virtual ETraitMode GetTraitMode() const override { return ETraitMode::Base; }
	};

	// Base class for additive traits that override behavior of other traits
	struct FAdditiveTrait : FTrait
	{
		DECLARE_ABSTRACT_ANIM_TRAIT(FAdditiveTrait, FTrait)

		virtual ETraitMode GetTraitMode() const override { return ETraitMode::Additive; }

#if WITH_EDITOR
		virtual bool MultipleInstanceSupport() const override { return true; }
#endif
	};

	/**
	 * FTraitStaticInitHook
	 *
	 * Allows traits to automatically register/unregister within the current scope.
	 * This can be used during static init.
	 */
	struct FTraitStaticInitHook final
	{
		UE_API explicit FTraitStaticInitHook(TraitConstructorFunc InTraitConstructor);
		UE_API ~FTraitStaticInitHook();

	private:
		TraitConstructorFunc TraitConstructor;
	};
}

#undef UE_API
