// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitPtr.h"
#include "TraitCore/TraitInterfaceUID.h"
#include "TraitCore/TraitStackBinding.h"
#include "TraitCore/TraitTemplate.h"

#include <type_traits>

#define UE_API UAFANIMGRAPH_API

struct FAnimNextTraitSharedData;

namespace UE::UAF
{
	struct FExecutionContext;
	struct FTraitInstanceData;
	struct ITraitInterface;
	struct FNodeDescription;

	namespace Private
	{
		struct FScopedInterfaceEntry;
	}

	/**
	 * FTraitBinding
	 * 
	 * Base class for all trait bindings.
	 * A trait binding contains untyped data about a specific trait instance.
	 */
	struct FTraitBinding
	{
		// Creates an empty/invalid binding.
		FTraitBinding() = default;

		// Returns whether or not this binding is valid.
		bool IsValid() const { return TraitImpl != nullptr; }

		// Resets the trait binding to an invalid state.
		void Reset()
		{
			new (this) FTraitBinding();
		}

		// Queries a node for a pointer to its trait shared data.
		// If the trait handle is invalid, a null pointer is returned.
		template<class SharedDataType>
		const SharedDataType* GetSharedData() const
		{
			static_assert(std::is_base_of<FAnimNextTraitSharedData, SharedDataType>::value, "Trait shared data must derive from FAnimNextTraitSharedData");

			if (!IsValid())
			{
				return nullptr;
			}

			const FTraitTemplate* TraitDescs = Stack->NodeTemplate->GetTraits();
			const FTraitTemplate* TraitTemplate = TraitDescs + TraitIndex;

			const SharedDataType* SharedData = static_cast<const SharedDataType*>(TraitTemplate->GetTraitDescription(*Stack->NodeDescription));
			check(IsAligned(SharedData, alignof(SharedDataType)));
			return SharedData;
		}

		// Queries a node for a pointer to its trait instance data.
		// If the trait handle is invalid, a null pointer is returned.
		template<class InstanceDataType>
		InstanceDataType* GetInstanceData() const
		{
			static_assert(std::is_base_of<FTraitInstanceData, InstanceDataType>::value, "Trait instance data must derive from FTraitInstanceData");

			if (!IsValid())
			{
				return nullptr;
			}

			const FTraitTemplate* TraitDescs = Stack->NodeTemplate->GetTraits();
			const FTraitTemplate* TraitTemplate = TraitDescs + TraitIndex;

			return static_cast<InstanceDataType*>(TraitTemplate->GetTraitInstance(*Stack->NodeInstance));
		}

		// Queries a node for a pointer to its trait latent properties.
		// If the trait handle is invalid or if we have no latent properties, a null pointer is returned.
		// WARNING: Latent property handles are in the order defined by the enumerator macro within the shared data,
		// @see FAnimNextTraitSharedData::GetLatentPropertyIndex
		const FLatentPropertyHandle* GetLatentPropertyHandles() const
		{
			if (!IsValid())
			{
				return nullptr;
			}

			const FTraitTemplate* TraitDescs = Stack->NodeTemplate->GetTraits();
			const FTraitTemplate* TraitTemplate = TraitDescs + TraitIndex;

			if (!TraitTemplate->HasLatentProperties())
			{
				return nullptr;
			}

			return TraitTemplate->GetTraitLatentPropertyHandles(*Stack->NodeDescription);
		}

		// Returns a pointer to the latent property specified by the provided handle or nullptr if the binding/handle are invalid
		template<typename PropertyType>
		const PropertyType* GetLatentProperty(FLatentPropertyHandle Handle) const
		{
			if (!IsValid())
			{
				return nullptr;
			}

			if (!Handle.IsOffsetValid())
			{
				return nullptr;
			}

			const uint8* NodeInstance = (const uint8*)Stack->NodeInstance;
			return (const PropertyType*)(NodeInstance + Handle.GetLatentPropertyOffset());
		}

		// Queries the trait stack for a trait that implements the specified interface.
		// If no such trait exists, false is returned.
		template<class TraitInterface>
		bool GetStackInterface(TTraitBinding<TraitInterface>& OutBinding) const
		{
			return IsValid() && Stack->GetInterface<TraitInterface>(OutBinding);
		}

		// Queries the trait stack for a trait lower on the stack that implements the specified interface.
		// If no such trait exists, false is returned.
		template<class TraitInterface>
		bool GetStackInterfaceSuper(TTraitBinding<TraitInterface>& OutSuperBinding) const
		{
			return IsValid() && Stack->GetInterfaceSuper<TraitInterface>(*this, OutSuperBinding);
		}

		// Queries the current trait for a new interface.
		// If no such interface is found, false it returned.
		template<class TraitInterface>
		bool AsInterface(TTraitBinding<TraitInterface>& OutBinding) const
		{
			static_assert(std::is_base_of<ITraitInterface, TraitInterface>::value, "TraitInterface type must derive from ITraitInterface");

			constexpr FTraitInterfaceUID InterfaceUID = TraitInterface::InterfaceUID;
			return AsInterfaceImpl(InterfaceUID, OutBinding);
		}

		// Queries the current trait for the presence of the specified interface.
		// If no such interface is found, false it returned.
		template<class TraitInterface>
		bool HasInterface() const
		{
			static_assert(std::is_base_of<ITraitInterface, TraitInterface>::value, "TraitInterface type must derive from ITraitInterface");

			constexpr FTraitInterfaceUID InterfaceUID = TraitInterface::InterfaceUID;
			return HasInterfaceImpl(InterfaceUID);
		}

		// Queries the trait stack for the presence of a trait that implements the specified interface.
		// If no such trait exists, false is returned.
		template<class TraitInterface>
		bool HasStackInterface() const
		{
			return IsValid() && Stack->HasInterface<TraitInterface>();
		}

		// Queries the trait stack for the presence of a trait lower on the stack that implements the specified interface.
		// If no such trait exists, false is returned.
		template<class TraitInterface>
		bool HasStackInterfaceSuper() const
		{
			return IsValid() && Stack->HasInterfaceSuper<TraitInterface>(*this);
		}

		// Returns the trait stack binding backing this trait binding.
		const FTraitStackBinding* GetStack() const { return Stack; }

		// Returns a trait binding to the base of the stack.
		// Returns true on success, false otherwise.
		bool GetStackBaseTrait(FTraitBinding& OutBinding) const
		{
			return IsValid() && Stack->GetBaseTrait(OutBinding);
		}

		// Returns the trait pointer we are bound to.
		FWeakTraitPtr GetTraitPtr() const { return FWeakTraitPtr(Stack != nullptr ? Stack->NodeInstance : nullptr, TraitIndex); }

		// Returns the trait index on the stack we are bound to.
		uint32 GetTraitIndex() const { return Stack != nullptr ? (TraitIndex - Stack->BaseTraitIndex) : 0; }

		// Returns a pointer to the trait implementation if valid and if the trait is present at runtime
		const FTrait* GetTrait() const { return TraitImpl; }

		// Returns the trait interface UID when bound, an invalid UID otherwise.
		UE_API FTraitInterfaceUID GetInterfaceUID() const;

		// Equality and inequality tests
		bool operator==(const FTraitBinding& RHS) const { return Stack == RHS.Stack && TraitIndex == RHS.TraitIndex && InterfaceThisOffset == RHS.InterfaceThisOffset; }
		bool operator!=(const FTraitBinding& RHS) const { return !operator==(RHS); }

	protected:
		FTraitBinding(const FTraitStackBinding* InStack, const FTrait* InTraitImpl, uint32 InTraitIndex, int32 InInterfaceThisOffset = -1)
			: Stack(InStack)
			, TraitImpl(InTraitImpl)
			, TraitIndex(InTraitIndex)
			, InterfaceThisOffset(InInterfaceThisOffset)
		{
		}

		// Performs a naked cast to the desired interface type
		template<class TraitInterfaceType>
		const TraitInterfaceType* GetInterfaceTyped() const
		{
			static_assert(std::is_base_of<ITraitInterface, TraitInterfaceType>::value, "Trait interface data must derive from ITraitInterface");
			check(InterfaceThisOffset != -1);
			return reinterpret_cast<const TraitInterfaceType*>(reinterpret_cast<const uint8*>(TraitImpl) + InterfaceThisOffset);
		}

		bool AsInterfaceImpl(FTraitInterfaceUID InterfaceUID, FTraitBinding& OutBinding) const
		{
			if (!IsValid())
			{
				return false;
			}

			if (const ITraitInterface* NewInterface = TraitImpl->GetTraitInterface(InterfaceUID))
			{
				const int32 NewInterfaceThisOffset = reinterpret_cast<const uint8*>(NewInterface) - reinterpret_cast<const uint8*>(TraitImpl);

				OutBinding = FTraitBinding(Stack, TraitImpl, TraitIndex, NewInterfaceThisOffset);
				return true;
			}

			return false;
		}

		bool HasInterfaceImpl(FTraitInterfaceUID InterfaceUID) const
		{
			if (!IsValid())
			{
				return false;
			}

			return TraitImpl->GetTraitInterface(InterfaceUID) != nullptr;
		}

		// A pointer to the bound trait stack or nullptr if we are not bound
		const FTraitStackBinding* Stack = nullptr;

		// A pointer to the trait implementation from which interfaces are found
		// Can be null if the trait is present at runtime but the trait implementation hasn't been registered/found
		const FTrait* TraitImpl = nullptr;

		// The index of the trait on the bound stack relative to the base of the stack (not relative to the base of the node)
		uint32 TraitIndex = 0;

		// The offset of the typed interface we are bound to or -1 if we are untyped
		// The offset is relative to the trait implementation pointer
		int32 InterfaceThisOffset = -1;

		friend FExecutionContext;
		friend FTraitStackBinding;
		friend Private::FScopedInterfaceEntry;
	};

	/**
	 * TTraitBinding
	 * 
	 * A templated proxy for trait interfaces. It is meant to be specialized per interface
	 * in order to allow a clean API and avoid human error. It wraps the necessary information
	 * to bind a trait to a specific interface. See existing interfaces for examples.
	 * 
	 * Here, we forward declare the template which every interface must specialize. Because we
	 * rely on specialization, it must be defined within the UE::UAF namespace where the
	 * declaration exists.
	 * 
	 * Specializations must derive from FTraitBinding to provide the necessary machinery.
	 * 
	 * @see IUpdate, IEvaluate, IHierarchy
	 */
	template<class TraitInterfaceType>
	struct TTraitBinding;
}

#undef UE_API
