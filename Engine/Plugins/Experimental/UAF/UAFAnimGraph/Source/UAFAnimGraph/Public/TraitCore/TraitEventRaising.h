// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FAnimNextTraitEvent;
struct FUAFGraphInstanceComponent;

namespace UE::UAF
{
	struct FExecutionContext;
	struct FTraitStackBinding;
	struct FTraitEventList;

	// Raises a trait event on the specified trait stack
	// This calls OnTraitEvent on each trait of the stack following propagation rules
	UAFANIMGRAPH_API void RaiseTraitEvent(FExecutionContext& Context, const FTraitStackBinding& StackBinding, FAnimNextTraitEvent& Event);

	// Raises every trait event from the input list on the specified trait stack
	// This calls OnTraitEvent on each trait of the stack following propagation rules
	UAFANIMGRAPH_API void RaiseTraitEvents(FExecutionContext& Context, const FTraitStackBinding& StackBinding, const FTraitEventList& EventList);

	// Raises every trait event from the input list on the specified component
	UAFANIMGRAPH_API void RaiseTraitEvents(FExecutionContext& Context, FUAFGraphInstanceComponent& Component, const FTraitEventList& EventList);
}
