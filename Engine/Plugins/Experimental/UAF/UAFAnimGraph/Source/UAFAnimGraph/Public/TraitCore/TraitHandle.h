// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/NodeHandle.h"

#include "TraitHandle.generated.h"

/**
 * Trait Handle
 * A trait handle represents a reference to a specific trait instance in the shared/read-only portion
 * of a sub-graph. It points to a FNodeDescription when resolved.
 * 
 * Internally, it contains a node handle and a trait index.
 * Trait handles can only be used within an AnimNext graph serialized with FTraitWriter.
 * 
 * @see FNodeDescription, FNodeHandle
 */
USTRUCT(BlueprintType)
struct FAnimNextTraitHandle final
{
	GENERATED_BODY()

	// Creates an invalid trait handle
	constexpr FAnimNextTraitHandle() noexcept
		: PackedTraitIndexAndNodeHandle(UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE)
	{}

	// Creates a trait handle pointing to the first trait of the specified node
	explicit constexpr FAnimNextTraitHandle(UE::UAF::FNodeHandle NodeHandle) noexcept
		: PackedTraitIndexAndNodeHandle(NodeHandle.GetPackedValue() & NODE_HANDLE_MASK)
	{}

	// Creates a trait handle pointing to the specified trait on the specified node
	FAnimNextTraitHandle(UE::UAF::FNodeHandle NodeHandle, uint32 TraitIndex)
		: PackedTraitIndexAndNodeHandle((NodeHandle.GetPackedValue() & NODE_HANDLE_MASK) | (TraitIndex << 24))
	{
		check(TraitIndex <= MAX_uint8);	// Make sure we don't truncate
	}

	// Returns true if this trait handle is valid, false otherwise
	constexpr bool IsValid() const noexcept { return GetNodeHandle().IsValid(); }

	// Returns the trait index
	constexpr uint32 GetTraitIndex() const noexcept { return PackedTraitIndexAndNodeHandle >> TRAIT_INDEX_SHIFT; }

	// Returns a handle to the node referenced
	constexpr UE::UAF::FNodeHandle GetNodeHandle() const noexcept { return UE::UAF::FNodeHandle::FromPackedValue(PackedTraitIndexAndNodeHandle); }

	UAFANIMGRAPH_API bool Serialize(class FArchive& Ar);

private:
	// Bottom 24 bits are used by the node handle while the top 8 bits by the trait index
	static constexpr uint32 TRAIT_INDEX_SHIFT = 24;
	static constexpr uint32 NODE_HANDLE_MASK = ~0u >> (32 - TRAIT_INDEX_SHIFT);

	UPROPERTY()
	uint32		PackedTraitIndexAndNodeHandle;
};

template<>
struct TStructOpsTypeTraits<FAnimNextTraitHandle> : public TStructOpsTypeTraitsBase2<FAnimNextTraitHandle>
{
	enum
	{
		WithSerializer = true,
	};
};
