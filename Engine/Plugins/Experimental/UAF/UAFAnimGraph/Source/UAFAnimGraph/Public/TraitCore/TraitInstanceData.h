// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitPtr.h"

namespace UE::UAF
{
	struct FTraitBinding;
	struct FExecutionContext;

	/**
	 * FTraitInstanceData
	 * A trait instance represents allocated data for specific trait instance.
	 * 
	 * @see FNodeInstance
	 * 
	 * A FTraitInstanceData is the base type that trait instance data derives from.
	 */
	struct FTraitInstanceData
	{
		// Called after the constructor has been called when a new instance is created.
		// This is called after the default constructor.
		// You can override this function by adding a new one with the same name on your
		// derived type.
		// Traits are constructed from the bottom to the top.
		void Construct(const FExecutionContext& Context, const FTraitBinding& Binding) noexcept
		{
		}

		// Called before the destructor has been called when an instance is destroyed.
		// This is called before the default destructor.
		// You can override this function by adding a new one with the same name on your
		// derived type.
		// Traits are destructed from the top to the bottom.
		void Destruct(const FExecutionContext& Context, const FTraitBinding& Binding) noexcept
		{
		}
	};
}
