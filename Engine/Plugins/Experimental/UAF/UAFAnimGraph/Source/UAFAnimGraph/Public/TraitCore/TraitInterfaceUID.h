// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/Fnv.h"

namespace UE::UAF
{
	// Type alias for a raw trait UID, not typesafe
	using FTraitInterfaceUIDRaw = uint32;

	/**
	 * FTraitInterfaceUID
	 *
	 * Encapsulates an interface global UID.
	 *
	 * The whole struct is meant to be 'constexpr' to allow inlining within the assembly as constants.
	 */
	struct FTraitInterfaceUID final
	{
		// Constructs an invalid UID
		constexpr FTraitInterfaceUID()
			: UID(INVALID_UID)
#if !UE_BUILD_SHIPPING || WITH_EDITOR
			, InterfaceName(TEXT("<Invalid Interface UID>"))
#endif
		{}

		// Constructs an interface UID from its raw value
		explicit constexpr FTraitInterfaceUID(FTraitInterfaceUIDRaw InUID)
			: UID(InUID)
#if !UE_BUILD_SHIPPING || WITH_EDITOR
			, InterfaceName(TEXT("<Unknown Trait Interface Name>"))
#endif
		{
		}

		// Constructs a trait UID from a string literal
		template<typename CharType, int32 Len>
		static UE_CONSTEVAL FTraitInterfaceUID MakeUID(const CharType(&InInterfaceName)[Len]) noexcept
		{
			return FTraitInterfaceUID(UE::HashStringFNV1a<FTraitInterfaceUIDRaw>(MakeStringView(InInterfaceName, Len - 1)), InInterfaceName);
		}

#if !UE_BUILD_SHIPPING || WITH_EDITOR
		// Returns a string literal to the interface name
		constexpr const TCHAR* GetInterfaceName() const { return InterfaceName; }
#endif

		// Returns the interface global UID
		constexpr FTraitInterfaceUIDRaw GetUID() const { return UID; }

		// Returns whether this UID is valid or not
		constexpr bool IsValid() const { return UID != INVALID_UID; }

	private:
		static constexpr FTraitInterfaceUIDRaw INVALID_UID = 0;

		// Constructs an interface UID
		explicit constexpr FTraitInterfaceUID(FTraitInterfaceUIDRaw InUID, const TCHAR* InInterfaceName) noexcept
			: UID(InUID)
#if !UE_BUILD_SHIPPING || WITH_EDITOR
			, InterfaceName(InInterfaceName)
#endif
		{
		}

		FTraitInterfaceUIDRaw		UID;

#if !UE_BUILD_SHIPPING || WITH_EDITOR
		const TCHAR*				InterfaceName = nullptr;
#endif
	};

	// Compares for equality and inequality
	constexpr bool operator==(FTraitInterfaceUID LHS, FTraitInterfaceUID RHS) { return LHS.GetUID() == RHS.GetUID(); }
	constexpr bool operator!=(FTraitInterfaceUID LHS, FTraitInterfaceUID RHS) { return LHS.GetUID() != RHS.GetUID(); }

	// For sorting
	constexpr bool operator<(FTraitInterfaceUID LHS, FTraitInterfaceUID RHS) { return LHS.GetUID() < RHS.GetUID(); }
}
