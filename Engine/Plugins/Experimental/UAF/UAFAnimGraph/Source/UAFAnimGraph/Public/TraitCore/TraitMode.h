// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	/**
	 * ETraitMode
	 *
	 * Describes how a trait behaves once attached to an animation node.
	 */
	enum class ETraitMode
	{
		/**
		 * Invalid value is used for default variable initialization
		 */
		Invalid = 0,

		/**
		 * Base traits can live on their own in an animation node and have no 'Super'.
		 * As a result, calls to GetInterface() do not forward to other traits below
		 * them on the node stack. Multiple base traits can exist in a single animation
		 * node but they behave as independent nodes (functionally speaking).
		 */
		Base,

		/**
		 * Additive traits override or augment behavior on prior traits on the node stack.
		 * At least one base trait must be present for an additive trait to be added
		 * on top. Calls to GetInterface() will pass-through to other traits below on the stack
		 * until a base trait is found.
		 */
		Additive,

		/**
		 * Max value is used for iteration / static checks
		 */
		Num = Additive
	};
}
