// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FNodeInstance;
	struct FWeakTraitPtr;

	/**
	 * Trait Pointer
	 * A trait pointer represents a shared pointer to allocated instance data.
	 * It manages reference counting.
	 * It points to a FTraitInstanceData or FNodeInstance when resolved.
	 *
	 * A node pointer can also be weak, meaning that it does not update reference counting.
	 * Note that weak pointers will point to garbage if the trait instance is destroyed.
	 *
	 * Weak trait pointers should be used carefully!
	 * 
	 * @see FNodeInstance, FTraitInstanceData
	 */
	struct FTraitPtr
	{
		// Constructs an invalid pointer handle
		constexpr FTraitPtr() noexcept = default;

		// Constructs a pointer handle to the provided instance
		UE_API FTraitPtr(FNodeInstance* InNodeInstance, uint32 InTraitIndex);

		UE_API FTraitPtr(const FTraitPtr& TraitPtr);
		UE_API explicit FTraitPtr(const FWeakTraitPtr& TraitPtr);
		UE_API FTraitPtr(FTraitPtr&& TraitPtr) noexcept;
		UE_API ~FTraitPtr();

		UE_API FTraitPtr& operator=(const FWeakTraitPtr& TraitPtr);
		UE_API FTraitPtr& operator=(const FTraitPtr& TraitPtr);
		UE_API FTraitPtr& operator=(FTraitPtr&& TraitPtr) noexcept;

		// Returns a pointer to the node instance
		FNodeInstance* GetNodeInstance() const noexcept { return reinterpret_cast<FNodeInstance*>(PackedPointerAndFlags & ~FLAGS_MASK); }

		// Returns the trait index this pointer handle references
		constexpr uint32 GetTraitIndex() const noexcept { return TraitIndex; }

		// Returns true when the pointer is valid, false otherwise
		constexpr bool IsValid() const noexcept { return PackedPointerAndFlags != 0; }

		// Returns true if this pointer handle is weak, false otherwise
		constexpr bool IsWeak() const noexcept { return (PackedPointerAndFlags & IS_WEAK_BIT) != 0; }

		// Clears the handle and renders it invalid
		UE_API void Reset();

		// Equality and inequality tests
		bool operator==(const FTraitPtr& RHS) const { return GetNodeInstance() == RHS.GetNodeInstance() && TraitIndex == RHS.TraitIndex; }
		bool operator!=(const FTraitPtr& RHS) const { return GetNodeInstance() != RHS.GetNodeInstance() || TraitIndex != RHS.TraitIndex; }
		bool operator==(const FWeakTraitPtr& RHS) const;
		bool operator!=(const FWeakTraitPtr& RHS) const;

	private:
		// Various flags stored in the pointer alignment bits, assumes an alignment of at least 4 bytes
		enum EFlags : uintptr_t
		{
			// When true, this handle is weak and it will not update the instance reference count
			IS_WEAK_BIT = 0x01,

			// Bit mask to clear all flags from the packed pointer value
			FLAGS_MASK = 0x03,
		};

		// Constructs a pointer handle to the provided instance
		UE_API FTraitPtr(FNodeInstance* InNodeInstance, EFlags InFlags, uint32 InTraitIndex);

		// Packed pointer value contains a pointer along with flags in the alignment bits
		// The pointer part points to a node instance
		uintptr_t	PackedPointerAndFlags = 0;
		uint32		TraitIndex = 0;				// Only need 8 bits, but use 32 since we have ample padding, avoids truncation

		friend struct FWeakTraitPtr;
		friend struct FExecutionContext;
	};

	/**
	 * Weak Trait Pointer
	 * Same as a FTraitPtr but is strongly typed to be weak.
	 *
	 * Weak trait pointers should be used carefully!
	 * 
	 * @see FTraitPtr
	 */
	struct FWeakTraitPtr
	{
		// Constructs an invalid weak pointer handle
		constexpr FWeakTraitPtr() noexcept = default;

		// Constructs a weak pointer handle from a shared pointer handle
		FWeakTraitPtr(const FTraitPtr& TraitPtr) noexcept
			: NodeInstance(TraitPtr.GetNodeInstance())
			, TraitIndex(TraitPtr.TraitIndex)
		{}

		// Constructs a weak pointer handle to the provided instance
		UE_API FWeakTraitPtr(FNodeInstance* InNodeInstance, uint32 InTraitIndex);

		// Returns a pointer to the node instance
		constexpr FNodeInstance* GetNodeInstance() const noexcept { return NodeInstance; }

		// Returns the trait index this pointer handle references
		constexpr uint32 GetTraitIndex() const noexcept { return TraitIndex; }

		// Returns true when the pointer is valid, false otherwise
		constexpr bool IsValid() const noexcept { return NodeInstance != nullptr; }

		// Clears the handle and renders it invalid
		UE_API void Reset();

		// Equality and inequality tests
		bool operator==(const FWeakTraitPtr& RHS) const { return NodeInstance == RHS.NodeInstance && TraitIndex == RHS.TraitIndex; }
		bool operator!=(const FWeakTraitPtr& RHS) const { return NodeInstance != RHS.NodeInstance || TraitIndex != RHS.TraitIndex; }
		bool operator==(const FTraitPtr& RHS) const { return NodeInstance == RHS.GetNodeInstance() && TraitIndex == RHS.GetTraitIndex(); }
		bool operator!=(const FTraitPtr& RHS) const { return NodeInstance != RHS.GetNodeInstance() || TraitIndex != RHS.GetTraitIndex(); }

	private:
		FNodeInstance*	NodeInstance = nullptr;
		uint32			TraitIndex = 0;			// Only need 8 bits, but use 32 since we have ample padding, avoids truncation
	};

	//////////////////////////////////////////////////////////////////////////

	inline bool FTraitPtr::operator==(const FWeakTraitPtr& RHS) const { return GetNodeInstance() == RHS.GetNodeInstance() && TraitIndex == RHS.GetTraitIndex(); }
	inline bool FTraitPtr::operator!=(const FWeakTraitPtr& RHS) const { return GetNodeInstance() != RHS.GetNodeInstance() || TraitIndex != RHS.GetTraitIndex(); }
}

#undef UE_API
