// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	/**
	 * FTraitRegistryHandle
	 *
	 * Encapsulates a value used as a handle in the trait registry.
	 * When valid, it can be used to retrieve a pointer to the corresponding trait.
	 */
	struct FTraitRegistryHandle final
	{
		// Default constructed handles are invalid
		FTraitRegistryHandle() noexcept = default;

		// Returns whether or not this handle points to a valid trait
		bool IsValid() const noexcept { return HandleValue != 0; }

		// Returns whether or not this handle is valid and points to a static trait
		bool IsStatic() const noexcept { return IsValid() && HandleValue > 0; }

		// Returns whether or not this handle is valid and points to a dynamic trait
		bool IsDynamic() const noexcept { return IsValid() && HandleValue < 0; }

		// Returns the static buffer offset for this handle when valid, otherwise INDEX_NONE
		int32 GetStaticOffset() const noexcept { return IsStatic() ? (HandleValue - 1) : INDEX_NONE; }

		// Returns the dynamic array index for this handle when valid, otherwise INDEX_NONE
		int32 GetDynamicIndex() const noexcept { return IsDynamic() ? (-HandleValue - 1) : INDEX_NONE; }

		// Compares for equality and inequality
		bool operator==(FTraitRegistryHandle RHS) const noexcept { return HandleValue == RHS.HandleValue; }
		bool operator!=(FTraitRegistryHandle RHS) const noexcept { return HandleValue != RHS.HandleValue; }

	private:
		explicit FTraitRegistryHandle(int16 HandleValue_) noexcept
			: HandleValue(HandleValue_)
		{}

		// Creates a static handle based on a trait offset in the static buffer
		static FTraitRegistryHandle MakeStatic(int32 TraitOffset)
		{
			check(TraitOffset >= 0 && TraitOffset < (1 << 15));
			return FTraitRegistryHandle(TraitOffset + 1);
		}

		// Creates a dynamic handle based on a trait index in the dynamic array
		static FTraitRegistryHandle MakeDynamic(int32 TraitIndex)
		{
			// Convert the index from 0-based to 1-based since we reserve 0 for our invalid handle
			check(TraitIndex >= 0 && TraitIndex < (1 << 15));
			return FTraitRegistryHandle(-TraitIndex - 1);
		}

		// When 0, the handle is invalid
		// When positive, it is a 1-based offset in the registry's static buffer
		// When negative, it is a 1-based index in the registry's dynamic array
		int16 HandleValue = 0;

		friend struct FTraitRegistry;
	};
}
