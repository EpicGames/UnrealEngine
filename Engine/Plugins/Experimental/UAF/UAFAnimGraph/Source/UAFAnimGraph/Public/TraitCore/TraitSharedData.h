// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/LatentPropertyHandle.h"

#include <type_traits>

#include "TraitSharedData.generated.h"

namespace UE::UAF
{
	struct FTraitBinding;
}

#define ANIM_NEXT_IMPL_EXECUTE_LATENT_CONSTRUCTOR_FOR_PROPERTY(PropertyName) \
	/* @see GetLatentPropertyIndex below for details */ \
	[&Binding, LatentPropertyHandles](int32 LatentPropertyIndex) \
	{ \
		const UE::UAF::FLatentPropertyHandle LatentPropertyHandle = LatentPropertyHandles[LatentPropertyIndex - 1]; \
		if (LatentPropertyHandle.IsOffsetValid()) \
		{ \
			/* We remove the 'const' explicitly to avoid exposing a mutable version of the getter */ \
			new(const_cast<decltype(PropertyName)*>(Binding.GetLatentProperty<decltype(PropertyName)>(LatentPropertyHandle))) decltype(PropertyName)(); \
		} \
	}(GetLatentPropertyIndex(offsetof(Self, PropertyName))); \

#define ANIM_NEXT_IMPL_DEFINE_LATENT_CONSTRUCTOR(EnumeratorMacro) \
	/** @see FAnimNextTraitSharedData::ConstructLatentProperties */ \
	static void ConstructLatentProperties(const UE::UAF::FTraitBinding& Binding) \
	{ \
		/* Construct our latent properties */ \
		const UE::UAF::FLatentPropertyHandle* LatentPropertyHandles = Binding.GetLatentPropertyHandles(); \
		EnumeratorMacro(ANIM_NEXT_IMPL_EXECUTE_LATENT_CONSTRUCTOR_FOR_PROPERTY) \
	} \

#define ANIM_NEXT_IMPL_EXECUTE_LATENT_DESTRUCTOR_FOR_PROPERTY(PropertyName) \
	/* @see GetLatentPropertyIndex below for details */ \
	[&Binding, LatentPropertyHandles](int32 LatentPropertyIndex) \
	{ \
		const UE::UAF::FLatentPropertyHandle LatentPropertyHandle = LatentPropertyHandles[LatentPropertyIndex - 1]; \
		if (LatentPropertyHandle.IsOffsetValid()) \
		{ \
			/* We remove the 'const' explicitly to avoid exposing a mutable version of the getter */ \
			const_cast<decltype(PropertyName)*>(Binding.GetLatentProperty<decltype(PropertyName)>(LatentPropertyHandle))->~decltype(PropertyName)(); \
		} \
	}(GetLatentPropertyIndex(offsetof(Self, PropertyName))); \

#define ANIM_NEXT_IMPL_DEFINE_LATENT_DESTRUCTOR(EnumeratorMacro) \
	/** @see FAnimNextTraitSharedData::DestructLatentProperties */ \
	static void DestructLatentProperties(const UE::UAF::FTraitBinding& Binding) \
	{ \
		/* Destruct our latent properties */ \
		const UE::UAF::FLatentPropertyHandle* LatentPropertyHandles = Binding.GetLatentPropertyHandles(); \
		EnumeratorMacro(ANIM_NEXT_IMPL_EXECUTE_LATENT_DESTRUCTOR_FOR_PROPERTY) \
	} \

#define ANIM_NEXT_IMPL_GET_LATENT_PROPERTY_INDEX_FOR_PROPERTY(PropertyName) \
	LatentPropertyIndex--; \
	if (LatentPropertyOffset == offsetof(Self, PropertyName)) \
	{ \
		return -LatentPropertyIndex; \
	} \

#define ANIM_NEXT_IMPL_DEFINE_GET_LATENT_PROPERTY_INDEX(EnumeratorMacro) \
	/** @see FAnimNextTraitSharedData::GetLatentPropertyIndex */ \
	static constexpr int32 GetLatentPropertyIndex(const size_t LatentPropertyOffset) \
	{ \
		int32 LatentPropertyIndex = Super::GetLatentPropertyIndex(LatentPropertyOffset); \
		/* If the value is positive, the property lives in our base type */ \
		/* Otherwise the value is the negative (or zero) number of latent properties in the base type */ \
		if (LatentPropertyIndex > 0) \
		{ \
			return LatentPropertyIndex; \
		} \
		/* If a property in the struct is wrapped with WITH_EDITORONLY_DATA, then this code here */ \
		/* needs to be wrapped as well */ \
		EnumeratorMacro(ANIM_NEXT_IMPL_GET_LATENT_PROPERTY_INDEX_FOR_PROPERTY) \
		/* Latent property wasn't found, return the number of latent properties seen so far */ \
		return LatentPropertyIndex; \
	} \

#define ANIM_NEXT_IMPL_DEFINE_LATENT_GETTER(PropertyName) \
	const decltype(PropertyName)& Get##PropertyName(const UE::UAF::FTraitBinding& Binding) const \
	{ \
		/* We need a mapping of latent property name/offset to latent property index */ \
		/* This can be built once at runtime using the UE reflection and cached on first call or using a constexpr function, see below */ \
		constexpr size_t LatentPropertyOffset = offsetof(Self, PropertyName); \
		constexpr int32 LatentPropertyIndex = GetLatentPropertyIndex(LatentPropertyOffset); \
		/* A negative or zero property index means the property isn't latent, we can static assert since it isn't possible */ \
		static_assert(LatentPropertyIndex > 0, "Property " #PropertyName " isn't latent"); \
		const UE::UAF::FLatentPropertyHandle* LatentPropertyHandles = Binding.GetLatentPropertyHandles(); \
		const UE::UAF::FLatentPropertyHandle LatentPropertyHandle = LatentPropertyHandles[LatentPropertyIndex - 1]; \
		/* An invalid offset means we are inline, otherwise we are cached */ \
		if (!LatentPropertyHandle.IsOffsetValid()) \
		{ \
			return PropertyName; \
		} \
		else \
		{ \
			return *Binding.GetLatentProperty<decltype(PropertyName)>(LatentPropertyHandle); \
		} \
	} \

/**
  * This macro defines the necessary boilerplate for latent property support.
  * It takes as arguments the shared data struct name that owns the latent properties
  * and an enumerator macro. The enumerator macro should accept one argument which will
  * be a macro to apply to each latent property. We then use it to define what we need.
  */
#define GENERATE_TRAIT_LATENT_PROPERTIES(SharedDataType, EnumeratorMacro) \
	using Self = SharedDataType; \
	ANIM_NEXT_IMPL_DEFINE_LATENT_CONSTRUCTOR(EnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_LATENT_DESTRUCTOR(EnumeratorMacro) \
	ANIM_NEXT_IMPL_DEFINE_GET_LATENT_PROPERTY_INDEX(EnumeratorMacro) \
	EnumeratorMacro(ANIM_NEXT_IMPL_DEFINE_LATENT_GETTER) \

/**
 * FAnimNextTraitSharedData
 * Trait shared data represents a unique instance in the authored static graph.
 * Each instance of a graph will share instances of the read-only shared data.
 * Shared data typically contains hardcoded properties, RigVM latent pin information,
 * or pooled properties shared between multiple traits.
 * 
 * @see FNodeDescription
 *
 * A FAnimNextTraitSharedData is the base type that trait shared data derives from.
 */
USTRUCT()
struct FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/**
	 * Constructs the latent properties on the bound trait instance
	 */
	static void ConstructLatentProperties(const UE::UAF::FTraitBinding& Binding) {}

	/**
	 * Destructs the latent properties on the bound trait instance
	 */
	static void DestructLatentProperties(const UE::UAF::FTraitBinding& Binding) {}

	/**
	  * Returns the latent property index from a latent property offset
	  * If the property is found within this type, its latent property 1-based index is returned (a positive value)
	  * If the property isn't found, this function returns the number of latent properties (as a zero or negative value)
	  * Derived types will have latent property indices higher than their base type
	  * Note that the property index is derived from the order specified in the GENERATE_TRAIT_LATENT_PROPERTIES enumerator macro
	  * which might not match the order of the UStruct
	  */
	static constexpr int32 GetLatentPropertyIndex(const size_t LatentPropertyOffset) { return 0; }
};
