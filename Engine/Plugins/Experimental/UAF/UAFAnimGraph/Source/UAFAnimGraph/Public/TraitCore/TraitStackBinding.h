// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitInterfaceUID.h"
#include "TraitCore/TraitPtr.h"

#include <type_traits>

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FExecutionContext;
	struct FNodeDescription;
	struct FNodeInstance;
	struct FNodeTemplate;
	struct FTraitBinding;
	template<class TraitInterface> struct TTraitBinding;
	struct FTraitStackBinding;

	/**
	 * FTraitStackBinding
	 *
	 * To keep the node instance cost as low as possible, data such as pointers to the shared data
	 * are not stored per node and are instead fetched on demand. In order to be able to query for
	 * interfaces on a trait stack from a trait pointer, a trait stack binding must first be created.
	 */
	struct FTraitStackBinding
	{
		// Creates an empty/invalid binding.
		FTraitStackBinding() = default;

		// Returns whether or not this binding is valid.
		bool IsValid() const { return Context != nullptr; }



		//////////////////////////////////////////////////////////////////////////
		// Misc functions

		// Takes a snapshot of all latent properties on this trait stack.
		// Delta time supplied so that RigVM execution can access it
		// Properties can be marked as always updating or as supporting freezing (e.g. when a branch of the graph blends out).
		// A freezable property does not update when a snapshot is taken of a frozen stack.
		UE_API void SnapshotLatentProperties(float DeltaTime, bool bIsFrozen, bool bIsFirstUpdate);

		// Returns a trait pointer to the base of the trait stack.
		FWeakTraitPtr GetBaseTraitPtr() const { return FWeakTraitPtr(NodeInstance, BaseTraitIndex); }

		// Resets the stack binding to an invalid state.
		UE_API void Reset();

		// Equality testing
		UE_API bool operator==(const FTraitStackBinding& Other) const;
		bool operator!=(const FTraitStackBinding& Other) const { return !operator==(Other); }



		//////////////////////////////////////////////////////////////////////////
		// The following functions allow traversing the trait stack

		// Returns a trait binding to the top of the stack.
		// Returns true on success, false otherwise.
		UE_API bool GetTopTrait(FTraitBinding& OutBinding) const;

		// Returns a trait binding to the trait below the specified one (its parent).
		// Returns true on success, false otherwise.
		UE_API bool GetParentTrait(const FTraitBinding& ChildBinding, FTraitBinding& OutParentBinding) const;

		// Returns a trait binding to the base of the stack.
		// Returns true on success, false otherwise.
		UE_API bool GetBaseTrait(FTraitBinding& OutBinding) const;

		// Returns a trait binding to the trait above the specified one (its child).
		// Returns true on success, false otherwise.
		UE_API bool GetChildTrait(const FTraitBinding& ParentBinding, FTraitBinding& OutChildBinding) const;

		// Returns a trait binding to the trait at the specified index (relative to the start of the stack).
		// Returns true on success, false otherwise.
		UE_API bool GetTrait(uint32 TraitIndex, FTraitBinding& OutBinding) const;

		// Returns the number of traits on this stack.
		UE_API uint32 GetNumTraits() const;



		//////////////////////////////////////////////////////////////////////////
		// The following functions allow querying interfaces on the trait stack

		// Queries the trait stack for a trait that implements the specified interface.
		// The search begins at the top of the stack.
		// Returns true on success, false otherwise.
		template<class TraitInterface>
		bool GetInterface(TTraitBinding<TraitInterface>& OutBinding) const;

		// Queries the trait stack for a trait lower on the stack that implements the specified interface.
		// Returns true on success, false otherwise.
		template<class TraitInterface>
		bool GetInterfaceSuper(const FTraitBinding& Binding, TTraitBinding<TraitInterface>& OutSuperBinding) const;

		// Queries the trait stack for the presence of a trait that implements the specified interface.
		// The search begins at the top of the stack.
		// Returns true on success, false otherwise.
		template<class TraitInterface>
		bool HasInterface() const;

		// Queries the trait stack for the presence of a trait lower on the stack that implements the specified interface.
		// Returns true on success, false otherwise.
		template<class TraitInterface>
		bool HasInterfaceSuper(const FTraitBinding& Binding) const;


	private:
		// Creates a trait stack binding with the stack that owns the specified trait pointer.
		UE_API FTraitStackBinding(const FExecutionContext& InContext, const FWeakTraitPtr& TraitPtr);

		UE_API bool GetInterfaceImpl(FTraitInterfaceUID InterfaceUID, FTraitBinding& OutBinding) const;
		UE_API bool GetInterfaceSuperImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding, FTraitBinding& OutSuperBinding) const;
		UE_API bool HasInterfaceImpl(FTraitInterfaceUID InterfaceUID) const;
		UE_API bool HasInterfaceSuperImpl(FTraitInterfaceUID InterfaceUID, const FTraitBinding& Binding) const;

		// A pointer to the execution context that created the binding
		const FExecutionContext* Context = nullptr;

		// A pointer to the node instance data we are bound to
		FNodeInstance* NodeInstance = nullptr;

		// A pointer to the node shared data we are bound to
		const FNodeDescription* NodeDescription = nullptr;

		// A pointer to the node template used by the node we are bound to
		const FNodeTemplate* NodeTemplate = nullptr;

		// The base trait index of the bound trait stack on the node
		// A node can contain multiple independent trait stacks by having multiple base traits
		uint32 BaseTraitIndex = 0;	// Only need 8 bits, using 32 since we have padding anyway

		// The top trait index of the bound trait stack on the node
		uint32 TopTraitIndex = 0;	// Only need 8 bits, using 32 since we have padding anyway

		friend FExecutionContext;
		friend FTraitBinding;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementations

	template<class TraitInterface>
	inline bool FTraitStackBinding::GetInterface(TTraitBinding<TraitInterface>& OutBinding) const
	{
		static_assert(std::is_base_of<ITraitInterface, TraitInterface>::value, "TraitInterface type must derive from ITraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = TraitInterface::InterfaceUID;
		return GetInterfaceImpl(InterfaceUID, OutBinding);
	}

	template<class TraitInterface>
	inline bool FTraitStackBinding::GetInterfaceSuper(const FTraitBinding& Binding, TTraitBinding<TraitInterface>& OutSuperBinding) const
	{
		static_assert(std::is_base_of<ITraitInterface, TraitInterface>::value, "TraitInterface type must derive from ITraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = TraitInterface::InterfaceUID;
		return GetInterfaceSuperImpl(InterfaceUID, Binding, OutSuperBinding);
	}

	template<class TraitInterface>
	inline bool FTraitStackBinding::HasInterface() const
	{
		static_assert(std::is_base_of<ITraitInterface, TraitInterface>::value, "TraitInterface type must derive from ITraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = TraitInterface::InterfaceUID;
		return HasInterfaceImpl(InterfaceUID);
	}

	template<class TraitInterface>
	inline bool FTraitStackBinding::HasInterfaceSuper(const FTraitBinding& Binding) const
	{
		static_assert(std::is_base_of<ITraitInterface, TraitInterface>::value, "TraitInterface type must derive from ITraitInterface");

		constexpr FTraitInterfaceUID InterfaceUID = TraitInterface::InterfaceUID;
		return HasInterfaceSuperImpl(InterfaceUID, Binding);
	}
}

#undef UE_API
