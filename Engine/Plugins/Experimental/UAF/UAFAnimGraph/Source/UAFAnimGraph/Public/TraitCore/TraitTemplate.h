// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitMode.h"
#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitRegistryHandle.h"
#include "TraitCore/LatentPropertyHandle.h"

#include <type_traits>

class FArchive;
struct FAnimNextTraitSharedData;

namespace UE::UAF
{
	struct FTraitInstanceData;
	struct FNodeDescription;
	struct FNodeInstance;

	/**
	 * Trait Template
	 * A trait template represents a specific trait within a FNodeTemplate.
	 * 
	 * @see FNodeTemplate
	 */
	struct FTraitTemplate
	{
		// Returns the globally unique identifier for the trait
		FTraitUID GetUID() const noexcept { return FTraitUID(UID); }

		// Returns the trait registry handle
		FTraitRegistryHandle GetRegistryHandle() const noexcept { return RegistryHandle; }

		// Returns whether or not this trait template is valid
		// A trait template is invalid if the trait implementation hasn't been registered
		bool IsValid() const noexcept { return RegistryHandle.IsValid(); }

		// Returns the trait mode
		ETraitMode GetMode() const noexcept { return static_cast<ETraitMode>(Mode); }

		// Returns whether this trait template needs be registered with the reference collector
		bool IsGCAware() const noexcept { return bGCAware; }

		// For base traits only, returns the number of traits on the sub-stack
		uint32 GetNumStackTraits() const
		{
			check(GetMode() == ETraitMode::Base);
			return TraitIndexOrNumTraits;
		}

		// Returns the trait index relative to the base trait on the sub-stack
		uint32 GetTraitIndex() const
		{
			return GetMode() == ETraitMode::Base ? 0 : TraitIndexOrNumTraits;
		}

		// Returns the number of latent properties on this trait
		uint32 GetNumLatentPropreties() const noexcept { return NumLatentProperties; }

		// Returns the number of latent properties on the trait sub-stack
		// Only available on the base trait of the sub-stack
		uint32 GetNumSubStackLatentPropreties() const
		{
			check(GetMode() == ETraitMode::Base);
			return NumSubStackLatentProperties;
		}

		// Returns the offset into the shared data where the descriptor begins, relative to the root of the node's shared data
		uint32 GetNodeSharedOffset() const noexcept { return NodeSharedOffset; }

		// Returns the offset into the shared data where the latent property handles begin, relative to the root of the node's shared data
		// The base trait has an instance of FLatentPropertiesHeader preceding its latent handles of type FLatentPropertyHandle
		// Additive traits just have a list of FLatentPropertyHandle
		// Note that the offset might not be zero even if no latent properties live on this trait
		uint32 GetNodeSharedLatentPropertyHandlesOffset() const noexcept { return NodeSharedLatentPropertyHandlesOffset; }

		// Returns the offset into the instance data where the descriptor begins, relative to the root of the node's instance data
		uint32 GetNodeInstanceOffset() const noexcept { return NodeInstanceOffset; }

		// Returns whether or not we have latent properties on this trait
		bool HasLatentProperties() const noexcept { return NumLatentProperties != 0; }

		// Returns a pointer to the specified trait description on the current node
		FAnimNextTraitSharedData* GetTraitDescription(FNodeDescription& NodeDescription) const noexcept
		{
			return reinterpret_cast<FAnimNextTraitSharedData*>(reinterpret_cast<uint8*>(&NodeDescription) + GetNodeSharedOffset());
		}

		// Returns a pointer to the specified trait description on the current node
		const FAnimNextTraitSharedData* GetTraitDescription(const FNodeDescription& NodeDescription) const noexcept
		{
			return reinterpret_cast<const FAnimNextTraitSharedData*>(reinterpret_cast<const uint8*>(&NodeDescription) + GetNodeSharedOffset());
		}

		// Returns a reference to the specified trait latent properties header on the current node
		FLatentPropertiesHeader& GetTraitLatentPropertiesHeader(FNodeDescription& NodeDescription) const noexcept
		{
			check(GetMode() == ETraitMode::Base);	// Only the base trait has a header
			return *reinterpret_cast<FLatentPropertiesHeader*>(reinterpret_cast<uint8*>(&NodeDescription) + GetNodeSharedLatentPropertyHandlesOffset() - sizeof(FLatentPropertiesHeader));
		}

		// Returns a reference to the specified trait latent properties header on the current node
		const FLatentPropertiesHeader& GetTraitLatentPropertiesHeader(const FNodeDescription& NodeDescription) const noexcept
		{
			check(GetMode() == ETraitMode::Base);	// Only the base trait has a header
			return *reinterpret_cast<const FLatentPropertiesHeader*>(reinterpret_cast<const uint8*>(&NodeDescription) + GetNodeSharedLatentPropertyHandlesOffset() - sizeof(FLatentPropertiesHeader));
		}

		// Returns a pointer to the specified trait latent property handles on the current node
		// WARNING: Latent property handles are in the order defined by the enumerator macro within the shared data,
		// @see FAnimNextTraitSharedData::GetLatentPropertyIndex
		FLatentPropertyHandle* GetTraitLatentPropertyHandles(FNodeDescription& NodeDescription) const noexcept
		{
			return reinterpret_cast<FLatentPropertyHandle*>(reinterpret_cast<uint8*>(&NodeDescription) + GetNodeSharedLatentPropertyHandlesOffset());
		}

		// Returns a pointer to the specified trait latent property handles on the current node
		// WARNING: Latent property handles are in the order defined by the enumerator macro within the shared data,
		// @see FAnimNextTraitSharedData::GetLatentPropertyIndex
		const FLatentPropertyHandle* GetTraitLatentPropertyHandles(const FNodeDescription& NodeDescription) const noexcept
		{
			return reinterpret_cast<const FLatentPropertyHandle*>(reinterpret_cast<const uint8*>(&NodeDescription) + GetNodeSharedLatentPropertyHandlesOffset());
		}

		// Returns a pointer to the specified trait instance on the current node
		FTraitInstanceData* GetTraitInstance(FNodeInstance& NodeInstance) const noexcept
		{
			return reinterpret_cast<FTraitInstanceData*>(reinterpret_cast<uint8*>(&NodeInstance) + GetNodeInstanceOffset());
		}

		// Returns a pointer to the specified trait instance on the current node
		const FTraitInstanceData* GetTraitInstance(const FNodeInstance& NodeInstance) const noexcept
		{
			return reinterpret_cast<const FTraitInstanceData*>(reinterpret_cast<const uint8*>(&NodeInstance) + GetNodeInstanceOffset());
		}

		// Serializes this trait template instance
		UAFANIMGRAPH_API void Serialize(FArchive& Ar);

	private:
		friend struct FNodeTemplate;
		friend struct FNodeTemplateBuilder;

		FTraitTemplate(FTraitUID InUID, FTraitRegistryHandle InRegistryHandle, ETraitMode InMode, bool bInGCAware, uint32 InTraitIndexOrNumTraits) noexcept
			: UID(InUID.GetUID())
			, RegistryHandle(InRegistryHandle)
			, Mode(static_cast<uint8>(InMode))
			, bGCAware(bInGCAware)
			, Padding0(0)
			, TraitIndexOrNumTraits(InTraitIndexOrNumTraits)
			, NumLatentProperties(0)
			, NumSubStackLatentProperties(0)
			// For shared and instance data, 0 is an invalid offset since the data follows their respective header (FNodeDescription or FNodeInstance)
			, NodeSharedOffset(0)
			, NodeSharedLatentPropertyHandlesOffset(0)
			, NodeInstanceOffset(0)
			, Padding1(0)
		{
		}

		// Trait globally unique identifier (32 bits)
		FTraitUIDRaw				UID;

		// Cached trait registry handle (16 bits)
		FTraitRegistryHandle	RegistryHandle;

		// Trait mode (we only need 2 bits)
		uint8	Mode : 2;
		static_assert(static_cast<uint8_t>(ETraitMode::Num) <= 2);
		
		// Flag set if a trait implements IGarbageCollection
		uint8   bGCAware : 1;
		
		// (Padding for extra flags)
		uint8   Padding0 : 5;

		// For base traits, this contains the number of traits on our sub-stack
		// For additive traits, this contains the trait index relative to the base trait
		// The first additive trait has index 1, there is no index 0 (we re-purpose the value for the base trait)
		uint8	TraitIndexOrNumTraits;

		// For each latent property defined on a trait, we store a handle in the per node shared data
		// These handles specify various metadata of the latent property: RigVM memory handle index, whether the property can freeze, instance data offset
		// The handles of each trait that lives on a sub-stack (base + additives) are stored contiguously
		// The base trait has the root handle offset and the total number as well as its local number of latent properties
		// Each additive trait has a handle offset that points into that contiguous list and its local number of latent properties

		// How many latent properties are defined on this trait (cached value of FTrait::GetNumLatentPropreties to avoid repeated lookups)
		// Not serialized, @see FNodeTemplate::Finalize
		uint16	NumLatentProperties;

		// How many latent properties are defined on this trait sub-stack (stored on base trait only)
		// Not serialized, @see FNodeTemplate::Finalize
		uint16	NumSubStackLatentProperties;

		// Offsets into the shared read-only and instance data portions of a node
		// These are not serialized, @see FNodeTemplate::Finalize
		// These are relative to the root of the node description and instance data, respectively (max 64 KB per node each)
		// These offsets are fixed in the template meaning each node will have the same memory layout up to the last trait
		// Optional data like cached latent properties are stored after this fixed layout ends

		uint16	NodeSharedOffset;						// Start of shared data for this trait
		uint16	NodeSharedLatentPropertyHandlesOffset;	// Start of shared latent property handles for this trait
		uint16	NodeInstanceOffset;						// Start of instance data for this trait

		uint16	Padding1;						// Unused for now

		// TODO: We could cache which parent/base trait handles which interface
		// This would avoid the need to iterate on every trait to look up a 'Super'. Perhaps only common interfaces could be cached.
	};

	static_assert(std::is_trivially_copyable_v<FTraitTemplate>, "FTraitTemplate needs to be trivially copyable");
}
