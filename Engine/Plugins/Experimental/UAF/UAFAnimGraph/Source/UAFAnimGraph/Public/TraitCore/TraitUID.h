// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/Fnv.h"

namespace UE::UAF
{
	// Type alias for a raw trait UID, not typesafe
	using FTraitUIDRaw = uint32;

	/**
	 * FTraitUID
	 *
	 * Encapsulates a trait global UID.
	 *
	 * The whole struct is meant to be 'constexpr' to allow inlining within the assembly as constants.
	 */
	struct FTraitUID final
	{
		// Constructs an invalid UID
		constexpr FTraitUID() noexcept
			: UID(INVALID_UID)
#if !UE_BUILD_SHIPPING
			, TraitName(TEXT("<Invalid trait UID>"))
#endif
		{
		}

		// Constructs a trait UID from its raw value
		explicit constexpr FTraitUID(FTraitUIDRaw InUID) noexcept
			: UID(InUID)
#if !UE_BUILD_SHIPPING
			, TraitName(TEXT("<Unknown Trait Name>"))
#endif
		{
		}

		// Constructs a trait UID from a string literal
		template<typename CharType, int32 Len>
		static UE_CONSTEVAL FTraitUID MakeUID(const CharType(&InTraitName)[Len]) noexcept
		{
			return FTraitUID(UE::HashStringFNV1a<FTraitUIDRaw>(MakeStringView(InTraitName, Len - 1)), InTraitName);
		}

#if !UE_BUILD_SHIPPING
		// Returns a string literal to the trait name
		constexpr const TCHAR* GetTraitName() const noexcept { return TraitName; }
#endif

		// Returns the trait global UID
		constexpr FTraitUIDRaw GetUID() const noexcept { return UID; }

		// Returns whether this UID is valid or not
		constexpr bool IsValid() const noexcept { return UID != INVALID_UID; }

	private:
		static constexpr FTraitUIDRaw INVALID_UID = 0;

		// Constructs a trait UID
		explicit constexpr FTraitUID(FTraitUIDRaw InUID, const TCHAR* InTraitName) noexcept
			: UID(InUID)
#if !UE_BUILD_SHIPPING
			, TraitName(InTraitName)
#endif
		{
		}

		FTraitUIDRaw	UID;

#if !UE_BUILD_SHIPPING
		const TCHAR*	TraitName;
#endif
	};

	// Compares for equality and inequality
	constexpr bool operator==(FTraitUID LHS, FTraitUID RHS) noexcept { return LHS.GetUID() == RHS.GetUID(); }
	constexpr bool operator!=(FTraitUID LHS, FTraitUID RHS) noexcept { return LHS.GetUID() != RHS.GetUID(); }
	constexpr bool operator==(FTraitUID LHS, FTraitUIDRaw RHS) noexcept { return LHS.GetUID() == RHS; }
	constexpr bool operator!=(FTraitUID LHS, FTraitUIDRaw RHS) noexcept { return LHS.GetUID() != RHS; }
	constexpr bool operator==(FTraitUIDRaw LHS, FTraitUID RHS) noexcept { return LHS == RHS.GetUID(); }
	constexpr bool operator!=(FTraitUIDRaw LHS, FTraitUID RHS) noexcept { return LHS != RHS.GetUID(); }
}
