// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "Animation/InputScaleBias.h"

#include "IAlphaInputArgs.generated.h"

#define UE_API UAFANIMGRAPH_API

USTRUCT(meta = (DisplayName = "Alpha Input Args"))
struct FAlphaInputTraitArgs : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Float"))
	float Alpha = 1.f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Float"))
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, DisplayAfter = "AlphaCurveName", EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Float || AlphaInputType == EAnimAlphaInputType::Curve"))
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "bEnabled", EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Bool"))
	bool bAlphaBoolEnabled = true;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, DisplayName = "Blend Settings", EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Bool"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	/** Note: For curve types, the additive branch will always be evaluated. Curve weight blending will occur at task evaluation time. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Curve"))
	FName AlphaCurveName = NAME_None;

	UPROPERTY(EditAnywhere, Category = Settings)
	EAnimAlphaInputType AlphaInputType = EAnimAlphaInputType::Float;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \
		GeneratorMacro(bAlphaBoolEnabled) \
		GeneratorMacro(AlphaBoolBlend) \
		GeneratorMacro(AlphaCurveName) \
		GeneratorMacro(AlphaInputType) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAlphaInputTraitArgs, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};


namespace UE::UAF
{
	/**
	 * IAlphaInputArgs
	 *
	 * This interface exposes alpha input args & convience methods for operating on those args
	 * 
	 * Alpha input is typically a curve / float / bool that controls the strength of some other trait.
	 * Ex: 0 - no effect, 1 - full effect.
	 */
	struct IAlphaInputArgs : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IAlphaInputArgs)

		/**
		 * Returns the args captured by this trait interface.
		 * Note: We return a copy as some implementations may return a temp struct & our return type is relatively cheap to copy.
		 */
		UE_API virtual FAlphaInputTraitArgs Get(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const;

		/**
		 * Returns current alpha input type
		 */
		UE_API virtual EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const;

		/**
		 * Returns current alpha curve name, if any
		 */
		UE_API virtual FName GetAlphaCurveName(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const;

		/**
		 * Returns alpha value for current input. For curve types this will be 1.0, so curves should use trait args to resolve alpha in a task.
		 *
		 * @param DeltaTime - Time since last API call using DeltaTime (Ex: Time since last 'ComputeAlphaValue' call).
		 */
		UE_API virtual float GetCurrentAlphaValue(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const;

		/**
		 * Returns callback to clamp alpha values, typically used for curve sampling
		 * Note: Callback may perform a deferred write to alpha input instance data.
		 */
		UE_API virtual TFunction<float(float)> GetInputScaleBiasClampCallback(const FExecutionContext& Context, const TTraitBinding<IAlphaInputArgs>& Binding) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IAlphaInputArgs> : FTraitBinding
	{
		// @see IAlphaInputArgs::Get
		FAlphaInputTraitArgs Get(const FExecutionContext& Context) const
		{
			return GetInterface()->Get(Context, *this);
		}

		// @see IAlphaInputArgs::GetAlphaInputType
		EAnimAlphaInputType GetAlphaInputType(const FExecutionContext& Context) const
		{
			return GetInterface()->GetAlphaInputType(Context, *this);
		}

		// @see IAlphaInputArgs::GetAlphaCurveName
		FName GetAlphaCurveName(const FExecutionContext& Context) const
		{
			return GetInterface()->GetAlphaCurveName(Context, *this);
		}

		// @see IAlphaInputArgs::GetCurrentAlphaValue
		float GetCurrentAlphaValue(const FExecutionContext& Context) const
		{
			return GetInterface()->GetCurrentAlphaValue(Context, *this);
		}

		// @see IAlphaInputArgs::GetInputScaleBiasClampCallback
		TFunction<float(float)> GetInputScaleBiasClampCallback(const FExecutionContext& Context) const
		{
			return GetInterface()->GetInputScaleBiasClampCallback(Context, *this);
		}

	protected:
		const IAlphaInputArgs* GetInterface() const { return GetInterfaceTyped<IAlphaInputArgs>(); }
	};
}

#undef UE_API
