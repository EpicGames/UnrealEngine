// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

class UAnimSequenceBase;

namespace UE::UAF
{
	/**
	 * Root motion attribute sampling may occur during execution (Ex: Conditionally sampling attributes based on active root motion).
	 * This signature is used to support that deferred execution, each implementing trait is responsible for ensuring callback lifetimes.
	 */
	DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FOnExtractRootMotionAttribute, float /* StartTime */, float /* DeltaTime */, bool /* bAllowLooping */);

	/**
	 * IAttributeProvider
	 *
	 * This interface provides access to animation attributes such as root motion
	 */
	struct IAttributeProvider : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IAttributeProvider)

		// Returns a callback that can be used to extract root motion at a later time
		UE_API virtual FOnExtractRootMotionAttribute GetOnExtractRootMotionAttribute(FExecutionContext& Context, const TTraitBinding<IAttributeProvider>& Binding) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IAttributeProvider> : FTraitBinding
	{
		// @see IAttributeProvider::GetOnExtractRootMotionAttribute
		FOnExtractRootMotionAttribute GetOnExtractRootMotionAttribute(FExecutionContext& Context) const
		{
			return GetInterface()->GetOnExtractRootMotionAttribute(Context, *this);
		}

	protected:
		const IAttributeProvider* GetInterface() const { return GetInterfaceTyped<IAttributeProvider>(); }
	};
}

#undef UE_API
