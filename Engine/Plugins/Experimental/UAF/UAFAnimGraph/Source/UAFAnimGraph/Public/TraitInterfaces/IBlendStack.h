// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/ExecutionContext.h"
#include "AlphaBlend.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Animation/BlendProfile.h"
#include "TraitInterfaces/IHierarchy.h"

#define UE_API UAFANIMGRAPH_API

class UUAFBlendProfile;

namespace UE::UAF
{
	/**
	 * IBlendStack
	 *
	 * This interface exposes anything needed to push a new subgraph
	 */
	struct IBlendStack : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IBlendStack)

		enum class EBlendMode : uint8
		{
			// Uses standard weight based blend
			Standard,

			// Uses inertialization. Requires an inertialization trait somewhere earlier in the graph.
			Inertialization,
		};

		enum class EGraphRequestType : uint8
		{
			// Graph is allocated and owned by the blend stack
			Owned,

			// Graph is a child handle in the blend stacks graph instance
			Child,
		};

		struct FGraphRequest
		{
			/**
			 * TODO: Add more blend options here as we need them.
			 * Consider making dynamic payload if we want to implement a special blend framework.
			 */

			// Blend-in duration for this graph request 
			FAlphaBlendArgs BlendArgs;

			// Optional blend profile for this graph request
			TObjectPtr<const UUAFBlendProfile> BlendProfile;

			// The template graph to use for the new graph instance
			TObjectPtr<const UUAFAnimGraph> AnimationGraph;

			// The object that was used for factory generation of the graph.
			TObjectPtr<const UObject> FactoryObject;

			// The additional params used for factory generation of the graph.
			FAnimNextFactoryParams FactoryParams;

			// Child ptr when Type == Child
			FTraitPtr ChildPtr;

			// The Blend mode to use
			EBlendMode BlendMode = EBlendMode::Standard;

			// What type of request this is
			EGraphRequestType Type = EGraphRequestType::Owned;
		};

		typedef FGraphRequest* FGraphRequestPtr;

		// Pushes a new subgraph along with blend settings defined in GraphRequest. Outputs the in-place created subgraph OutGraphInstance.
		// @return the child index of the new graph, or INDEX_NONE if we could not push a graph
		UE_API virtual int32 PushGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequest&& GraphRequest) const;

		// Gets the graph request info from the most recent PushGraph
		// @return the child index of the active graph, or INDEX_NONE if no active graph was found
		UE_API virtual int32 GetActiveGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, IBlendStack::FGraphRequestPtr& OutGraphRequest) const;

		// Gets the graph request for the specified child index
		UE_API virtual IBlendStack::FGraphRequestPtr GetGraph(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const;

		// Gets the graph instance for the specified child index.
		// Instance's lifetime may not persist after this call, so do not store the returned ptr 
		virtual FAnimNextGraphInstance* GetGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, int32 InChildIndex) const;

		// Gets the graph instance from the most recent PushGraph
		// Note that if a child graph instance was passed, the instance returned will be the same as that of the hosting trait stack
		// @return the child index of the active graph instance, or INDEX_NONE if no active graph instance was found
		virtual int32 GetActiveGraphInstance(FExecutionContext& Context, const TTraitBinding<IBlendStack>& Binding, FAnimNextGraphInstance*& OutGraphInstance) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IBlendStack> : FTraitBinding
	{
		// @see IBlendStack::PushGraph
		int32 PushGraph(FExecutionContext& Context, IBlendStack::FGraphRequest&& GraphRequest) const
		{
			return GetInterface()->PushGraph(Context, *this, MoveTemp(GraphRequest));
		}

		// @see IBlendStack::GetActiveGraph
		int32 GetActiveGraph(FExecutionContext& Context, IBlendStack::FGraphRequestPtr& OutGraphRequest) const
		{
			OutGraphRequest = nullptr; // Ensure request will be initialized
			return GetInterface()->GetActiveGraph(Context, *this, OutGraphRequest);
		}

		// @see IBlendStack::GetGraph
		IBlendStack::FGraphRequestPtr GetGraph(FExecutionContext& Context, int32 InChildIndex) const
		{
			return GetInterface()->GetGraph(Context, *this, InChildIndex);
		}

		// @see IBlendStack::GetGraphInstance
		virtual int32 GetActiveGraphInstance(FExecutionContext& Context, FAnimNextGraphInstance*& OutGraphInstance) const
		{
			return GetInterface()->GetActiveGraphInstance(Context, *this, OutGraphInstance);
		}
		
		// @see IBlendStack::GetGraphInstance
		virtual FAnimNextGraphInstance* GetGraphInstance(FExecutionContext& Context, int32 InChildIndex) const
		{
			return GetInterface()->GetGraphInstance(Context, *this, InChildIndex);
		}		

	protected:
		const IBlendStack* GetInterface() const { return GetInterfaceTyped<IBlendStack>(); }
	};
}

#undef UE_API
