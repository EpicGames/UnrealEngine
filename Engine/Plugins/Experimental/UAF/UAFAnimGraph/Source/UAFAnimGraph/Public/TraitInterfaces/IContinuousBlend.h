// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	/**
	 * IContinuousBlend
	 *
	 * This interface exposes continuous blend related information.
	 * 
	 * Assuming that each blend source is temporally coherent (Has no snaps or pops in itself).
	 * Then, a continuous blend's poses should be temporally coherent from one pose to the next.
	 * Hence these poses will have no need for things like smoothing between each other.
	 */
	struct IContinuousBlend : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IContinuousBlend)

		// Returns the blend weight for the specified child
		// Multiple children can have non-zero weight but their sum must be 1.0
		// Returns -1.0 if the child index is invalid
		UE_API virtual float GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IContinuousBlend> : FTraitBinding
	{
		// @see IContinuousBlend::GetBlendWeight
		float GetBlendWeight(const FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendWeight(Context, *this, ChildIndex);
		}

	protected:
		const IContinuousBlend* GetInterface() const { return GetInterfaceTyped<IContinuousBlend>(); }
	};
}

#undef UE_API
