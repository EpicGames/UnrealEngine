// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

struct FAlphaBlend;

namespace UE::UAF
{
	/**
	 * IDiscreteBlend
	 *
	 * This interface exposes discrete blend related information.
	 * 
	 * Discrete blends may snap or pop from one blend source to another, they may be non-continuous.
	 */
	struct IDiscreteBlend : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IDiscreteBlend)

		// Returns the blend weight for the specified child
		// Multiple children can have non-zero weight but their sum must be 1.0
		// Returns -1.0 if the child index is invalid
		UE_API virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const;

		// Returns the blend alpha for the specified child
		// Returns nullptr if the child index is invalid
		// Allows additive traits to query the internal state of the base trait
		UE_API virtual const FAlphaBlend* GetBlendState(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const;

		// Returns the blend destination child index (aka the active child index)
		// Returns INDEX_NONE if no child is active
		UE_API virtual int32 GetBlendDestinationChildIndex(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding) const;

		// Called when a blend transition between children occurs
		// OldChildIndex can be INDEX_NONE if there was no previously active child
		// NewChildIndex can be larger than the current number of known children to support a dynamic number of children at runtime
		// When this occurs, the number of children increments by one
		// The number of children never shrinks
		UE_API virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const;

		// Called when the blend for specified child is initiated
		UE_API virtual void OnBlendInitiated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const;

		// Called when the blend for specified child terminates
		UE_API virtual void OnBlendTerminated(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IDiscreteBlend> : FTraitBinding
	{
		// @see IDiscreteBlend::GetBlendWeight
		float GetBlendWeight(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendWeight(Context, *this, ChildIndex);
		}

		// @see IDiscreteBlend::GetBlendState
		const FAlphaBlend* GetBlendState(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendState(Context, *this, ChildIndex);
		}

		// @see IDiscreteBlend::GetBlendDestinationChildIndex
		int32 GetBlendDestinationChildIndex(FExecutionContext& Context) const
		{
			return GetInterface()->GetBlendDestinationChildIndex(Context, *this);
		}

		// @see IDiscreteBlend::OnBlendTransition
		void OnBlendTransition(FExecutionContext& Context, int32 OldChildIndex, int32 NewChildIndex) const
		{
			GetInterface()->OnBlendTransition(Context, *this, OldChildIndex, NewChildIndex);
		}

		// @see IDiscreteBlend::OnBlendInitiated
		void OnBlendInitiated(FExecutionContext& Context, int32 ChildIndex) const
		{
			GetInterface()->OnBlendInitiated(Context, *this, ChildIndex);
		}

		// @see IDiscreteBlend::OnBlendTerminated
		void OnBlendTerminated(FExecutionContext& Context, int32 ChildIndex) const
		{
			GetInterface()->OnBlendTerminated(Context, *this, ChildIndex);
		}

	protected:
		const IDiscreteBlend* GetInterface() const { return GetInterfaceTyped<IDiscreteBlend>(); }
	};
}

#undef UE_API
