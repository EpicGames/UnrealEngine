// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "EvaluationVM/KeyframeState.h"

#define UE_API UAFANIMGRAPH_API

struct FAnimNextGraphLODPose;
struct FRigUnit_AnimNextRunAnimationGraph_v1;
struct FRigUnit_AnimNextRunAnimationGraph_v2;
struct FRigUnit_UAFRunAsset;
struct FAnimNode_AnimNextGraph;
class UAbstractSkeletonSetBinding;

namespace UE::UAF
{
	/**
	* FEvaluateGraphContext
	*
	* Data required to perform an EvaluateGraph
	*
	*/
	struct FEvaluateGraphContext
	{
		FEvaluateGraphContext() = delete;
		FEvaluateGraphContext(FAnimNextGraphInstance& InGraphInstance, const FReferencePose& InRefPose, int32 InGraphLODLevel)
			: GraphInstance(InGraphInstance)
			, RefPose(InRefPose)
			, GraphLODLevel(InGraphLODLevel)
		{
		}

		FAnimNextGraphInstance& GetGraphInstance() const { return GraphInstance; }
		const FReferencePose& GetRefPose() const { return RefPose; }
		int32 GetGraphLODLevel() const { return GraphLODLevel; }

		// Only set if the new abstract skeleton runtime is enabled
		UAbstractSkeletonSetBinding* GetSkeletonSetBinding() const { return SkeletonSetBinding; }
		FName GetSkeletonSetName() const { return SkeletonSetName; }
		USkeletalMesh* GetSkeletalMesh() const { return SkeletalMesh; }

	protected:
		UE_API TWeakObjectPtr<const USkeletalMeshComponent> GetBindingObject() const;

		void SetSkeletonSet(UAbstractSkeletonSetBinding* InSkeletonSetBinding, USkeletalMesh* InSkeletalMesh, FName InSkeletonSetName)
		{
			SkeletonSetBinding = InSkeletonSetBinding;
			SkeletonSetName = InSkeletonSetName;
			SkeletalMesh = InSkeletalMesh;
		}

		FAnimNextGraphInstance& GraphInstance;

		const FReferencePose& RefPose;

		UAbstractSkeletonSetBinding* SkeletonSetBinding = nullptr;
		USkeletalMesh* SkeletalMesh = nullptr;
		FName SkeletonSetName;

		int32 GraphLODLevel = 0;

		friend UE_API FEvaluationProgram EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext, const FWeakTraitPtr& GraphRootPtr);
		friend struct ::FRigUnit_AnimNextRunAnimationGraph_v1;
		friend struct ::FRigUnit_AnimNextRunAnimationGraph_v2;
		friend struct ::FRigUnit_UAFRunAsset;
		friend struct ::FAnimNode_AnimNextGraph;
	};

	/**
	 * FEvaluateTraversalContext
	 *
	 * Contains all relevant transient data for an evaluate traversal and wraps the execution context.
	 */
	struct FEvaluateTraversalContext final : FExecutionContext
	{
		// Appends a new task into the evaluation program, tasks mutate state in the order they have been appended in
		// This means that child nodes need to evaluate first, tasks will usually be appended in IEvaluate::PostEvaluate
		// Tasks are moved into their final memory location, caller can allocate the task anywhere, it is no longer needed after this operation
		// @see FEvaluationProgram, FEvaluationTask, FEvaluationVM
		template<class TaskType>
		void AppendTask(TaskType&& Task) { EvaluationProgram.AppendTask(MoveTemp(Task)); }

		template<class TaskType>
		void AppendTaskPtr(const TSharedPtr<TaskType>& TaskPtr) { EvaluationProgram.AppendTaskPtr(TaskPtr); }

	private:
		explicit FEvaluateTraversalContext(FEvaluationProgram& InEvaluationProgram);

		FEvaluationProgram& EvaluationProgram;

		friend UE_API FEvaluationProgram EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext, const FWeakTraitPtr& GraphRootPtr);
	};

	/**
	 * IEvaluate
	 * 
	 * This interface is called during the evaluation traversal. It aims to produce an evaluation program.
	 * 
	 * When a node is visited, PreEvaluate is first called on its top trait. It is responsible for forwarding
	 * the call to the next trait that implements this interface on the trait stack of the node. Once
	 * all traits have had the chance to PreEvaluate, the children of the trait are queried through
	 * the IHierarchy interface. The children will then evaluate and PostEvaluate will then be called afterwards
	 * on the original trait.
	 * 
	 * The execution context contains what to evaluate.
	 * @see FEvaluationProgram
	 */
	struct IEvaluate : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IEvaluate)

		// Called before a traits children are evaluated
		UE_API virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const;

		// Called after a traits children have been evaluated
		UE_API virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IEvaluate> : FTraitBinding
	{
		// @see IEvaluate::PreEvaluate
		void PreEvaluate(FEvaluateTraversalContext& Context) const
		{
			GetInterface()->PreEvaluate(Context, *this);
		}

		// @see IEvaluate::PostEvaluate
		void PostEvaluate(FEvaluateTraversalContext& Context) const
		{
			GetInterface()->PostEvaluate(Context, *this);
		}

	protected:
		const IEvaluate* GetInterface() const { return GetInterfaceTyped<IEvaluate>(); }
	};

	/**
	 * Evaluates a sub-graph starting at its root and produces an evaluation program.
	 * Evaluation should be deterministic and repeated calls should yield the same evaluation program.
	 *
	 * For each node:
	 *     - We call PreEvaluate on all its traits
	 *     - We call GetChildren on all its traits
	 *     - We evaluate all children found
	 *     - We call PostEvaluate on all its traits
	 *
	 * @see IEvaluate::PreEvaluate, IEvaluate::PostEvaluate, IHierarchy::GetChildren
	 */
	[[nodiscard]] UE_API FEvaluationProgram EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext);

	/**
	 * Evaluates a sub-graph starting at its root and produces an evaluation program.
	 * Evaluation starts at the top of the stack that includes the graph root trait.
	 * Evaluation should be deterministic and repeated calls should yield the same evaluation program.
	 *
	 * For each node:
	 *     - We call PreEvaluate on all its traits
	 *     - We call GetChildren on all its traits
	 *     - We evaluate all children found
	 *     - We call PostEvaluate on all its traits
	 *
	 * @see IEvaluate::PreEvaluate, IEvaluate::PostEvaluate, IHierarchy::GetChildren
	 */
	[[nodiscard]] UE_API FEvaluationProgram EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext, const FWeakTraitPtr& GraphRootPtr);

	/**
	 * Evaluates a sub-graph starting at its root and produces an output pose
	 */
	UE_API void EvaluateGraph(const FEvaluateGraphContext& EvaluateGraphContext, FAnimNextGraphLODPose& OutputPose);

	// Returns whether or not we should use the new abstract skeleton runtime
	UE_API bool UseAbstractSkeletonRuntime();
}

#undef UE_API
