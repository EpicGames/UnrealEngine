// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

class FReferenceCollector;

namespace UE::UAF
{
	/**
	 * IGarbageCollection
	 * 
	 * This interface exposes garbage collection reference tracking.
	 */
	struct IGarbageCollection : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IGarbageCollection)

		// Called when garbage collection requests hard/strong object references
		// @see UObject::AddReferencedObjects
		UE_API virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const;

#if WITH_EDITOR
		virtual bool IsInternal() const override { return true; }

		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IGarbageCollection> : FTraitBinding
	{
		// @see IGarbageCollection::AddReferencedObjects
		void AddReferencedObjects(const FExecutionContext& Context, FReferenceCollector& Collector) const
		{
			GetInterface()->AddReferencedObjects(Context, *this, Collector);
		}

	protected:
		const IGarbageCollection* GetInterface() const { return GetInterfaceTyped<IGarbageCollection>(); }
	};
}

#undef UE_API
