// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/StructView.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "UAF/UAFAssetData.h"

#define UE_API UAFANIMGRAPH_API

class UUAFAnimGraph;
struct FAnimNextDataInterfacePayload;
struct FAnimNextFactoryParams;

namespace UE::UAF
{
	/**
	 * IGraphFactory
	 *
	 * This interface allows traits to participate in graph building via FAnimGraphFactory
	 */
	struct IGraphFactory : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IGraphFactory)

		// Called to modify/extend factory parameters for a generated graph.
		// Object used to generate the base graph supplied for context.
		// Fills in InOutParams with appropriately-initialized additive traits.
		UE_API virtual void GetFactoryParams(FExecutionContext& Context, const TTraitBinding<IGraphFactory>& Binding, FGraphAssetHandleConstView AssetHandle, FAnimNextFactoryParams& InOutParams) const;

		// If the supplied object is a UUAFAnimGraph, returns it
		// If not then uses the parameters (od the default parameters if they are not valid) combined with the current trait stack's collective calls
		// to GetFactoryParams to build the graph procedurally
		UE_API static const UUAFAnimGraph* GetOrBuildGraph(FExecutionContext& Context, const FTraitBinding& Binding, FGraphAssetHandleConstView AssetHandle, FAnimNextFactoryParams& InOutParams);

		UE_API static const UUAFAnimGraph* GetOrBuildGraph(FExecutionContext& Context, const FTraitBinding& Binding, const UObject* Asset, FAnimNextFactoryParams& InOutParams);

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IGraphFactory> : FTraitBinding
	{
		// @see IGraphFactory::GetFactoryParams
		void GetFactoryParams(FExecutionContext& Context, FGraphAssetHandleConstView AssetHandle, FAnimNextFactoryParams& InOutParams) const
		{
			GetInterface()->GetFactoryParams(Context, *this, AssetHandle, InOutParams);
		}

	protected:
		const IGraphFactory* GetInterface() const { return GetInterfaceTyped<IGraphFactory>(); }
	};
}

#undef UE_API
