// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "TraitInterfaces/ITimeline.h"

#include "IGroupSynchronization.generated.h"

#define UE_API UAFANIMGRAPH_API

class FMarkerTickContext;

/**
 * Synchronization Role
 * 
 * Timelines part of a named group will synchronize together. To do so, a leader is first identified.
 * The remaining members of the group will be followers. The synchronization role for a member of
 * the group determines which role it can play within that group.
 */
UENUM(BlueprintType)
enum class EAnimGroupSynchronizationRole : uint8
{
	/** This timeline host can be the leader, as long as it has a higher blend weight than the previous best leader. */
	CanBeLeader,
		
	/** This timeline host will always be a follower (unless there are only followers, in which case the first one ticked wins). */
	AlwaysFollower,

	/** This timeline host will always be a leader (if more than one timeline host is AlwaysLeader, the last one ticked wins). */
	AlwaysLeader,

	/** This timeline host will be excluded from the sync group while blending in. Once blended in it will be the sync group leader until blended out*/
	TransitionLeader,

	/** This timeline host will be excluded from the sync group while blending in. Once blended in it will be a follower until blended out*/
	TransitionFollower,

	/** This timeline host will always be a leader. If it fails to be ticked as a leader it will be run as ungrouped (EAnimGroupSynchronizationMode::NoSynchronization) .*/
	ExclusiveAlwaysLeader,
};

/**
 * Synchronization Mode
 * 
 * Timelines part of a named group can be synchronized together. When synchronization is enabled,
 * two options are presented:
 *     - A user specified group name can be used
 *     - A uniquely generated group name can be used
 * 
 * Having an explicitly specified group name is useful to synchronize different things together (e.g. two anim sequences).
 * However, sometimes we want certain things to synchronize only within themselves (e.g. blend space). This is called
 * self-synchronization. This arises when we want to specify the synchronization behavior at a higher level than individual
 * timelines. In such cases, we cannot use a user supplied group because every instance would then attempt to use the same
 * group name, which can be problematic. Consider a blend space embedded within a sub-graph that can blend with itself (e.g. is re-entrant).
 * This commonly occurs when blend stacks are used (e.g. through motion matching). We would like the anim sequences within
 * the blend space to self-synchronize but we do not want those to synchronize with other instances of the same blend space.
 * To solve this, every instance of the blend space needs a unique group name.
 */
UENUM(BlueprintType)
enum class EAnimGroupSynchronizationMode : uint8
{
	/** This timeline host will not synchronize. */
	NoSynchronization,

	/** This timeline host will synchronize and use the provided group name. */
	SynchronizeUsingGroupName,

	/** This timeline host will synchronize and use a uniquely generated group name. */
	SynchronizeUsingUniqueGroupName,
};

namespace UE::UAF
{
	// Encapsulates the parameters that control sync group behavior
	struct FSyncGroupParameters
	{
		// The group name to belong to (if None, syncing is disabled)
		FName GroupName;

		// The role within the group
		EAnimGroupSynchronizationRole GroupRole = EAnimGroupSynchronizationRole::CanBeLeader;

		// The synchronization mode
		EAnimGroupSynchronizationMode SyncMode = EAnimGroupSynchronizationMode::NoSynchronization;

		// Whether or not to match the group sync point when joining as leader or follower with markers
		// When disabled, the start position of synced sequences must be properly set to avoid pops
		bool bMatchSyncPoint = true;
	};

	/**
	 * IGroupSynchronization
	 *
	 * This interface exposes group synchronization related information and behavior.
	 */
	struct IGroupSynchronization : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IGroupSynchronization)

		// Returns the group parameters used for synchronization
		UE_API virtual FSyncGroupParameters GetGroupParameters(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding) const;

		// Called by the sync group graph instance component to advance time
		UE_API virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<IGroupSynchronization>& Binding, float DeltaTime, bool bDispatchEvents) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IGroupSynchronization> : FTraitBinding
	{
		// @see IGroupSynchronization::GetGroupParameters
		FSyncGroupParameters GetGroupParameters(FExecutionContext& Context) const
		{
			return GetInterface()->GetGroupParameters(Context, *this);
		}

		// @see IGroupSynchronization::AdvanceBy
		void AdvanceBy(FExecutionContext& Context, float DeltaTime, bool bDispatchEvents) const
		{
			GetInterface()->AdvanceBy(Context, *this, DeltaTime, bDispatchEvents);
		}

	protected:
		const IGroupSynchronization* GetInterface() const { return GetInterfaceTyped<IGroupSynchronization>(); }
	};
}

#undef UE_API
