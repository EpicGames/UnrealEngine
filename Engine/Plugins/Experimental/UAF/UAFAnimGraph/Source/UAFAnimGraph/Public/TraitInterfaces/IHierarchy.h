// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

namespace UE::UAF
{
	// An array of children pointers
	// We reserve a small amount inline and spill on the memstack
	using FChildrenArray = TArray<FWeakTraitPtr, TInlineAllocator<8, TMemStackAllocator<>>>;

	/**
	 * IHierarchy
	 * 
	 * This interface exposes hierarchy traversal information to navigate the graph.
	 */
	struct IHierarchy : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IHierarchy)

		// Returns the number of children of the trait implementation (not the whole stack)
		// Includes inactive children
		UAFANIMGRAPH_API virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const;

		// Returns the child trait to forward stack forwarding interface requests to
		// Default implementation will return the first child of single child hierarchies, and not forward at all for multi child hierarchies
		UAFANIMGRAPH_API virtual FWeakTraitPtr GetStackForwardingChild(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const;

		// Appends weak handles to any children we wish to traverse on the trait implementation (not the whole stack).
		// Traits are responsible for allocating and releasing child instance data.
		// Empty handles can be appended.
		UAFANIMGRAPH_API virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const;

		// Queries the trait stack and calls GetChildren for each trait, appending the result.
		UAFANIMGRAPH_API static void GetStackChildren(const FExecutionContext& Context, const FTraitStackBinding& Binding, FChildrenArray& Children);

		// Queries the trait stack of the specified binding and calls GetChildren for each trait, appending the result.
		static void GetStackChildren(const FExecutionContext& Context, const FTraitBinding& Binding, FChildrenArray& Children);

		// Queries the trait stack and calls GetNumChildren for each trait, accumulating the result.
		UAFANIMGRAPH_API static uint32 GetNumStackChildren(const FExecutionContext& Context, const FTraitStackBinding& Binding);

		// Queries the trait stack of the specified binding and calls GetNumChildren for each trait, accumulating the result.
		static uint32 GetNumStackChildren(const FExecutionContext& Context, const FTraitBinding& Binding);

		// Queries the trait stack and then its forwarding children for a trait implementing the specified interface.
		// The trait child found implementing the interface is returned
		template<class TraitInterface>
		static bool GetForwardedStackInterface(const FExecutionContext& Context, const FTraitStackBinding& StackBinding, FTraitStackBinding& OutStackBinding, TTraitBinding<TraitInterface>& OutBinding);

		// Queries the trait's forwarding children for a trait implementing the specified interface.
		// The first child found implementing the interface is returned
		template<class TraitInterface>
		static bool GetForwardedStackInterface(const FExecutionContext& Context, const FTraitBinding& Binding, FTraitStackBinding& OutStackBinding, TTraitBinding<TraitInterface>& OutBinding)
		{
			return Binding.IsValid() ? GetForwardedStackInterface(Context, *Binding.GetStack(), OutStackBinding, OutBinding) : false;
		}

	public:
#if WITH_EDITOR
		UAFANIMGRAPH_API virtual const FText& GetDisplayName() const override;
		UAFANIMGRAPH_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IHierarchy> : FTraitBinding
	{
		// @see IHierarchy::GetNumChildren
		uint32 GetNumChildren(const FExecutionContext& Context) const
		{
			return GetInterface()->GetNumChildren(Context, *this);
		}

		// @see IHierarchy::GetChildren
		void GetChildren(const FExecutionContext& Context, FChildrenArray& Children) const
		{
			GetInterface()->GetChildren(Context, *this, Children);
		}

		FWeakTraitPtr GetStackForwardingChild(const FExecutionContext& Context)
		{
			return GetInterface()->GetStackForwardingChild(Context, *this);
		}
		
	protected:
		const IHierarchy* GetInterface() const { return GetInterfaceTyped<IHierarchy>(); }
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementations

	inline void IHierarchy::GetStackChildren(const FExecutionContext& Context, const FTraitBinding& Binding, FChildrenArray& Children)
	{
		if (Binding.IsValid())
		{
			GetStackChildren(Context, *Binding.GetStack(), Children);
		}
	}

	inline uint32 IHierarchy::GetNumStackChildren(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		return Binding.IsValid() ? GetNumStackChildren(Context, *Binding.GetStack()) : 0;
	}
	template<class TraitInterface>
	inline bool IHierarchy::GetForwardedStackInterface(const FExecutionContext& Context, const FTraitStackBinding& StackBinding, FTraitStackBinding& OutStackBinding, TTraitBinding<TraitInterface>& OutBinding)
	{
		// Start with the supplied stack
		OutStackBinding = StackBinding;
		while (true)
		{
			if (OutStackBinding.GetInterface<TraitInterface>(OutBinding))
			{
				return true;
			}

			// Forward to child stack, if any
			// Visit the trait stack and query for a forwarding child
			FWeakTraitPtr ForwardingChild;

			TTraitBinding<IHierarchy> HierarchyTrait;
			OutStackBinding.GetInterface(HierarchyTrait);
			if (HierarchyTrait.IsValid())
			{
				ForwardingChild = HierarchyTrait.GetStackForwardingChild(Context);
			}

			if (!ForwardingChild.IsValid())
			{
				return false;
			}
			ensure(Context.GetStack(ForwardingChild, OutStackBinding));
		}
	}

}
