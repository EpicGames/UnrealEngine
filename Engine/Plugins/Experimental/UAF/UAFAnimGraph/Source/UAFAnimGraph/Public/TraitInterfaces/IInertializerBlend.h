// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	/**
	 * IInertializerBlend
	 *
	 * This interface exposes inertializing blend related information.
	 * 
	 * Inertialization is a process in which the delta between two blend sources is made to eventually be zero over time.
	 * It can be used as a form of blend smoothing.
	 * Examples: Storing pose deltas and fading out over time, dead blending.
	 */
	struct IInertializerBlend : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IInertializerBlend)

		// Returns the desired blend time for the specified child
		UE_API virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<IInertializerBlend>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IInertializerBlend> : FTraitBinding
	{
		// @see IInertializerBlend::GetBlendTime
		float GetBlendTime(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendTime(Context, *this, ChildIndex);
		}

	protected:
		const IInertializerBlend* GetInterface() const { return GetInterfaceTyped<IInertializerBlend>(); }
	};
}

#undef UE_API
