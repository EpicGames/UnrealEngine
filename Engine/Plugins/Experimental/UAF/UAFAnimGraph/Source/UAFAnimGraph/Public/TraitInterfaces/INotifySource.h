// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

struct FAnimNotifyEventReference;

namespace UE::UAF
{
	/**
	 * INotifySource
	 *
	 * This interface exposes a source of animation notify events
	 */
	struct INotifySource : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(INotifySource)

		// Returns the events of this timeline in the specified time interval
		UE_API virtual void GetNotifies(FExecutionContext& Context, const TTraitBinding<INotifySource>& Binding, float StartPosition, float Duration, bool bLooping, TArray<FAnimNotifyEventReference>& OutNotifies) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<INotifySource> : FTraitBinding
	{
		// @see INotifySource::GetNotifies
		void GetNotifies(FExecutionContext& Context, float StartPosition, float Duration, bool bLooping, TArray<FAnimNotifyEventReference>& OutNotifies) const
		{
			return GetInterface()->GetNotifies(Context, *this, StartPosition, Duration, bLooping, OutNotifies);
		}

	protected:
		const INotifySource* GetInterface() const { return GetInterfaceTyped<INotifySource>(); }
	};
}

#undef UE_API
