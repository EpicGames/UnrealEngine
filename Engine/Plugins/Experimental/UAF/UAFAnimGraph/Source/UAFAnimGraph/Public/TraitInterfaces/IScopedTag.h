// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/IScopedTraitInterface.h"
#include "TraitCore/TraitBinding.h"

namespace UE::UAF
{
struct IScopedTagInterface : IScopedTraitInterface
{
	DECLARE_ANIM_TRAIT_INTERFACE(IScopedTagInterface)

	virtual FName GetTag(const FExecutionContext& Context, const TTraitBinding<IScopedTagInterface>& Binding) const;

	static bool IsTagInScope(const FExecutionContext& Context, FName Tag);
};

template<>
struct TTraitBinding<IScopedTagInterface> : FTraitBinding
{
	FName GetTag(const FExecutionContext& Context) const
	{
		return GetInterface()->GetTag(Context, *this);
	}

protected:
	const IScopedTagInterface* GetInterface() const { return GetInterfaceTyped<IScopedTagInterface>(); }
};

inline FName IScopedTagInterface::GetTag(const FExecutionContext& Context, const TTraitBinding<IScopedTagInterface>& Binding) const
{
	TTraitBinding<IScopedTagInterface> SuperBinding;
	if (Binding.GetStackInterfaceSuper(SuperBinding))
	{
		return SuperBinding.GetTag(Context);
	}

	return NAME_None;
}

inline bool IScopedTagInterface::IsTagInScope(const FExecutionContext& Context, FName Tag)
{
	bool bResult = false;

	Context.ForEachScopedInterface<IScopedTagInterface>([&Context, Tag, &bResult](const TTraitBinding<IScopedTagInterface>& InterfaceBinding)
		{
			if (Tag == InterfaceBinding.GetTag(Context))
			{
				// We found our tag, stop iterating
				bResult = true;
				return false;
			}

			// Keep searching
			return true;
		});

	return bResult;
}
}
