// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

class UCurveFloat;

namespace UE::UAF
{
	/**
	 * ISmoothBlend
	 *
	 * This interface exposes blend smoothing related information.
	 * 
	 * Blend smoothing is a process that may be needed for discrete blends to make their pose transition appear continuous (IE: smooth).
	 */
	struct ISmoothBlend : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(ISmoothBlend)

		// Returns the desired blend time for the specified child
		UE_API virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const;

		// Returns the desired blend type for the specified child
		UE_API virtual EAlphaBlendOption GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const;

		// Returns the desired blend curve for the specified child
		UE_API virtual UCurveFloat* GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<ISmoothBlend> : FTraitBinding
	{
		// @see ISmoothBlend::GetBlendTime
		float GetBlendTime(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendTime(Context, *this, ChildIndex);
		}

		// @see ISmoothBlend::GetBlendType
		EAlphaBlendOption GetBlendType(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendType(Context, *this, ChildIndex);
		}

		// @see ISmoothBlend::GetCustomBlendCurve
		UCurveFloat* GetCustomBlendCurve(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetCustomBlendCurve(Context, *this, ChildIndex);
		}

	protected:
		const ISmoothBlend* GetInterface() const { return GetInterfaceTyped<ISmoothBlend>(); }
	};
}

#undef UE_API
