// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

class UCurveFloat;
class UUAFBlendProfile;

namespace UE::UAF
{
	/**
	 * ISmoothBlendPerBone
	 *
	 * This interface exposes blend smoothing per bone related information.
	 */
	struct ISmoothBlendPerBone : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(ISmoothBlendPerBone)

		// Returns the desired blend profile for the specified child
		UE_API virtual const UUAFBlendProfile* GetBlendProfile(FExecutionContext& Context, const TTraitBinding<ISmoothBlendPerBone>& Binding, int32 ChildIndex) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<ISmoothBlendPerBone> : FTraitBinding
	{
		// @see ISmoothBlendPerBone::GetBlendProfile
		const UUAFBlendProfile* GetBlendProfile(FExecutionContext& Context, int32 ChildIndex) const
		{
			return GetInterface()->GetBlendProfile(Context, *this, ChildIndex);
		}

	protected:
		const ISmoothBlendPerBone* GetInterface() const { return GetInterfaceTyped<ISmoothBlendPerBone>(); }
	};
}

#undef UE_API
