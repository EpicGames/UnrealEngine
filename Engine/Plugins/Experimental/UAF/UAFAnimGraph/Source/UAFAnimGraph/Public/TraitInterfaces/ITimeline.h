// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimSequenceBase.h"
#include <limits>
#include "Misc/MemStack.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	struct FTimelineSyncMarker
	{
		// Default construct empty
		FTimelineSyncMarker() = default;

		// Construct with a specific sync marker name and position
		FTimelineSyncMarker(FName InName, float InPosition)
			: Name(InName)
			, Position(InPosition)
		{
		}

		// Returns the sync marker name
		FName GetName() const { return Name; }

		// Returns the sync marker position (in seconds) on the timeline between [0.0, TimelineDuration]
		float GetPosition() const { return Position; }

	private:
		// The sync marker name
		FName Name;

		// The sync marker position
		float Position = 0.0f;
	};

	// An array of sync markers
	// We reserve a small amount inline and spill on the memstack
	using FTimelineSyncMarkerArray = TArray<FTimelineSyncMarker, TInlineAllocator<8, TMemStackAllocator<>>>;

	/**
	 * Timeline State
	 *
	 * Encapsulates the state of a timeline.
	 */
	struct FTimelineState
	{
		// Default construct with no state
		FTimelineState() = default;

		// Construct with a specific state
		FTimelineState(float InPosition, float InDuration, float InPlayRate, bool bInIsLooping)
			: Position(InPosition)
			, Duration(InDuration)
			, PlayRate(InPlayRate)
			, bIsLooping(bInIsLooping)
		{
		}

		// Resets the timeline state to its default state
		void Reset()
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			DebugName = NAME_None;
#endif

			Position = 0.0f;
			Duration = 0.0f;
			PlayRate = 0.0f;
			bIsLooping = false;
		}

		// Returns the timeline name
		FName GetDebugName() const
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			return DebugName;
#else
			return NAME_None;
#endif
		}

		// Returns the timeline position in seconds
		float GetPosition() const { return Position; }

		// Returns the timeline position as a ratio (0.0 = start of timeline, 1.0 = end of timeline)
		// For infinite timelines, this will return infinity
		float GetPositionRatio() const
		{
			if (IsFinite())
			{
				return Duration != 0.0f ? FMath::Clamp(Position / Duration, 0.0f, 1.0f) : 0.0f;
			}
			return std::numeric_limits<float>::infinity();
		}

		// Returns the time left to play in the timeline
		// For infinite timelines this will return infinity
		float GetTimeLeft() const
		{
			if (IsFinite())
			{
				return PlayRate >= 0.0f ? (Duration - Position) : Position;
			}
			return std::numeric_limits<float>::infinity();
		}

		// Returns the timeline duration in seconds
		// For infinite timelines this will return infinity
		float GetDuration() const { return Duration; }

		// Returns the play rate of the timeline
		float GetPlayRate() const { return PlayRate; }

		// Returns whether or not the timeline is looping
		bool IsLooping() const { return bIsLooping; }

		// Returns whether or not this timeline is finite
		bool IsFinite() const { return Duration != std::numeric_limits<float>::infinity(); }

		// Creates a new instance of the timeline state with the supplied debug name
		[[nodiscard]] FTimelineState WithDebugName(FName InDebugName) const
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			FTimelineState Result(*this);
			Result.DebugName = InDebugName;
			return Result;
#else
			return *this;
#endif
		}

		// Creates a new instance of the timeline state with the supplied position
		[[nodiscard]] FTimelineState WithPosition(float InPosition) const
		{
			FTimelineState Result(*this);
			Result.Position = InPosition;
			return Result;
		}

		// Creates a new instance of the timeline state with the supplied duration
		[[nodiscard]] FTimelineState WithDuration(float InDuration) const
		{
			FTimelineState Result(*this);
			Result.Duration = InDuration;
			return Result;
		}

		// Creates a new instance of the timeline state with the supplied play rate
		[[nodiscard]] FTimelineState WithPlayRate(float InPlayRate) const
		{
			FTimelineState Result(*this);
			Result.PlayRate = InPlayRate;
			return Result;
		}

		// Creates a new instance of the timeline state with the supplied looping flag
		[[nodiscard]] FTimelineState AsLooping(bool InAsLooping = true) const
		{
			FTimelineState Result(*this);
			Result.bIsLooping = InAsLooping;
			return Result;
		}

	private:
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// A debug name associated with this timeline
		FName DebugName;
#endif

		// Timeline position in seconds
		float Position = 0.0f;

		// Timeline duration in seconds
		float Duration = 0.0f;

		// Timeline play rate
		float PlayRate = 0.0f;

		// Timeline looping status
		bool bIsLooping = false;
	};

	/**
	 * Timeline Delta
	 *
	 * Encapsulates the delta state of a timeline, i.e. the changes that occured during the last AdvanceBy call.
	 */
	struct FTimelineDelta
	{
		// Default construct with no delta
		FTimelineDelta() = default;

		// Construct with a specific delta
		FTimelineDelta(float InDeltaTime, ETypeAdvanceAnim InAdvanceType)
			: DeltaTime(InDeltaTime)
			, AdvanceType(InAdvanceType)
		{
		}

		// Returns the delta time in seconds when last advanced
		float GetDeltaTime() const { return DeltaTime; }

		// Returns the result of the last advance
		ETypeAdvanceAnim GetAdvanceType() const { return AdvanceType; }

	private:
		// Delta time in seconds when last advanced
		float DeltaTime = 0.0f;

		// The result of the last advance
		ETypeAdvanceAnim AdvanceType = ETAA_Default;
	};

	/**
	 * ITimeline
	 *
	 * This interface exposes timeline related information.
	 */
	struct ITimeline : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(ITimeline)

		// Returns the sync markers of this timeline sorted by time
		UE_API virtual void GetSyncMarkers(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineSyncMarkerArray& OutSyncMarkers) const;

		// Returns the state of this timeline
		// It is possible for the timeline to be in an 'unknown' state if this is called before supporting data is copied/read, in which case this 
		// function will return false and the state will not be filled
		UE_API virtual bool GetState(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineState& OutTimelineState) const;

		// Returns the last change in state of this timeline
		// It is possible for the timeline to be in an 'unknown' state if this is called before supporting data is copied/read, in which case this 
		// function will return false and the delta will not be filled
		UE_API virtual bool GetDelta(const FExecutionContext& Context, const TTraitBinding<ITimeline>& Binding, FTimelineDelta& OutTimelineDelta) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<ITimeline> : FTraitBinding
	{
		// @see ITimeline::GetSyncMarkers
		void GetSyncMarkers(const FExecutionContext& Context, FTimelineSyncMarkerArray& OutSyncMarkers) const
		{
			return GetInterface()->GetSyncMarkers(Context, *this, OutSyncMarkers);
		}

		// @see ITimeline::GetState
		bool GetState(const FExecutionContext& Context, FTimelineState& OutTimelineState) const
		{
			return GetInterface()->GetState(Context, *this, OutTimelineState);
		}

		// @see ITimeline::GetDelta
		bool GetDelta(const FExecutionContext& Context, FTimelineDelta& OutTimelineDelta) const
		{
			return GetInterface()->GetDelta(Context, *this, OutTimelineDelta);
		}

	protected:
		const ITimeline* GetInterface() const { return GetInterfaceTyped<ITimeline>(); }
	};
}

#undef UE_API
