// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"

#define UE_API UAFANIMGRAPH_API

namespace UE::UAF
{
	/**
	 * ITimelinePlayer
	 *
	 * This interface exposes timeline playing behavior.
	 */
	struct ITimelinePlayer : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(ITimelinePlayer)

		// Advances time by the provided delta time (positive or negative) on this timeline player
		UE_API virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<ITimelinePlayer> : FTraitBinding
	{
		// @see ITimelinePlayer::AdvanceBy
		void AdvanceBy(FExecutionContext& Context, float DeltaTime, bool bDispatchEvents) const
		{
			GetInterface()->AdvanceBy(Context, *this, DeltaTime, bDispatchEvents);
		}

	protected:
		const ITimelinePlayer* GetInterface() const { return GetInterfaceTyped<ITimelinePlayer>(); }
	};
}

#undef UE_API
