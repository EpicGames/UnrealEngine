// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitEventList.h"

#include <type_traits>

#define UE_API UAFANIMGRAPH_API

class FMemStack;
struct FRigUnit_AnimNextRunAnimationGraph_v1;
struct FRigUnit_AnimNextRunAnimationGraph_v2;
struct FAnimNode_AnimNextGraph;

namespace UE::UAF
{
	struct FUpdateTraversalQueue;

	namespace Private
	{
		struct FUpdateEntry;
		struct FUpdateEventBookkeepingList;
		struct FUpdateEventBookkeepingEntry;
		enum class FUpdateEventBookkeepingAction : uint8;
	}

	/**
	* FUpdateGraphContext
	*
	* Data required to perform an UpdateGraph
	*
	*/
	struct FUpdateGraphContext
	{
		FUpdateGraphContext() = delete;
		FUpdateGraphContext(FAnimNextGraphInstance& InGraphInstance, float InDeltaTime)
			: GraphInstance(InGraphInstance)
			, DeltaTime(InDeltaTime)
		{
		}

		void PushInputEvent(FAnimNextTraitEventPtr Event) { InputEventList.Push(MoveTemp(Event)); }

		const FAnimNextGraphInstance& GetGraphInstance() const { return GraphInstance; }
		FAnimNextGraphInstance& GetGraphInstance() { return GraphInstance; }

		const FTraitEventList& GetInputEventList() const { return InputEventList; }
		FTraitEventList& GetInputEventList() { return InputEventList; }

		const FTraitEventList& GetOutputEventList() const { return OutputEventList; }
		FTraitEventList& GetOutputEventList() { return OutputEventList; }

		float GetDeltaTime() const { return DeltaTime; }

	protected:
		TWeakObjectPtr<const USkeletalMeshComponent> GetBindingObject() const;

		FAnimNextGraphInstance& GraphInstance;
		FTraitEventList InputEventList;
		FTraitEventList OutputEventList;
		float DeltaTime = 0.f;

		friend UAFANIMGRAPH_API void UpdateGraph(FUpdateGraphContext& UpdateGraphContext);
		friend struct ::FRigUnit_AnimNextRunAnimationGraph_v1;
		friend struct ::FRigUnit_AnimNextRunAnimationGraph_v2;
		friend struct ::FAnimNode_AnimNextGraph;
	};

	/**
	  * FTraitUpdateState
	  * 
	  * State that propagates down the graph during the update traversal.
	  * 
	  * We align this to 16 bytes and maintain that same size to ensure efficient copying.
	  */
	struct alignas(16) FTraitUpdateState final
	{
		FTraitUpdateState() = default;

		explicit FTraitUpdateState(float InDeltaTime)
			: DeltaTime(InDeltaTime)
		{}

		// Returns the delta time for this trait
		float GetDeltaTime() const { return DeltaTime; }

		// Returns the total weight for this trait
		float GetTotalWeight() const { return TotalWeight; }

		// Returns the total trajectory weight for this trait
		float GetTotalTrajectoryWeight() const { return TotalTrajectoryWeight; }

		// Returns whether or not this trait is blending out
		bool IsBlendingOut() const { return !!bIsBlendingOut; }

		// Returns whether or not this trait is newly relevant
		bool IsNewlyRelevant() const { return !!bIsNewlyRelevant; }

		// Creates a new instance of the update state with the total weight scaled by the supplied weight
		[[nodiscard]] FTraitUpdateState WithWeight(float Weight) const
		{
			FTraitUpdateState Result(*this);
			Result.TotalWeight *= Weight;
			return Result;
		}

		// Creates a new instance of the update state with the total trajectory weight scaled by the supplied weight
		[[nodiscard]] FTraitUpdateState WithTrajectoryWeight(float Weight) const
		{
			FTraitUpdateState Result(*this);
			Result.TotalTrajectoryWeight *= Weight;
			return Result;
		}

		// Creates a new instance of the update state with the delta time scaled by the supplied scale factor
		[[nodiscard]] FTraitUpdateState WithTimeScale(float TimeScale) const
		{
			FTraitUpdateState Result(*this);
			Result.DeltaTime *= TimeScale;
			return Result;
		}

		// Creates a new instance of the update state with the specified delta time
		[[nodiscard]] FTraitUpdateState WithDeltaTime(float InDeltaTime) const
		{
			FTraitUpdateState Result(*this);
			Result.DeltaTime = InDeltaTime;
			return Result;
		}

		// Creates a new instance of the update state marking it as blending out
		[[nodiscard]] FTraitUpdateState AsBlendingOut(bool InIsBlendingOut = true) const
		{
			FTraitUpdateState Result(*this);
			Result.bIsBlendingOut = InIsBlendingOut;
			return Result;
		}

		// Creates a new instance of the update state marking it as newly relevant
		[[nodiscard]] FTraitUpdateState AsNewlyRelevant(bool InIsNewlyRelevant = true) const
		{
			FTraitUpdateState Result(*this);
			Result.bIsNewlyRelevant = InIsNewlyRelevant;
			return Result;
		}

	private:
		// The amount of time to move forward by at allowing parents to time scale their children
		float DeltaTime = 0.0f;

		// The current total weight factoring in all inherited blend weights
		float TotalWeight = 1.0f;

		// The current total trajectory weight factoring in all inherited blend weights
		float TotalTrajectoryWeight = 1.0f;

		// Whether or not we are blending out
		int8 bIsBlendingOut = false;

		// Whether or not we newly became relevant
		int8 bIsNewlyRelevant = false;
	};

	static_assert(sizeof(FTraitUpdateState) <= 16, "Keep the size to 16 bytes for efficient copying");

	/**
	 * FUpdateTraversalContext
	 *
	 * Contains all relevant transient data for an update traversal and wraps the execution context.
	 */
	struct FUpdateTraversalContext final : FExecutionContext
	{
		// Raises an input trait event
		// The raised event will be seen by every child of the currently updating trait stack
		// Input trait events can only be raised during PreUpdate
		UE_API virtual void RaiseInputTraitEvent(FAnimNextTraitEventPtr Event) override;

		// Raises an output trait event
		// The raised event will be seen by every parent of the currently updating trait stack
		UE_API virtual void RaiseOutputTraitEvent(FAnimNextTraitEventPtr Event) override;

	private:
		// Constructs a new traversal context
		FUpdateTraversalContext() = default;

		// Pops entries from the traversal queue and pushes them onto the update stack
		void PushQueuedUpdateEntries(FUpdateTraversalQueue& TraversalQueue, Private::FUpdateEntry* ParentEntry);

		// Pushes an entry onto the update stack
		void PushUpdateEntry(Private::FUpdateEntry* Entry);

		// Pops an entry from the top of the update stack
		Private::FUpdateEntry* PopUpdateEntry();

		// Pushes an entry onto the free entry stack
		void PushFreeEntry(Private::FUpdateEntry* Entry);

		// Returns a new entry suitable for update queuing
		// If an entry isn't found in the free stack, a new one is allocated from the memstack
		Private::FUpdateEntry* GetNewEntry(const FWeakTraitPtr& TraitPtr, const FTraitUpdateState& TraitState);

		// Returns a new bookkeeping entry suitable for queuing
		// If an entry isn't found in the free stack, a new one is allocated from the memstack
		Private::FUpdateEventBookkeepingEntry* GetNewBookkeepingEntry(Private::FUpdateEventBookkeepingAction Action, FAnimNextTraitEventPtr Event);

		// Pushes a bookkeeping entry onto the free bookkeeping entry stack
		void PushFreeBookkeepingEntry(Private::FUpdateEventBookkeepingEntry* Entry);

		// Executes the event bookkeeping actions and clears the bookkeeping list
		void ExecuteBookkeepingActions(Private::FUpdateEventBookkeepingList& BookkeepingList);

		// The input and output event lists
		UE::UAF::FTraitEventList* InputEventList = nullptr;
		UE::UAF::FTraitEventList* OutputEventList = nullptr;

		// The currently executing entry
		Private::FUpdateEntry* ExecutingEntry = nullptr;

		// The head pointer of the update stack
		// This is the traversal execution stack and it contains entries that are
		// pending their pre-update call and entries waiting for post-update to be called
		Private::FUpdateEntry* UpdateStackHead = nullptr;

		// The head pointer of the free entry stack
		// Entries are allocated from the memstack and are re-used in LIFO since they'll
		// be warmer in the CPU cache
		Private::FUpdateEntry* FreeEntryStackHead = nullptr;

		// The head pointer of the free bookkeeping entry stack
		// Entries are allocated from the memstack and are re-used in LIFO since they'll
		// be warmer in the CPU cache
		Private::FUpdateEventBookkeepingEntry* FreeBookkeepingEntryStackHead = nullptr;

		// The root node doesn't have a parent but we need a bookkeeping list regardless
		Private::FUpdateEventBookkeepingList* RootParentBookkeepingEntryList = nullptr;

		friend UAFANIMGRAPH_API void UpdateGraph(FUpdateGraphContext& UpdateGraphContext);
		friend FUpdateTraversalQueue;
	};

	/**
	 * FUpdateTraversalQueue
	 *
	 * A queue of children to traverse.
	 * @see IUpdateTraversal::QueueChildrenForTraversal
	 */
	struct FUpdateTraversalQueue final
	{
		// Queued a child for traversal
		// Children are processed in the same order they are queued
		UAFANIMGRAPH_API void Push(const FWeakTraitPtr& ChildPtr, const FTraitUpdateState& ChildTraitState);

	private:
		explicit FUpdateTraversalQueue(FUpdateTraversalContext& InTraversalContext);

		// The current traversal context
		FUpdateTraversalContext& TraversalContext;

		// The head pointer of the queued update stack
		// When a child is queued for traversal, it is first appended to this stack
		// Once all children have been queued and pre-update terminates, this stack is
		// emptied and pushed onto the update stack
		Private::FUpdateEntry* QueuedUpdateStackHead = nullptr;

		friend UAFANIMGRAPH_API void UpdateGraph(FUpdateGraphContext& UpdateGraphContext);
		friend FUpdateTraversalContext;
	};

	/**
	 * IUpdate
	 *
	 * This interface is called during the update traversal.
	 *
	 * When a node is visited, PreUpdate is first called on its top trait. It is responsible for forwarding
	 * the call to the next trait that implements this interface on the trait stack of the node. Once
	 * all traits have had the chance to PreUpdate, the children will then evaluate and PostUpdate will
	 * then be called afterwards on the original trait.
	 */
	struct IUpdate : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IUpdate)

		// Called before the first update when a trait stack becomes relevant
		UE_API virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const;

		// Called before a traits children are updated
		UE_API virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const;

		// Called after a traits children have been updated
		UE_API virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const;
	
#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * IUpdateTraversal
	 *
	 * This interface is called during the update traversal.
	 *
	 * If a trait needs to modify the update state of its children, it needs to implement this interface.
	 * There is no need to call the Super to forward the call down the trait stack as the update traversal
	 * takes care of it.
	 * 
	 * If this interface is not implemented, the IHierarchy interface will be used to retrieve and queue
	 * the children with the same update state as the owning trait.
	 */
	struct IUpdateTraversal : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IUpdateTraversal)

		// Called after PreUpdate to request that children be queued with the provided context
		UE_API virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const;

#if WITH_EDITOR
		UE_API virtual const FText& GetDisplayName() const override;
		UE_API virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IUpdate> : FTraitBinding
	{
		// @see IUpdate::OnBecomeRelevant
		void OnBecomeRelevant(FUpdateTraversalContext& Context, const FTraitUpdateState& TraitState) const
		{
			GetInterface()->OnBecomeRelevant(Context, *this, TraitState);
		}

		// @see IUpdate::PreUpdate
		void PreUpdate(FUpdateTraversalContext& Context, const FTraitUpdateState& TraitState) const
		{
			GetInterface()->PreUpdate(Context, *this, TraitState);
		}

		// @see IUpdate::PostUpdate
		void PostUpdate(FUpdateTraversalContext& Context, const FTraitUpdateState& TraitState) const
		{
			GetInterface()->PostUpdate(Context, *this, TraitState);
		}

	protected:
		const IUpdate* GetInterface() const { return GetInterfaceTyped<IUpdate>(); }
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IUpdateTraversal> : FTraitBinding
	{
		// @see IUpdateTraversal::QueueChildrenForTraversal
		void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
		{
			GetInterface()->QueueChildrenForTraversal(Context, *this, TraitState, TraversalQueue);
		}

	protected:
		const IUpdateTraversal* GetInterface() const { return GetInterfaceTyped<IUpdateTraversal>(); }
	};

	/**
	 * Updates a sub-graph starting at its root.
	 *
	 * For each node:
	 *     - We call PreUpdate on all its traits
	 *     - We update all children
	 *     - We call PostUpdate on all its traits
	 * 
	 * During our update, we can append new input/output events.
	 *
	 * @see IUpdate::PreUpdate, IUpdate::PostUpdate, IHierarchy::GetChildren
	 */
	UAFANIMGRAPH_API void UpdateGraph(FUpdateGraphContext& UpdateGraphContext);
}

#undef UE_API
