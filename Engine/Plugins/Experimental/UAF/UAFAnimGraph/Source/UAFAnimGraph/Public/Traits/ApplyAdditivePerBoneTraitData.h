// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"

#include "ApplyAdditivePerBoneTraitData.generated.h"

class UUAFBlendMask;

namespace UE::UAF
{

// A trait that can apply a mesh or local space additive with an additional mask.
USTRUCT(meta = (DisplayName = "Apply Additive Per Bone", ShowTooltip=true))
struct FApplyAdditivePerBoneTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Base to apply additive to.
	UPROPERTY()
	FAnimNextTraitHandle Base;

	// Additive to apply.
	UPROPERTY()
	FAnimNextTraitHandle Additive;
	
	// Blend mask asset
	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<UUAFBlendMask> BlendMask = nullptr;
	
	// How much to apply of the additive 
	UPROPERTY(EditAnywhere, Category = "Default")
	float Alpha = 1.0f;
	
	// Whether to apply in mesh or local space
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bMeshSpaceBlend = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(Alpha) \
	GeneratorMacro(BlendMask) \
	GeneratorMacro(bMeshSpaceBlend) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FApplyAdditivePerBoneTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};
}
