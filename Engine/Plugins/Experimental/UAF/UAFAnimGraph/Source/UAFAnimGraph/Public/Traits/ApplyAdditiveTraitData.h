// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "ApplyAdditiveTraitData.generated.h"

/**
 * A trait that can apply a mesh or local space additive to this trait stack.
 * Ex: LookAt's, minor hit reactions, up-down floating, etc.
 */
USTRUCT(meta = (DisplayName = "Apply Additive", ShowTooltip=true))
struct FAnimNextApplyAdditiveTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** Base to apply additive too. */
	UPROPERTY()
	FAnimNextTraitHandle Base;

	/** Additive to apply. */
	UPROPERTY()
	FAnimNextTraitHandle Additive;

	/** 
	 * Deprecated. Please add an IAlphaInputArgs additive trait to set alpha.
	 * @TODO: Remove pre 5.6 once removing latents doesn't cause crash
	 * 
	 * How much to apply our additive, default is 1. 
	 */
	UPROPERTY(EditAnywhere, Category = "Default")
	float Alpha = 1.0f;
	
	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(Alpha) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextApplyAdditiveTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};