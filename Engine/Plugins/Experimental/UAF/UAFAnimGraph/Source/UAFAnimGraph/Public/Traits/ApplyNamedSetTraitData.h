// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "ApplyNamedSetTraitData.generated.h"

namespace UE::UAF
{
	/**
	 * A trait that applies a new named set to evaluate with.
	 * Traits upstream will use the new named set to evaluate.
	 * The trait that consumes its output must handle mismatched named sets (e.g. layering).
	 */
	USTRUCT(meta = (DisplayName = "Apply Named Set", ShowTooltip=true))
	struct FApplyNamedSetSharedData : public FAnimNextTraitSharedData
	{
		GENERATED_BODY()

		/** Input to evaluate with the specified set name */
		UPROPERTY()
		FAnimNextTraitHandle Input;

		/** Named set to evaluate our children with */
		UPROPERTY(Category=Settings, EditAnywhere)
		FName SetName = NAME_None;

		#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
			GeneratorMacro(SetName) \

		GENERATE_TRAIT_LATENT_PROPERTIES(FApplyNamedSetSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
		#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
	};
}
