// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "BlendLayerTraitData.generated.h"

class UUAFBlendMask;

/** A trait that can blend a layer into an input. */
USTRUCT(meta = (DisplayName = "Blend Layer", ShowTooltip=true))
struct FUAFBlendLayerTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimNextTraitHandle ChildBase;

	UPROPERTY()
	FAnimNextTraitHandle ChildBlend;

	// The strength with which to apply the blend pose
	UPROPERTY(EditAnywhere, Category = "Default")
	float BlendWeight = 1.0f;

	/** Blend mask that configures what bones are included in the blend. */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	TObjectPtr<UUAFBlendMask> BlendMask = nullptr;
	
	// Whether or not to blend in mesh space or local space
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bBlendInMeshSpace = false;
	
	// If true, this will still update the child branch even if the blend weight is 0
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bAlwaysUpdateChildBlend = false;

	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(BlendWeight) \
	GeneratorMacro(BlendMask) \
	GeneratorMacro(bBlendInMeshSpace) \
	GeneratorMacro(bAlwaysUpdateChildBlend) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFBlendLayerTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};