// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AlphaBlend.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/ISmoothBlend.h"
#include "TraitInterfaces/IUpdate.h"

#include "BlendSmoother.generated.h"

class UCurveFloat;

/**
 * A trait that smoothly blends between discrete states over time.
 * It only implements the smoothing logic, it queries its required arguments using the ISmoothBlend interface.
 */
USTRUCT(meta = (DisplayName = "Blend Smoother Core", ShowTooltip=true))
struct FAnimNextBlendSmootherCoreTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// This struct is empty but required so that we can give a nice display name to the trait
};

namespace UE::UAF
{
	// Namespaced alias
	using FBlendSmootherCoreData = FAnimNextBlendSmootherCoreTraitSharedData; 
}

/**
 * A trait that smoothly blends between discrete states over time.
 * This trait implements both the logic and contains the arguments necessary.
 */
USTRUCT(meta = (DisplayName = "Blend Smoother"))
struct FAnimNextBlendSmootherTraitSharedData : public FAnimNextBlendSmootherCoreTraitSharedData
{
	GENERATED_BODY()

	/** How long to take when blending into each child. */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	TArray<float> BlendTimes;

	/** What type of blend equation to use when converting the time elapsed into a blend weight. */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	EAlphaBlendOption BlendType = UE::Anim::DefaultBlendOption;

	/** Custom curve to use when the Custom blend type is used. */
	UPROPERTY(EditAnywhere, Category = "Default", meta = (Inline))
	TObjectPtr<UCurveFloat> CustomBlendCurve;
};

namespace UE::UAF
{
	/**
	 * FBlendSmootherCoreTrait
	 *
	 * A trait that smoothly blends between discrete states over time.
	 * It only implements the smoothing logic, it queries its required arguments using the ISmoothBlend interface.
	 */
	struct FBlendSmootherCoreTrait : FAdditiveTrait, IEvaluate, IUpdate, IDiscreteBlend
	{
		DECLARE_ANIM_TRAIT(FBlendSmootherCoreTrait, FAdditiveTrait)

		using FSharedData = FAnimNextBlendSmootherCoreTraitSharedData;

		// Struct for tracking blends for each pose
		struct FBlendData
		{
			// Helper struct to update a time based weight
			FAlphaBlend Blend;

			// Current child weight (normalized with all children)
			float Weight = 0.0f;

			// Whether or not this child is actively blending
			bool bIsBlending = false;
		};

		struct FInstanceData : FTrait::FInstanceData
		{
			// Blend state per child
			TArray<FBlendData> PerChildBlendData;
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IDiscreteBlend impl
		virtual float GetBlendWeight(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual const FAlphaBlend* GetBlendState(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 ChildIndex) const override;
		virtual void OnBlendTransition(FExecutionContext& Context, const TTraitBinding<IDiscreteBlend>& Binding, int32 OldChildIndex, int32 NewChildIndex) const override;

		// Internal impl
		static void InitializeInstanceData(FExecutionContext& Context, const FTraitBinding& Binding, const FSharedData* SharedData, FInstanceData* InstanceData);

#if WITH_EDITOR
		virtual bool IsHidden() const override { return true; }
#endif
	};

	/**
	 * FBlendSmootherTrait
	 * 
	 * A trait that smoothly blends between discrete states over time.
	 * This trait implements both the logic and contains the arguments necessary.
	 */
	struct FBlendSmootherTrait : FBlendSmootherCoreTrait, ISmoothBlend
	{
		DECLARE_ANIM_TRAIT(FBlendSmootherTrait, FBlendSmootherCoreTrait)

		using FSharedData = FAnimNextBlendSmootherTraitSharedData;

		// ISmoothBlend impl
		virtual float GetBlendTime(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;
		virtual EAlphaBlendOption GetBlendType(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;
		virtual UCurveFloat* GetCustomBlendCurve(FExecutionContext& Context, const TTraitBinding<ISmoothBlend>& Binding, int32 ChildIndex) const override;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return false; } // override the base, as it is hidden
#endif
	};
}
