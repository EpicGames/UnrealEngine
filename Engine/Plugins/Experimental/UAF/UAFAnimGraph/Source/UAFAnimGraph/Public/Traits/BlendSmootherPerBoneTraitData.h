// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitSharedData.h"

#include "BlendSmootherPerBoneTraitData.generated.h"

/** An additive trait that smoothly blends with per-bone weights. */
USTRUCT(meta = (DisplayName = "Blend Smoother Per Bone Core", ShowTooltip=true))
struct FAnimNextBlendSmootherPerBoneCoreTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()
};