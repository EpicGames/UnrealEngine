// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "Animation/BlendSpace.h"
#include "TraitCore/TraitBinding.h"
#include "Factory/AnimAssetParams.h"

#include "BlendSpacePlayerTraitData.generated.h"

/** A trait that can play a blend space */
USTRUCT(meta = (DisplayName = "Blend Space Player", ShowTooltip=true))
struct FAnimNextBlendSpacePlayerTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** The blend space to play. */
	UPROPERTY(EditAnywhere, Category = "Blend Space Player", meta = (ExportAsReference = "true", FactorySource))
	TObjectPtr<const UBlendSpace> BlendSpace;

	/** The location on the x-axis to sample. */
	UPROPERTY(EditAnywhere, Category = "Blend Space Player", DisplayName="X")
	float XAxisSamplePoint = 0.0f;

	/** The location on the y-axis to sample. */
	UPROPERTY(EditAnywhere, Category = "Blend Space Player", DisplayName="Y")
	float YAxisSamplePoint = 0.0f;

	/** The play rate multiplier at which this blend space plays. */
	UPROPERTY(EditAnywhere, Category = "Blend Space Player")
	float PlayRate = 1.0f;

	/** The time at which we should start playing this blend space. This is normalized in the [0,1] range. */
	UPROPERTY(EditAnywhere, Category = "Blend Space Player")
	float StartPosition = 0.0f;

	/** Whether to use the asset's default looping behavior or force an override. */
	UPROPERTY(EditAnywhere, Category = "Blend Space Player")
	UE::UAF::EAnimAssetLoopMode LoopMode = UE::UAF::EAnimAssetLoopMode::Auto;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(BlendSpace) \
		GeneratorMacro(XAxisSamplePoint) \
		GeneratorMacro(YAxisSamplePoint) \
		GeneratorMacro(PlayRate) \
		GeneratorMacro(StartPosition) \
		GeneratorMacro(LoopMode) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextBlendSpacePlayerTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	// Namespaced alias
	using FBlendSpacePlayerData = FAnimNextBlendSpacePlayerTraitSharedData;
}