// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "BlendTwoWayTraitData.generated.h"

/** A trait that can blend two inputs. */
USTRUCT(meta = (DisplayName = "Blend Two Way", ShowTooltip=true))
struct FAnimNextBlendTwoWayTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** First output to be blended (full weight is 0.0). */
	UPROPERTY()
	FAnimNextTraitHandle ChildA;

	/** Second output to be blended (full weight is 1.0). */
	UPROPERTY()
	FAnimNextTraitHandle ChildB;

	/** How much to blend our two children: 0.0 is fully child A while 1.0 is fully child B. */
	UPROPERTY(EditAnywhere, Category = "Default")
	float BlendWeight = 0.0f;

	/** This reinitializes child when re-activated */
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bResetChildOnActivation = true;

	/** Always update children, regardless of whether or not that child has weight. */
	UPROPERTY(EditAnywhere, Category = "Default")
	bool bAlwaysUpdateChildren = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(BlendWeight) \
	GeneratorMacro(bResetChildOnActivation) \
	GeneratorMacro(bAlwaysUpdateChildren) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextBlendTwoWayTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};