// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BoneMapping.generated.h"

USTRUCT(BlueprintType)
struct FAnimNextBoneMapping
{
	GENERATED_BODY()

	/** Bone to map from  */
	UPROPERTY(EditAnywhere, Category = "Default")
	FName SourceBone;

	/** Bone to map to */
	UPROPERTY(EditAnywhere, Category = "Default")
	FName TargetBone;
};