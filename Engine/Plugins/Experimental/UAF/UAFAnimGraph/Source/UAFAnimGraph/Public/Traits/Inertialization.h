// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitEvent.h"
#include "TransformArray.h"

#include "Inertialization.generated.h"

namespace UE::UAF
{
	/**
	 * FInertializationRequest
	 *
	 * Data contained in an InertializationRequest
	 */
	struct FInertializationRequest
	{
		// Request Blend Time
		float BlendTime = 0.0f;
	};

	/**
	 * FDeadBlendingState
	 *
	 * State data required by the DeadBlending node when extrapolating the pose.
	 */
	struct FDeadBlendingState
	{
		inline void Empty()
		{
			BoneRotationDirections.Empty();
			SourcePose.Empty();
			SourceBoneTranslationVelocities.Empty();
			SourceBoneRotationVelocities.Empty();
			SourceBoneScaleVelocities.Empty();
			SourceBoneTranslationDecayHalfLives.Empty();
			SourceBoneRotationDecayHalfLives.Empty();
			SourceBoneScaleDecayHalfLives.Empty();
		}

		inline void SetNumUninitialized(const int32 NewBoneNum)
		{
			BoneRotationDirections.SetNumUninitialized(NewBoneNum);
			SourcePose.SetNumUninitialized(NewBoneNum);
			SourceBoneTranslationVelocities.SetNumUninitialized(NewBoneNum);
			SourceBoneRotationVelocities.SetNumUninitialized(NewBoneNum);
			SourceBoneScaleVelocities.SetNumUninitialized(NewBoneNum);
			SourceBoneTranslationDecayHalfLives.SetNumUninitialized(NewBoneNum);
			SourceBoneRotationDecayHalfLives.SetNumUninitialized(NewBoneNum);
			SourceBoneScaleDecayHalfLives.SetNumUninitialized(NewBoneNum);
		}

		TArray<FQuat4f> BoneRotationDirections;
		FTransformArraySoAHeap SourcePose;
		TArray<FVector3f> SourceBoneTranslationVelocities;
		TArray<FVector3f> SourceBoneRotationVelocities;
		TArray<FVector3f> SourceBoneScaleVelocities;
		TArray<FVector3f> SourceBoneTranslationDecayHalfLives;
		TArray<FVector3f> SourceBoneRotationDecayHalfLives;
		TArray<FVector3f> SourceBoneScaleDecayHalfLives;
	};
}

/**
 * FAnimNextInertializationRequestEvent
 *
 * Inertialization Request Event Object
 */
USTRUCT()
struct FAnimNextInertializationRequestEvent : public FAnimNextTraitEvent
{
	GENERATED_BODY()
	DECLARE_ANIM_TRAIT_EVENT(FAnimNextInertializationRequestEvent, FAnimNextTraitEvent)

	UE::UAF::FInertializationRequest Request;
};
