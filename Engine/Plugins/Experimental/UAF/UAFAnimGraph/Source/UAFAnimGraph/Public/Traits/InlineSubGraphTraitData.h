// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"
#include "Variables/AnimNextVariableReference.h"

#include "InlineSubGraphTraitData.generated.h"

class UUAFAnimGraph;

/**
 * A single binding that connects a child node in this graph to a Value Bundle
 * variable on the inner sub-graph's interface.
 */
USTRUCT(meta=(Hidden))
struct FUAFInlineSubGraphInputBinding
{
	GENERATED_BODY()

	/** A child node in this graph whose output is passed into the inner sub-graph. */
	UPROPERTY(EditAnywhere, Category = "Sub Graph")
	FAnimNextTraitHandle Input;

	/**
	 * The Value Bundle variable on the inner sub-graph that receives the value from Input.
	 * Must match a variable declared on the inner sub-graph asset.
	 */
	UPROPERTY(EditAnywhere, Category = "Sub Graph", meta = (HideSubPins, AllowedType = FUAFValueBundle))
	FAnimNextVariableReference Variable;
};

/**
 * Hosts an inner sub-graph and exposes a set of child nodes from this graph as typed
 * FUAFValueBundle inputs on the inner sub-graph's interface.
 *
 * Each entry in Inputs maps a child node in this graph to a Value Bundle variable
 * declared on the inner sub-graph. When the inner sub-graph becomes active those values
 * are bound once and remain constant for the lifetime of that graph instance.
 *
 * The inner sub-graph can be swapped at runtime via the Graph pin - blending between instances is supported.
 */
USTRUCT(meta = (DisplayName = "Inline Sub Graph", ShowTooltip = true))
struct FUAFInlineSubGraphTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** The inner sub-graph asset to host. */
	UPROPERTY(EditAnywhere, Category = "Sub Graph", meta = (ExportAsReference = "true"))
	TObjectPtr<const UUAFAnimGraph> Graph;

	/** Hidden reference pose fallback child, auto-populated during graph compilation. */
	UPROPERTY(meta = (Hidden))
	FAnimNextTraitHandle ReferencePoseChild;

	/**
	 * Connections from child nodes in this graph to Value Bundle variables on the inner
	 * sub-graph's interface. The inner sub-graph must declare each referenced variable with
	 * type FUAFValueBundle.
	 */
	UPROPERTY(EditAnywhere, Category = "Sub Graph", meta = (OnBecomeRelevant))
	TArray<FUAFInlineSubGraphInputBinding> Inputs;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(Graph) \
	GeneratorMacro(Inputs) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFInlineSubGraphTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};
