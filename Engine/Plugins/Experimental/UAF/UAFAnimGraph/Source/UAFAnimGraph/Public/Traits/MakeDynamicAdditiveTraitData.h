// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"

#include "MakeDynamicAdditiveTraitData.generated.h"


namespace UE::UAF
{
	// A trait that creates a dynamic additive based input poses
	USTRUCT(meta = (DisplayName = "Make Dynamic Additive", ShowTooltip=true))
	struct FMakeDynamicAdditiveTraitSharedData : public FAnimNextTraitSharedData
	{
		GENERATED_BODY()

		// Reference pose for additive delta calculation
		UPROPERTY()
		FAnimNextTraitHandle Base;

		// Pose to make additive
		UPROPERTY()
		FAnimNextTraitHandle Additive;
		
		// Do additive delta calculation in mesh space
		UPROPERTY(EditAnywhere, Category="Default")
		bool bMeshSpaceAdditive = false;
		
		// Latent pin support boilerplate
		#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(bMeshSpaceAdditive) \

		GENERATE_TRAIT_LATENT_PROPERTIES(FMakeDynamicAdditiveTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
		#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
	};
}