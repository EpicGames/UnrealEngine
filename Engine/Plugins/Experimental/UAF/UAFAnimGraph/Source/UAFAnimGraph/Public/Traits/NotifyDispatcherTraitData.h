// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"
#include "NotifyDispatcherTraitData.generated.h"

/** A trait that dispatches notifies according to a timeline advancing */
USTRUCT(meta = (DisplayName = "Notify Dispatcher", ShowTooltip=true))
struct FAnimNextNotifyDispatcherTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro)

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextNotifyDispatcherTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	// Namespaced alias
	using FNotifyDispatcherData = FAnimNextNotifyDispatcherTraitSharedData;
}