// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/IEvaluate.h"
#include "Animation/InputScaleBias.h"
// --- ---
#include "PassthroughBlendTrait.generated.h"

/**
 * It will generate an output using the base trait input keyframe and the selected alpha blend type (float, bool or curve)
 * Requires a base trait with a single input and PostEvaluate task creation (currently used to blend Control Rig Trait)
 * 
 * If alpha type is Float or Bool
 * Depending on the weight value
 * - 0 (not relevant)          : The input "as is" will be the output (passthrough). No call to PostEvaluate on the base will be performed.
 * - 1 (full weight)           : The output of the base will be used as the output without blending by calling base PostEvaluate
 * - Any other relevant weight : Input will be duplicated (with a task) in Post Evaluate
 *								The base Post Evaluate will be called to execute on the copy (normally will create a task)
 *								Both keyframes (original and copy with base modifications) will be blended using a two keyframes blend task, passing the calculated alpha blend weight
 *
 * If alpha type is Curve
 * The curve value is not known until we are in the task (because we need the keyframe to extract the curve value)
 * Currently, the process will generate a full blend :	A task will added to duplicate input keyframe at PostEvaluate
 *														Base PostEvaluate will be called to allow task creation
 *														Resulting keyframes will be blended with a two keyframes blend task, passing the curve data so the alpha can be obtained
 * If we try to get current alpha value of a Curve blend type, the returned value will be 1
 */
USTRUCT(meta = (DisplayName = "Passthrough Blend", ShowTooltip=true))
struct FPassthroughBlendTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// alpha value handler
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Float"))
	float Alpha = 1.f;

	UPROPERTY(EditAnywhere, Category=Settings, meta = (Hidden, EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Float"))
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, DisplayAfter = "AlphaCurveName", EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Float || AlphaInputType == EAnimAlphaInputType::Curve"))
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "bEnabled", EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Bool"))
	bool bAlphaBoolEnabled = true;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (Hidden, DisplayName = "Blend Settings", EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Bool"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "AlphaInputType == EAnimAlphaInputType::Curve"))
	FName AlphaCurveName = NAME_None;

	UPROPERTY(EditAnywhere, Category = Settings)
	EAnimAlphaInputType AlphaInputType = EAnimAlphaInputType::Float;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \
		GeneratorMacro(bAlphaBoolEnabled) \
		GeneratorMacro(AlphaBoolBlend) \
		GeneratorMacro(AlphaCurveName) \
		GeneratorMacro(AlphaInputType) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FPassthroughBlendTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

struct FPassthroughBlendTrait : FAdditiveTrait, IUpdate, IEvaluate
{
	typedef FBaseTrait Super;

	DECLARE_ANIM_TRAIT(FPassthroughBlendTrait, FAdditiveTrait)

	using FSharedData = FPassthroughBlendTraitSharedData;
		
	struct FInstanceData : FTrait::FInstanceData
	{
		FInputAlphaBoolBlend AlphaBoolBlend;
		FInputScaleBiasClamp AlphaScaleBiasClamp;

		float DeltaTime = 0.f;
		float ComputedAlphaValue = 0.f;

		void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
	};

	// IUpdate
	virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	static float ComputeAlphaValue(const EAnimAlphaInputType AlphaInputType, const FPassthroughBlendTraitSharedData* SharedData, FInstanceData* InstanceData, const TTraitBinding<IUpdate>& Binding, float DeltaTime);
};

} // namespace UE::UAF
