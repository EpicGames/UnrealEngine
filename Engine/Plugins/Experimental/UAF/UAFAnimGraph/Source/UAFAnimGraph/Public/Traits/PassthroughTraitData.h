// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "PassthroughTraitData.generated.h"

/** A trait that passes through the input without modification. */
USTRUCT(meta = (DisplayName = "Passthrough", ShowTooltip=true))
struct FAnimNextPassthroughSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** Input to pass to output */
	UPROPERTY()
	FAnimNextTraitHandle Input;
};