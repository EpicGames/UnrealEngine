// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "Animation/AnimSequence.h"
#include "Factory/AnimAssetParams.h"
#include "SequencePlayerTraitData.generated.h"

/** A trait that can play an animation sequence. */
USTRUCT(meta = (DisplayName = "Sequence Player", ShowTooltip=true))
struct FAnimNextSequencePlayerTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** The sequence to play. */
	UPROPERTY(EditAnywhere, Category = "Sequence Player", meta=(ExportAsReference="true", FactorySource, OnBecomeRelevant))
	TObjectPtr<const UAnimSequence> AnimSequence;

	/** The play rate multiplier at which this sequence plays. */
	UPROPERTY(EditAnywhere, Category = "Sequence Player")
	float PlayRate = 1.0f;

	/** The time at which we should start playing this sequence. */
	UPROPERTY(EditAnywhere, Category = "Sequence Player", meta = (OnBecomeRelevant))
	float StartPosition = 0.0f;

	/** Whether to use the asset's default looping behavior or force an override. */
	UPROPERTY(EditAnywhere, Category = "Sequence Player", meta = (OnBecomeRelevant))
	UE::UAF::EAnimAssetLoopMode LoopMode = UE::UAF::EAnimAssetLoopMode::Auto;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(AnimSequence) \
		GeneratorMacro(PlayRate) \
		GeneratorMacro(StartPosition) \
		GeneratorMacro(LoopMode) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextSequencePlayerTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	// Namespaced alias
	using FSequencePlayerData = FAnimNextSequencePlayerTraitSharedData;
}