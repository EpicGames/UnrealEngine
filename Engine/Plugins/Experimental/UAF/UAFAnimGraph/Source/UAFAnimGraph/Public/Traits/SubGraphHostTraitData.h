// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "SubGraphHostTraitData.generated.h"

class UUAFAnimGraph;

/** A trait that hosts and manages a sub-graph instance. */
USTRUCT(meta = (DisplayName = "SubGraph", ShowTooltip=true))
struct FAnimNextSubGraphHostTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	/** Asset to use as a sub-graph */
	UPROPERTY(EditAnywhere, Category = "Default", meta=(ExportAsReference="true"))
	TObjectPtr<const UUAFAnimGraph> AnimationGraph;

	/** A dummy child that we can use to output the bind pose. This property is hidden and automatically populated during compilation. */
	UPROPERTY(meta = (Hidden))
	FAnimNextTraitHandle ReferencePoseChild;

	/** Entry point in the Subgraph that we will use */
	UPROPERTY(EditAnywhere, Category = "Default", meta=(CustomWidget = "ParamName", AllowedParamType = "FAnimNextEntryPoint"))
	FName EntryPoint;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(AnimationGraph) \
	GeneratorMacro(EntryPoint) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextSubGraphHostTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};