// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"
#include "TraitInterfaces/IGroupSynchronization.h"
#include "SynchronizeUsingGroupsTraitData.generated.h"

/** A trait that synchronizes animation sequence playback using named groups. */
USTRUCT(meta = (DisplayName = "Synchronize Using Groups"))
struct FAnimNextSynchronizeUsingGroupsTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// The group name
	// If no name is provided, this trait is inactive
	UPROPERTY(EditAnywhere, Category = "Synchronize Using Groups")
	FName GroupName;

	// The role this player can assume within the group
	UPROPERTY(EditAnywhere, Category = "Synchronize Using Groups")
	EAnimGroupSynchronizationRole GroupRole = EAnimGroupSynchronizationRole::CanBeLeader;

	// The synchronization mode
	UPROPERTY(EditAnywhere, Category = "Synchronize Using Groups")
	EAnimGroupSynchronizationMode SyncMode = EAnimGroupSynchronizationMode::NoSynchronization;

	// Whether or not to match the group sync point when joining as leader or follower with markers
	// When disabled, the start position of synced sequences must be properly set to avoid pops
	UPROPERTY(EditAnywhere, Category = "Synchronize Using Groups")
	bool bMatchSyncPoint = true;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(GroupName) \
		GeneratorMacro(GroupRole) \
		GeneratorMacro(SyncMode) \
		GeneratorMacro(bMatchSyncPoint) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextSynchronizeUsingGroupsTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	// Namespaced alias
	using FSynchronizeUsingGroupsData = FAnimNextSynchronizeUsingGroupsTraitSharedData;
}