// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFAnimGraph : ModuleRules
	{
		public UAFAnimGraph(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"RigVM",
					"Engine",
					"UAF",
					"HierarchyTableRuntime",
					"HierarchyTableAnimationRuntime",
					"TraceLog",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"AssetRegistry",
						"RigVMDeveloper",
					}
				);
			}

			bAllowUETypesInNamespaces = true;
		}
	}
}