<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * UAF Visualizers
  *
  -->

  <!-- Trait visualizers -->

  <Type Name="UE::UAF::FNodeTemplate">
    <DisplayString>UID={UID,x}, NumTraits={(unsigned int)NumTraits}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[UID]">UID,x</Item>
      <Item Name="[NumTraits]">(unsigned int)NumTraits</Item>
      <Item Name="[SharedSize]">NodeSharedDataSize</Item>
      <Item Name="[InstanceSize]">NodeInstanceDataSize</Item>
      <ArrayItems>
        <Size>NumTraits</Size>
        <ValuePointer>(UE::UAF::FTraitTemplate*)((const char*)(this) + sizeof(FNodeTemplate))</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FTraitTemplate">
    <DisplayString>UID={UID,x}, Handle={RegistryHandle}, Mode={(UE::UAF::ETraitMode)Mode,en}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[UID]">UID,x</Item>
      <Item Name="[TraitHandle]">RegistryHandle</Item>
      <Item Name="[Mode]">(UE::UAF::ETraitMode)Mode,en</Item>
      <Item Name="[NumStackTraits]" Condition="Mode == UE::UAF::ETraitMode::Base">(unsigned int)TraitIndexOrNumTraits</Item>
      <Item Name="[TraitIndex]" Condition="Mode == UE::UAF::ETraitMode::Base">(unsigned int)0</Item>
      <Item Name="[TraitIndex]" Condition="Mode == UE::UAF::ETraitMode::Additive">(unsigned int)TraitIndexOrNumTraits</Item>
      <Item Name="[NumLatentProperties]">NumLatentProperties</Item>
      <Item Name="[NumSubStackLatentProperties]" Condition="Mode == UE::UAF::ETraitMode::Base">NumSubStackLatentProperties</Item>
      <Item Name="[NodeSharedOffset]">NodeSharedOffset</Item>
      <Item Name="[NodeSharedLatentPropertyHandlesOffset]">NodeSharedLatentPropertyHandlesOffset</Item>
      <Item Name="[NodeInstanceOffset]">NodeInstanceOffset</Item>
    </Expand>
  </Type>

  <!-- If we can't find the trait registry under the UnrealEditor-UAFAnimGraph module, try the normal resolution process -->
  <Type Name="UE::UAF::FTraitRegistryHandle">
    <DisplayString Condition="HandleValue &gt; 0">Static={HandleValue - 1}</DisplayString>
    <DisplayString Condition="HandleValue &lt; 0">Dynamic={-HandleValue - 1}</DisplayString>
    <DisplayString>Invalid</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[StaticOffset]" Condition="HandleValue &gt; 0">HandleValue - 1</Item>
      <Item Name="[DynamicIndex]" Condition="HandleValue &lt; 0">-HandleValue - 1</Item>
      <Item Name="[Trait]" Condition="HandleValue &gt; 0">(UE::UAF::FTrait*)&amp;UE::UAF::Private::GTraitRegistry->StaticTraitBuffer[HandleValue - 1]</Item>
      <Item Name="[Trait]" Condition="HandleValue &lt; 0">(UE::UAF::FTrait*)(((uintptr_t*)UE::UAF::Private::GTraitRegistry->DynamicTraits.AllocatorInstance.Data)[-HandleValue - 1])</Item>
    </Expand>
  </Type>

  <!-- We first look under the UnrealEditor-UAFAnimGraph module, otherwise it might not be found -->
  <Type Name="UE::UAF::FTraitRegistryHandle" Priority="High">
    <DisplayString Condition="HandleValue &gt; 0">Static={HandleValue - 1}</DisplayString>
    <DisplayString Condition="HandleValue &lt; 0">Dynamic={-HandleValue - 1}</DisplayString>
    <DisplayString>Invalid</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[StaticOffset]" Condition="HandleValue &gt; 0">HandleValue - 1</Item>
      <Item Name="[DynamicIndex]" Condition="HandleValue &lt; 0">-HandleValue - 1</Item>
      <Item Name="[Trait]" Condition="HandleValue &gt; 0">(UE::UAF::FTrait*)&amp;UnrealEditor-UAFAnimGraph!UE::UAF::Private::GTraitRegistry->StaticTraitBuffer[HandleValue - 1]</Item>
      <Item Name="[Trait]" Condition="HandleValue &lt; 0">(UE::UAF::FTrait*)(((uintptr_t*)UnrealEditor-UAFAnimGraph!UE::UAF::Private::GTraitRegistry->DynamicTraits.AllocatorInstance.Data)[-HandleValue - 1])</Item>
    </Expand>
  </Type>

  <!-- If we can't find the node template registry under the UnrealEditor-UAFAnimGraph module, try the normal resolution process -->
  <Type Name="UE::UAF::FNodeTemplateRegistryHandle">
    <DisplayString Condition="TemplateOffset == 0">Invalid</DisplayString>
    <DisplayString Condition="TemplateOffset &gt; 0">{TemplateOffset - 1}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[TemplateOffset]" Condition="TemplateOffset &gt; 0">TemplateOffset - 1</Item>
      <Item Name="[Template]" Condition="TemplateOffset &gt; 0">(UE::UAF::FNodeTemplate*)&amp;((uint8*)UE::UAF::Private::GNodeTemplateRegistry->TemplateBuffer.AllocatorInstance.Data)[TemplateOffset - 1]</Item>
    </Expand>
  </Type>

  <!-- We first look under the UnrealEditor-UAFAnimGraph module, otherwise it might not be found -->
  <Type Name="UE::UAF::FNodeTemplateRegistryHandle" Priority="High">
    <DisplayString Condition="TemplateOffset == 0">Invalid</DisplayString>
    <DisplayString Condition="TemplateOffset &gt; 0">{TemplateOffset - 1}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[TemplateOffset]" Condition="TemplateOffset &gt; 0">TemplateOffset - 1</Item>
      <Item Name="[Template]" Condition="TemplateOffset &gt; 0">(UE::UAF::FNodeTemplate*)&amp;((uint8*)UnrealEditor-UAFAnimGraph!UE::UAF::Private::GNodeTemplateRegistry->TemplateBuffer.AllocatorInstance.Data)[TemplateOffset - 1]</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FNodeID">
    <DisplayString Condition="ID == INVALID_NODE_ID">Invalid</DisplayString>
    <DisplayString>{ID - 1}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FNodeHandle">
    <DisplayString Condition="(PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK) == INVALID_NODE_HANDLE_RAW_VALUE">Invalid</DisplayString>
    <DisplayString Condition="(PackedSharedOffsetOrID &amp; NODE_ID_MARKER_BIT_MASK) == 0">SharedOffset={PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK}</DisplayString>
    <DisplayString>NodeIndex={((PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK) &gt;&gt; NODE_ID_SHIFT) - 1}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[SharedOffset]" Condition="(PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK) != INVALID_NODE_HANDLE_RAW_VALUE &amp;&amp; (PackedSharedOffsetOrID &amp; NODE_ID_MARKER_BIT_MASK) == 0">PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK</Item>
      <Item Name="[NodeIndex]" Condition="(PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK) != INVALID_NODE_HANDLE_RAW_VALUE &amp;&amp; (PackedSharedOffsetOrID &amp; NODE_ID_MARKER_BIT_MASK) == 1">((PackedSharedOffsetOrID &amp; NODE_HANDLE_MASK) &gt;&gt; NODE_ID_SHIFT) - 1</Item>
      <Item Name="[Raw]">PackedSharedOffsetOrID,x</Item>
    </Expand>
  </Type>

  <Type Name="FAnimNextTraitHandle">
    <DisplayString Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) == UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE">Invalid</DisplayString>
    <DisplayString Condition="(PackedTraitIndexAndNodeHandle &amp; UE::UAF::FNodeHandle::NODE_ID_MARKER_BIT_MASK) == 0">NodeSharedOffset={PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK}, TraitIndex={PackedTraitIndexAndNodeHandle &gt;&gt; TRAIT_INDEX_SHIFT}</DisplayString>
    <DisplayString>NodeIndex={((PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) &gt;&gt; UE::UAF::FNodeHandle::NODE_ID_SHIFT) - 1}, TraitIndex={PackedTraitIndexAndNodeHandle &gt;&gt; TRAIT_INDEX_SHIFT}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[NodeSharedOffset]" Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) != UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE &amp;&amp; (PackedTraitIndexAndNodeHandle &amp; UE::UAF::FNodeHandle::NODE_ID_MARKER_BIT_MASK) == 0">PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK</Item>
      <Item Name="[NodeIndex]" Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) != UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE &amp;&amp; (PackedTraitIndexAndNodeHandle &amp; UE::UAF::FNodeHandle::NODE_ID_MARKER_BIT_MASK) == 1">((PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) &gt;&gt; UE::UAF::FNodeHandle::NODE_ID_SHIFT) - 1</Item>
      <Item Name="[TraitIndex]" Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) != UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE">PackedTraitIndexAndNodeHandle &gt;&gt; TRAIT_INDEX_SHIFT</Item>
      <Item Name="[Raw]">PackedTraitIndexAndNodeHandle,x</Item>
    </Expand>
  </Type>

  <Type Name="FAnimNextEntryPointHandle">
    <DisplayString Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) == UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE">Invalid</DisplayString>
    <DisplayString>NodeIndex={((PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) &gt;&gt; UE::UAF::FNodeHandle::NODE_ID_SHIFT) - 1}, TraitIndex={PackedTraitIndexAndNodeHandle &gt;&gt; TRAIT_INDEX_SHIFT}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[NodeIndex]" Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) != UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE &amp;&amp; (PackedTraitIndexAndNodeHandle &amp; UE::UAF::FNodeHandle::NODE_ID_MARKER_BIT_MASK) == 1">((PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) &gt;&gt; UE::UAF::FNodeHandle::NODE_ID_SHIFT) - 1</Item>
      <Item Name="[TraitIndex]" Condition="(PackedTraitIndexAndNodeHandle &amp; NODE_HANDLE_MASK) != UE::UAF::FNodeHandle::INVALID_NODE_HANDLE_RAW_VALUE">PackedTraitIndexAndNodeHandle &gt;&gt; TRAIT_INDEX_SHIFT</Item>
      <Item Name="[Raw]">PackedTraitIndexAndNodeHandle,x</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FTraitPtr">
    <DisplayString Condition="PackedPointerAndFlags == 0">Invalid</DisplayString>
    <DisplayString Condition="PackedPointerAndFlags != 0 &amp;&amp; (PackedPointerAndFlags &amp; IS_WEAK_BIT) == 0">[Strong] {(void*)(PackedPointerAndFlags &amp; ~FLAGS_MASK)} TraitIndex={(unsigned int)TraitIndex} NodeHandle={((FNodeInstance*)(PackedPointerAndFlags &amp; ~FLAGS_MASK))->NodeHandle}</DisplayString>
    <DisplayString Condition="PackedPointerAndFlags != 0 &amp;&amp; (PackedPointerAndFlags &amp; IS_WEAK_BIT) != 0">[Weak] {(void*)(PackedPointerAndFlags &amp; ~FLAGS_MASK)} TraitIndex={(unsigned int)TraitIndex} NodeHandle={((FNodeInstance*)(PackedPointerAndFlags &amp; ~FLAGS_MASK))->NodeHandle}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[IsWeak]" Condition="PackedPointerAndFlags != 0">(PackedPointerAndFlags &amp; IS_WEAK_BIT) != 0</Item>
      <Item Name="[TraitIndex]" Condition="PackedPointerAndFlags != 0">(unsigned int)TraitIndex</Item>
      <Item Name="[NodeHandle]" Condition="PackedPointerAndFlags != 0">((FNodeInstance*)(PackedPointerAndFlags &amp; ~FLAGS_MASK))->NodeHandle</Item>
      <Item Name="[NodeInstance]" Condition="PackedPointerAndFlags != 0">(FNodeInstance*)(PackedPointerAndFlags &amp; ~FLAGS_MASK)</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FWeakTraitPtr">
    <DisplayString Condition="NodeInstance == nullptr">Invalid</DisplayString>
    <DisplayString Condition="NodeInstance != nullptr">[Weak] {(void*)NodeInstance} TraitIndex={(unsigned int)TraitIndex} NodeHandle={NodeInstance->NodeHandle}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[IsWeak]">true</Item>
      <Item Name="[TraitIndex]" Condition="NodeInstance != nullptr">(unsigned int)TraitIndex</Item>
      <Item Name="[NodeHandle]" Condition="NodeInstance != nullptr">NodeInstance->NodeHandle</Item>
      <Item Name="[NodeInstance]" Condition="NodeInstance != nullptr">NodeInstance</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FTraitInterfaceUID">
    <DisplayString Condition="UID == 0">Invalid</DisplayString>
    <DisplayString Condition="UID != 0 &amp;&amp; InterfaceName == nullptr">Interface UID={UID,x}</DisplayString>
    <DisplayString Condition="UID != 0 &amp;&amp; InterfaceName != nullptr">Interface UID={UID,x}, Name={InterfaceName,su}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FTraitUID">
    <DisplayString Condition="UID == 0">Invalid</DisplayString>
    <DisplayString Condition="UID != 0 &amp;&amp; TraitName == nullptr">Trait UID={UID,x}</DisplayString>
    <DisplayString Condition="UID != 0 &amp;&amp; TraitName != nullptr">Trait UID={UID,x}, Name={TraitName,su}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FTraitStackBinding">
    <DisplayString Condition="Context == nullptr">Invalid</DisplayString>
    <DisplayString>NumTraits={TopTraitIndex - BaseTraitIndex + 1}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[ExecutionContext]">Context</Item>
      <Item Name="[NodeInstance]">NodeInstance</Item>
      <Item Name="[NodeDescription]">NodeDescription</Item>
      <Item Name="[NodeTemplate]">NodeTemplate</Item>
      <Item Name="[BaseTraitIndex]">BaseTraitIndex</Item>
      <Item Name="[NumTraits]" Condition="Context != nullptr">TopTraitIndex - BaseTraitIndex + 1</Item>
      <Item Name="[NumTraits]" Condition="Context == nullptr">0</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FTraitBinding">
    <DisplayString Condition="TraitImpl == nullptr">Invalid</DisplayString>
    <DisplayString>TraitIndex={TraitIndex - Stack->BaseTraitIndex}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[Stack]">Stack</Item>
      <Item Name="[Trait]">TraitImpl</Item>
      <Item Name="[TraitIndex]" Condition="TraitImpl != nullptr">TraitIndex - Stack->BaseTraitIndex</Item>
      <Item Name="[InterfaceThisOffset]">InterfaceThisOffset</Item>
    </Expand>
  </Type>

  <!-- If we can't find the interface under the UnrealEditor-UAFAnimGraph module, try the normal resolution process -->
  <Type Name="UE::UAF::TTraitBinding&lt;*&gt;">
    <DisplayString Condition="Interface == nullptr">Invalid</DisplayString>
    <DisplayString Condition="Interface != nullptr">{$T1::InterfaceUID}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[Interface]" Condition="Interface != nullptr">($T1*)Interface</Item>
      <Item Name="[TraitPtr]" Condition="Interface != nullptr">TraitPtr</Item>
      <Item Name="[SharedData]" Condition="Interface != nullptr">SharedData</Item>
      <Item Name="[InstanceData]" Condition="Interface != nullptr">InstanceData</Item>
    </Expand>
  </Type>

  <!-- We first look under the UnrealEditor-UAFAnimGraph module, otherwise it might not be found -->
  <Type Name="UE::UAF::TTraitBinding&lt;*&gt;" Priority="High">
    <DisplayString Condition="Interface == nullptr">Invalid</DisplayString>
    <DisplayString Condition="Interface != nullptr">{UnrealEditor-UAFAnimGraph!$T1::InterfaceUID}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="[Interface]" Condition="Interface != nullptr">(UnrealEditor-UAFAnimGraph!$T1*)Interface</Item>
      <Item Name="[TraitPtr]" Condition="Interface != nullptr">TraitPtr</Item>
      <Item Name="[SharedData]" Condition="Interface != nullptr">SharedData</Item>
      <Item Name="[InstanceData]" Condition="Interface != nullptr">InstanceData</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FLatentPropertyHandle">
    <DisplayString Condition="((PackedIndexCanFreezeValue &amp; HANDLE_INDEX_MASK) == INVALID_HANDLE_VALUE) &amp;&amp; (OffsetValue == INVALID_OFFSET_VALUE)">Invalid</DisplayString>
    <DisplayString Condition="((PackedIndexCanFreezeValue &amp; HANDLE_INDEX_MASK) == INVALID_HANDLE_VALUE)">Cached Only, Offset={OffsetValue}, CanFreeze={PackedIndexCanFreezeValue &gt;&gt; HANDLE_CAN_FREEZE_SHIFT_OFFSET}</DisplayString>
    <DisplayString>Index={PackedIndexCanFreezeValue &amp; HANDLE_INDEX_MASK}, Offset={OffsetValue}, CanFreeze={PackedIndexCanFreezeValue &gt;&gt; HANDLE_CAN_FREEZE_SHIFT_OFFSET}</DisplayString>
    <Expand  HideRawView="true">
    </Expand>
  </Type>

</AutoVisualizer>
