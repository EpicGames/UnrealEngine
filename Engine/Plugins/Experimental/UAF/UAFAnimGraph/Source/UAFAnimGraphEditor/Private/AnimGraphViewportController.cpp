// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphViewportController.h"

#include "AdvancedPreviewScene.h"
#include "AnimNextAnimGraphWorkspaceAssetUserData.h"
#include "Component/AnimNextComponent.h"
#include "UAF/Viewport/ViewportSceneDescription.h"
#include "Engine/PreviewMeshCollection.h"
#include "Components/SkeletalMeshComponent.h"
#include "Module/UAFSystemAssetData.h"

namespace UE::UAF::Editor
{
	void FAnimGraphViewportController::OnEnter(const FViewportContext& InViewportContext)
	{
		UUAFAnimGraph* Graph = CastChecked<UUAFAnimGraph>(InViewportContext.OutlinerObject);
		UUAFViewportSceneDescription* UAFViewportSceneDescription = CastChecked<UUAFViewportSceneDescription>(InViewportContext.SceneDescription);

		// TODO: Consider building this module programmatically
		TObjectPtr<UUAFSystem> Module = LoadObject<UUAFSystem>(GetTransientPackage(), TEXT("/UAFAnimGraph/Internal/S_SingleGraph.S_SingleGraph"));
		
		const TObjectPtr<UWorld> PreviewWorld = InViewportContext.PreviewScene->GetWorld();

		const auto AddActor = [&](USkeletalMesh* InSkeletalMesh)
			{
				AActor* PreviewActor = PreviewWorld->SpawnActor<AActor>(AActor::StaticClass());
				check(PreviewActor);

				USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(PreviewActor);
				InViewportContext.PreviewScene->AddComponent(SkeletalMeshComponent, FTransform::Identity);
				PreviewActor->SetRootComponent(SkeletalMeshComponent);
				SkeletalMeshComponent->SetEnableAnimation(false);
				SkeletalMeshComponent->SetSkeletalMesh(InSkeletalMesh);

				UUAFComponent* UAFComponent = NewObject<UUAFComponent>(PreviewActor);
				PreviewActor->AddInstanceComponent(UAFComponent);
				UAFComponent->SetAssetInternal(FUAFSystemFactoryAsset_System(Module));
				UAFComponent->RegisterComponent();
				UAFComponent->InitializeComponent();

				check(UAFComponent->SetVariable<TObjectPtr<UObject>>(FAnimNextVariableReference::FromName("Graph", Module), Graph));

				PreviewActors.Add(PreviewActor);
			};

		if (UAFViewportSceneDescription->SkeletalMesh)
		{
			AddActor(UAFViewportSceneDescription->SkeletalMesh);
		}

		if (UAFViewportSceneDescription->AdditionalMeshes)
		{
			for (int32 MeshIndex = 0; MeshIndex < UAFViewportSceneDescription->AdditionalMeshes->SkeletalMeshes.Num(); ++MeshIndex)
			{
				const TSoftObjectPtr<USkeletalMesh> SkelMeshSoftPtr = UAFViewportSceneDescription->AdditionalMeshes->SkeletalMeshes[MeshIndex].SkeletalMesh;
				
				if (!SkelMeshSoftPtr.IsNull())
				{
					USkeletalMesh* SkeletalMesh = SkelMeshSoftPtr.LoadSynchronous();
					if (SkeletalMesh)
					{
						AddActor(SkeletalMesh);
					}
				}
			}
		}
	}
	
	void FAnimGraphViewportController::OnExit(FAdvancedPreviewScene* PreviewScene)
	{
		for (AActor* PreviewActor : PreviewActors)
		{
			if (PreviewActor)
			{
				if (PreviewScene && PreviewActor->GetRootComponent())
				{
					PreviewScene->RemoveComponent(PreviewActor->GetRootComponent());
				}
			
				PreviewActor->Destroy();
				PreviewActor = nullptr;
			}
		}
		
		
		PreviewActors.Empty();
	}
}
