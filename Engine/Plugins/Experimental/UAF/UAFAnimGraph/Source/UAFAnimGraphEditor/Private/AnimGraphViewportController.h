// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceViewportController.h"

namespace UE::UAF::Editor
{
	class FAnimGraphViewportController : public Workspace::IWorkspaceViewportController
	{
		virtual void OnEnter(const FViewportContext& InViewportContext) override;
		virtual void OnExit(FAdvancedPreviewScene* PreviewScene) override;

		TArray<AActor*> PreviewActors;
	};
}