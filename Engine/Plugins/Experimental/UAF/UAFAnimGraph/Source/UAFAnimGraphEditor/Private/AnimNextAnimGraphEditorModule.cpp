// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedPreviewScene.h"
#include "AnimNextAnimGraphWorkspaceAssetUserData.h"
#include "AssetPreview/UAFGraphAssetPreview.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"
#include "AnimNextAnimGraphSettings.h"
#include "EdGraphUtilities.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditorModule.h"
#include "PropertyEditorModule.h"
#include "TraitStackEditor.h"
#include "AnimGraphViewportController.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "Features/IModularFeatures.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphItemDetails.h"
#include "Graph/AnimNextAnimationGraphMenuExtensions.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "Graph/AnimNextGraphPanelNodeFactory.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Traits/AnimNextCallFunctionSharedDataDetails.h"
#include "Traits/UAFCallFunctionInfoDetails.h"
#include "Traits/CallFunction.h"
#include "IWorkspaceEditor.h"
#include "Graph/AnimGraphEditorSchemaActions.h"
#include "Graph/AnimNextAnimationGraphSchema.h"
#include "Common/SActionMenu.h"
#include "Common/AssetPreview/UAFAssetPreviewFactorySubsystem.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "AnimNextTraitStackUnitNode.h"
#include "RigVMCore/RigVMFunction.h"
#include "EditorUtils.h"
#include "Graph/AnimNextAnimGraph.h"
#include "Graph/AnimNextGraphDetails.h"
#include "Graph/PostProcessAnimationCustomization.h"
#include "Module/AnimNextEventGraphSchema.h"
#include "RewindDebugger/AnimNextAnimGraphTraceModule.h"
#include "RewindDebugger/EvaluationProgramTrack.h"
#include "RewindDebugger/SequenceInfoTrack.h"
#include "PersonaModule.h"
#include "Component/AnimNextComponent.h"
#include "Graph/AnimNextGraphPanelPinFactory.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Factory/AnimNextFactoryParams.h"
#include "TraceServices/ModuleService.h"
#include "Factory/AnimNextFactoryParamsDetails.h"
#include "Graph/GraphNodeTemplateDetails.h"
#include "UObject/UObjectIterator.h"
#include "Templates/GraphNodeTemplateRegistry.h"
#include "UObject/TopLevelAssetPath.h"

#define LOCTEXT_NAMESPACE "FAnimNextAnimGraphEditorModule"

namespace
{
	FAnimNextAnimGraphTraceModule GAnimNextAnimGraphTraceModule;
	UE::UAF::Editor::FEvaluationProgramTrackCreator GAnimNextModulesTrackCreator;
	UE::UAF::Editor::FSequenceInfoTrackCreator GSequenceInfoTrackCreator;
}

namespace UE::UAF::Editor
{

class FAnimNextAnimGraphEditorModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		AnimGraphTopLevelAssetPath = UUAFAnimGraph::StaticClass()->GetClassPathName();

		// Register settings
		ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
		SettingsModule.RegisterSettings("Project", "Plugins", "UAFAnimGraph",
			LOCTEXT("SettingsName", "UAF Anim Graph"),
			LOCTEXT("SettingsDescription", "Configure options for UAF animation graphs."),
			GetMutableDefault<UAnimNextAnimGraphSettings>());

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(
			FAnimNextCallFunctionSharedData::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FCallFunctionSharedDataDetails>(); }));
		PropertyModule.RegisterCustomPropertyTypeLayout(
			FUAFCallFunctionInfo::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FCallFunctionInfoDetails>(); }));
		PropertyModule.RegisterCustomPropertyTypeLayout(
					FAnimNextAnimGraph::StaticStruct()->GetFName(),
					FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FAnimNextGraphDetails>(); }));
					
		PropertyModule.RegisterCustomPropertyTypeLayout(
							FAnimNextSequenceTraceInfo::StaticStruct()->GetFName(),
							FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<UE::UAF::Editor::FAnimNextSequenceTraceInfoCustomization>(); }));

		PropertyModule.RegisterCustomPropertyTypeLayout(
			FAnimNextFactoryParams::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FAnimNextFactoryParamsDetails>(); }));

		PropertyModule.RegisterCustomClassLayout(
			UUAFGraphNodeTemplate::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateLambda([](){ return MakeShared<FGraphNodeTemplateDetails>(); }));
		
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &GAnimNextAnimGraphTraceModule);
		IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GAnimNextModulesTrackCreator);
		IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GSequenceInfoTrackCreator);

		// Temp register asset previews while we still use the UUAFAssetPreviewFactorySubsystem
		FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
		{
			if (GEditor)
			{
				GEditor->GetEditorSubsystem<UUAFAssetPreviewFactorySubsystem>()->AddAssetPreviewFactory(MakeShared<FUAFGraphAssetPreviewFactory>());
			}
		});

		Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");

		// --- AnimNextAnimationGraph ---
		Workspace::FObjectDocumentArgs AnimNextAnimationGraphDocumentArgs(
			Workspace::FOnRedirectWorkspaceContext::CreateLambda([](const Workspace::FWorkspaceDocument& Document) -> UObject*
			{
				URigVMEdGraph* EdGraph = nullptr;
				
				UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(Document.GetObject());
				UUAFAnimGraph_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph);

				UAnimNextAnimationGraphEntry* AnimationGraphEntry = Cast<UAnimNextAnimationGraphEntry>(EditorData->FindEntry(FRigUnit_AnimNextGraphRoot::DefaultEntryPoint));
				// Redirect to the inner graph
				if (ensure(AnimationGraphEntry))
				{
					EdGraph = AnimationGraphEntry->GetEdGraph();
				}
				return EdGraph;
			}));

		WorkspaceEditorModule.RegisterObjectDocumentType(AnimGraphTopLevelAssetPath, AnimNextAnimationGraphDocumentArgs);

		WorkspaceEditorModule.RegisterViewportControllerFactory(UUAFAnimGraph::StaticClass(), []()
			{
				return MakeUnique<FAnimGraphViewportController>();
			});

/*		WorkspaceEditorModule.OnRegisterTabsForEditor().AddLambda([](FWorkflowAllowedTabSet& TabFactories, const TSharedRef<FTabManager>& InTabManager, TSharedPtr<UE::Workspace::IWorkspaceEditor> InEditorPtr)
		{
			TSharedRef<FTraitEditorTabSummoner> TraitEditorTabSummoner = MakeShared<FTraitEditorTabSummoner>(InEditorPtr);
			TabFactories.RegisterFactory(TraitEditorTabSummoner);
			TraitEditorTabSummoner->RegisterTabSpawner(InTabManager, nullptr);
		});
		
		WorkspaceEditorModule.OnExtendTabs().AddLambda([](FLayoutExtender& InLayoutExtender, TSharedPtr<UE::Workspace::IWorkspaceEditor> InEditorPtr)
		{
			FTabManager::FTab TraitEditorTab(FTabId(TraitEditorTabName), ETabState::ClosedTab);
			InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::TopRightDocumentArea), ELayoutExtensionPosition::After, TraitEditorTab);
		});*/
		
		const TSharedPtr<FAnimNextAnimationGraphItemDetails> AssetItemDetails = MakeShared<FAnimNextAnimationGraphItemDetails>();
		WorkspaceEditorModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextAnimationGraphOutlinerData::StaticStruct()->GetFName()), AssetItemDetails);

		IAnimNextEditorModule& AnimNextEditorModule = FModuleManager::LoadModuleChecked<IAnimNextEditorModule>("UAFEditor");
		AnimNextEditorModule.AddWorkspaceSupportedAssetClass(UUAFAnimGraph::StaticClass()->GetClassPathName());
		CollectMenuActionsDelegateHandle = AnimNextEditorModule.RegisterGraphMenuActionsProvider(IAnimNextEditorModule::FOnCollectGraphMenuActionsDelegate::CreateStatic(CollectContextMenuActions));

		TraitStackEditor = MakeShared<FTraitStackEditor>();
		IModularFeatures::Get().RegisterModularFeature(ITraitStackEditor::ModularFeatureName, TraitStackEditor.Get());

		AnimNextGraphPanelNodeFactory = MakeShared<FAnimNextGraphPanelNodeFactory>();
		FEdGraphUtilities::RegisterVisualNodeFactory(AnimNextGraphPanelNodeFactory);

		AnimNextGraphPanelPinFactory = MakeShared<FAnimNextGraphPanelPinFactory>();
		FEdGraphUtilities::RegisterVisualPinFactory(AnimNextGraphPanelPinFactory);
		
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		TArray<FPersonaModule::FOnCustomizeMeshDetails>& CustomizeMeshDetailsDelegates = PersonaModule.GetCustomizeMeshDetailsDelegates();
		CustomizeMeshDetailsDelegates.Add(FPersonaModule::FOnCustomizeMeshDetails::CreateStatic(&FPostProcessAnimationCustomization::OnCustomizeMeshDetails));

		FAnimationGraphMenuExtensions::RegisterMenus();
	}

	virtual void ShutdownModule() override
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "UAFAnimGraph");
		}
		
		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &GAnimNextAnimGraphTraceModule);
		IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GAnimNextModulesTrackCreator);

		if(FModuleManager::Get().IsModuleLoaded("WorkspaceEditor"))
		{
			Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
			WorkspaceEditorModule.UnregisterObjectDocumentType(AnimGraphTopLevelAssetPath);
			if(UObjectInitialized())
			{
				WorkspaceEditorModule.UnregisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextAnimationGraphOutlinerData::StaticStruct()->GetFName()));
				WorkspaceEditorModule.UnregisterViewportControllerFactory(UUAFAnimGraph::StaticClass());
			}
		}

		if(FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextCallFunctionSharedData");
			PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextAnimGraph");
			PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextFactoryParamsDetails");
			PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextSequenceTraceInfo");
			PropertyModule.UnregisterCustomClassLayout("UAFGraphNodeTemplate");
		}

		IModularFeatures::Get().UnregisterModularFeature(ITraitStackEditor::ModularFeatureName, TraitStackEditor.Get());
		TraitStackEditor.Reset();

		if (IAnimNextEditorModule* AnimNextEditorModule = FModuleManager::GetModulePtr<IAnimNextEditorModule>("UAFEditor"))
		{
			AnimNextEditorModule->UnregisterGraphMenuActionsProvider(CollectMenuActionsDelegateHandle);
		}

		FEdGraphUtilities::UnregisterVisualNodeFactory(AnimNextGraphPanelNodeFactory);
		FEdGraphUtilities::UnregisterVisualPinFactory(AnimNextGraphPanelPinFactory);
		
		FAnimationGraphMenuExtensions::UnregisterMenus();
	}

	static void CollectContextMenuActions(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak, FGraphContextMenuBuilder& InContextMenuBuilder, const FActionMenuContextData& ActionMenuContextData)
	{
		if (const URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(InContextMenuBuilder.CurrentGraph))
		{
			if (RigVMEdGraph->GetModel()->GetSchemaClass() == UAnimNextAnimationGraphSchema::StaticClass())
			{
				for (const TPair<FTopLevelAssetPath, UE::UAF::FGraphNodeTemplateInfo>& TemplatePair : UE::UAF::FGraphNodeTemplateRegistry::GetAllTemplates())
				{
					InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_AddTemplateNode>(TemplatePair.Value));
				}
			}
			else if (RigVMEdGraph->GetModel()->GetSchemaClass() == UAnimNextEventGraphSchema::StaticClass())
			{
				InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_NotifyEvent>());
			}
		}

		// Add trait stack using a custom RigUnit node class
		UScriptStruct* Struct = FRigUnit_AnimNextTraitStack::StaticStruct();
		const FString FunctionName = FString::Printf(TEXT("%s::%s"), *Struct->GetStructCPPName(), *FRigVMStruct::ExecuteName.ToString());
		const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(*FunctionName);
		if (ensure(Function != nullptr))
		{
			if (ActionMenuContextData.RigVMSchema != nullptr && ActionMenuContextData.RigVMSchema->SupportsUnitFunction(ActionMenuContextData.RigVMController, Function))
			{
				UE::UAF::Editor::FUtils::AddSchemaRigUnitAction(UAnimNextTraitStackUnitNode::StaticClass(), Struct, *Function, InContextMenuBuilder);
			}
		}
	}

	/** Graph context menu collect actions delegate handle */
	FDelegateHandle CollectMenuActionsDelegateHandle;

	/** Trait stack editor modular feature */
	TSharedPtr<FTraitStackEditor> TraitStackEditor;

	/** Node factory for the AnimNext graph */
	TSharedPtr<FAnimNextGraphPanelNodeFactory> AnimNextGraphPanelNodeFactory;

	/** Pin factory for the AnimNext graph */
	TSharedPtr<FAnimNextGraphPanelPinFactory> AnimNextGraphPanelPinFactory;

	FTopLevelAssetPath AnimGraphTopLevelAssetPath;
};

} // end namespace

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::UAF::Editor::FAnimNextAnimGraphEditorModule, UAFAnimGraphEditor);

