// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFGraphAssetPreview.h"

#include "Animation/AnimationAsset.h"
#include "Component/AnimNextComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Actor.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "IWorkspaceEditor.h"
#include "Module/AnimNextModule.h"
#include "Module/UAFWeakSystemReference.h"
#include "UAF/Viewport/ViewportSceneDescription.h"

#define LOCTEXT_NAMESPACE "UAFGraphAssetPreview"

namespace UE::UAF::Editor
{


//////////////////////////////////////////////////////////////////////////
// FUAFGraphAssetPreviewFactory

TSharedRef<IUAFAssetPreview> FUAFGraphAssetPreviewFactory::CreateAssetPreviewWidget(TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData) const
{
	return SNew(SUAFGraphAssetPreview, InAssetEditorToolkit, InAssetData);
}

const UStruct* FUAFGraphAssetPreviewFactory::GetPreviewType() const
{
	return UUAFAnimGraph::StaticClass();
}


//////////////////////////////////////////////////////////////////////////
// SUAFGraphAssetPreview

const UStruct* SUAFGraphAssetPreview::GetAssetPreviewType() const
{
	return UUAFAnimGraph::StaticClass();
}

bool SUAFGraphAssetPreview::OnSameTypeAssetSelected(const FAssetData& InAssetData)
{
	bool bAssetValidForPreview = false;

	UUAFAnimGraph* Graph = Cast<UUAFAnimGraph>(CachedCurrentAssetData.GetAsset());
	if (Graph
		&& UAFComponent
		&& UAFSceneDescription 
		&& UAFSceneDescription->SkeletalMesh )
	{
		const UUAFSystem* Module = UAFComponent->GetSystemReference().GetSystem();
		check(UAFComponent->SetVariable<TObjectPtr<UObject>>(FAnimNextVariableReference::FromName("Graph", Module), Graph));
		bAssetValidForPreview = true;
	}

	ViewportWidget->SetVisibility(bAssetValidForPreview ? EVisibility::Visible : EVisibility::Hidden);
	return bAssetValidForPreview;
}

void SUAFGraphAssetPreview::OnCustomizePreviewScene(FAdvancedPreviewScene& InPreviewScene, FEditorViewportClient& InEditorViewportClient)
{
	// @TODO: Note for review. Most impl is from AnimGraphViewportController.cpp

	// TODO: Consider building this module programmatically
	TObjectPtr<UUAFSystem> Module = LoadObject<UUAFSystem>(GetTransientPackage(), TEXT("/UAFAnimGraph/Internal/S_SingleGraph.S_SingleGraph"));

	auto AddActor = [&](USkeletalMesh* InSkeletalMesh)
	{
		AActor* PreviewActor = PreviewScene->GetWorld()->SpawnActor<AActor>(AActor::StaticClass());
		check(PreviewActor);

		PreviewComponent = NewObject<USkeletalMeshComponent>(PreviewActor);
		// PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones; 
		PreviewScene->AddComponent(PreviewComponent, FTransform::Identity);
		PreviewActor->SetRootComponent(PreviewComponent);
		PreviewComponent->SetEnableAnimation(false);
		PreviewComponent->SetSkeletalMesh(InSkeletalMesh);

		UAFComponent = NewObject<UUAFComponent>(PreviewActor);
		PreviewActor->AddInstanceComponent(UAFComponent);
		UAFComponent->SetAssetFromObject(Module);
		UAFComponent->RegisterComponent();
		UAFComponent->InitializeComponent();
	};

	if (TSharedPtr<FAssetEditorToolkit> OwningEditor = AssetEditorToolkitPtr.Pin())
	{
		using namespace UE::Workspace;

		if (OwningEditor->GetToolkitFName() == IWorkspaceEditor::GetWorkspaceEditorToolkitName())
		{
			TSharedPtr<IWorkspaceEditor> OwningWorkspaceEditor = StaticCastSharedPtr<IWorkspaceEditor>(OwningEditor);
			if (UUAFViewportSceneDescription* WorkspaceSceneDescription = Cast<UUAFViewportSceneDescription>(OwningWorkspaceEditor->GetSceneDescription()))
			{
				UAFSceneDescription = WorkspaceSceneDescription;
			}
		}
	}

	if (UAFSceneDescription && UAFSceneDescription->SkeletalMesh)
	{
		AddActor(UAFSceneDescription->SkeletalMesh);
	}
}

//////////////////////////////////////////////////////////////////////////

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

