// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/AssetPreview/IUAFAssetViewportPreview.h"

class FSceneViewport;
class FUAFAssetPreviewViewportClient;
class SAssetEditorViewport;
class UUAFComponent;
class USkeletalMeshComponent;
class UUAFViewportSceneDescription;

namespace UE::UAF::Editor
{


/**
 * Experimental. May be removed / relocated whenever we switch to the future generic content browser IAssetPreviewCustomization API.
 *
 * Factory for UAF Graph Asset Preview widgets.
 */
struct FUAFGraphAssetPreviewFactory : public FUAFAssetPreviewFactory
{
public:

	//~ Begin FUAFAssetPreviewFactory Interface
	virtual TSharedRef<IUAFAssetPreview> CreateAssetPreviewWidget(TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData) const override;
	virtual const UStruct* GetPreviewType() const override;
	//~ End FUAFAssetPreviewFactory Interface
};

/** 
 * Experimental. May be removed / relocated whenever we switch to the future generic content browser IAssetPreviewCustomization API. 
 * 
 * Preview widget for UAF Graph Assets. Includes an interactable viewport.
 * Currently these previews only work in a UAF workspace that has set it's preview skeletal mesh.
 */
class SUAFGraphAssetPreview : public IUAFAssetViewportPreview
{
public:

	//~ Begin IUAFAssetPreview Interface
	virtual const UStruct* GetAssetPreviewType() const override;
	//~ End IUAFAssetPreview Interface

protected:

	//~ Begin IUAFAssetPreview Interface
	virtual bool OnSameTypeAssetSelected(const FAssetData& InAssetData) override;
	//~ End IUAFAssetPreview Interface

	//~ Begin IUAFAssetViewportPreview Interface
	virtual void OnCustomizePreviewScene(FAdvancedPreviewScene& InPreviewScene, FEditorViewportClient& InEditorViewportClient) override;
	//~ End IUAFAssetViewportPreview Interface

protected:

	/** Skeletal component to preview the animation asset on. Lifetime maintained by PreviewScene in IUAFAssetViewportPreview */
	USkeletalMeshComponent* PreviewComponent = nullptr;

	/** UAF Graph Preview Driver. Lifetime maintained by PreviewScene in IUAFAssetViewportPreview */
	UUAFComponent* UAFComponent = nullptr;

	/** 
	 * Object used to describe UAF previews. Lifetime maintained by workspace editor.
	 * 
	 * Currently we only acquire this by checking if the current editor is a workspace editor & acquiring it's viewport scene description.
	 */
	UUAFViewportSceneDescription* UAFSceneDescription = nullptr;
};


} // namespace UE::UAF::Editor