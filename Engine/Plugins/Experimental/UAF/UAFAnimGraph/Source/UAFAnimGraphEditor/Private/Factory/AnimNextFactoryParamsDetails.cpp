// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextFactoryParamsDetails.h"

#include "UAFStyle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "InstancedStructDetails.h"
#include "IPropertyAccessEditor.h"
#include "IStructureDataProvider.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Algo/Compare.h"
#include "Factory/AnimGraphFactory.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Modules/ModuleManager.h"
#include "Styling/StyleColors.h"
#include "TraitCore/TraitRegistry.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Graph/AnimNextAnimationGraph.h"

#define LOCTEXT_NAMESPACE "AnimNextFactoryParamsDetails"

namespace UE::UAF::Editor
{

void FAnimNextFactoryParamsDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FAnimNextFactoryParamsDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> BuilderHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextFactoryParams, Builder));
	BuilderHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> StacksHandle = BuilderHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextSimpleAnimGraphBuilder, Stacks));
	TSharedPtr<IPropertyHandle> StackHandle = StacksHandle->GetChildHandle(0);
	if (StackHandle.IsValid())
	{
		TraitDescsHandle = StackHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextSimpleAnimGraphBuilderTraitStackDesc, TraitDescs));
	}
	
	bool bIsInvalidOrAnimGraph = false; 

	// Find property for the factory source & common default params set
	TOptional<FAnimNextFactoryParams> CommonDefaults;

	TSet<const UScriptStruct*> AddedStructs;

	if (TraitDescsHandle.IsValid())
	{
		// Find common 'current' set
		TSet<const UScriptStruct*> CommonStructs;
		TraitDescsHandle->EnumerateRawData([&CommonStructs](void* RawData, const int32 DataIndex, const int32 NumDatas)
		{
			TArray<FAnimNextSimpleAnimGraphBuilderTraitDesc>* TraitDescs = static_cast<TArray<FAnimNextSimpleAnimGraphBuilderTraitDesc>*>(RawData);
			if (CommonStructs.IsEmpty())
			{
				for (const FAnimNextSimpleAnimGraphBuilderTraitDesc& TraitDesc : *TraitDescs)
				{
					CommonStructs.Add(TraitDesc.TraitData.GetScriptStruct());
				}
			}
			else
			{
				for (auto It = CommonStructs.CreateIterator(); It; ++It)
				{
					const UScriptStruct* CommonStruct = *It;
					if (!TraitDescs->ContainsByPredicate([CommonStruct](const FAnimNextSimpleAnimGraphBuilderTraitDesc& InTraitDesc){ return CommonStruct == InTraitDesc.TraitData.GetScriptStruct(); }))
					{
						It.RemoveCurrent();
					}
				}
			}
			return true;
		});

		// No common structs, cannot edit (all struct types for edited objects are mutually exclusive)
		if (CommonStructs.IsEmpty())
		{
			return;
		}

		// Add entries for instanced structs that are common to all objects and have a common type at this index
		uint32 NumChildren = 0;
		if (TraitDescsHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
		{
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedRef<IPropertyHandle> TraitDescHandle = TraitDescsHandle->GetChildHandle(ChildIndex).ToSharedRef();
				TOptional<const UScriptStruct*> CommonType;
				TraitDescHandle->EnumerateRawData([&CommonType](void* RawData, const int32 DataIndex, const int32 NumDatas)
				{
					FAnimNextSimpleAnimGraphBuilderTraitDesc* TraitDesc = static_cast<FAnimNextSimpleAnimGraphBuilderTraitDesc*>(RawData);
					if (!CommonType.IsSet())
					{
						CommonType = TraitDesc->TraitData.GetScriptStruct();
					}
					else if (CommonType.GetValue() != TraitDesc->TraitData.GetScriptStruct())
					{
						CommonType.Reset();
						return false;
					}
					return true;
				});

				if (CommonType.IsSet() && CommonStructs.Contains(CommonType.GetValue()))
				{
					AddedStructs.Add(CommonType.GetValue());

					const FName UniqueName(TraitDescHandle->GetPropertyPath());
					TSharedPtr<IPropertyHandle> InstancedStructHandle = TraitDescHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextSimpleAnimGraphBuilderTraitDesc, TraitData));
					check(InstancedStructHandle.IsValid());
					TSharedPtr<FInstancedStructProvider> StructProvider = MakeShared<FInstancedStructProvider>(InstancedStructHandle);
					
					if (StructProvider->GetBaseStructure() == nullptr)
					{
						continue;
					}
					
					IDetailPropertyRow* Row = InChildBuilder.AddChildStructure(InstancedStructHandle.ToSharedRef(), StructProvider, UniqueName);
					FDetailWidgetRow& CustomWidget = Row->CustomWidget(true);

					CustomWidget
					.NameContent()
					[
						SNew(SBorder)
						.Padding(FMargin(6.0f, 2.0f))
						.BorderImage(FUAFStyle::Get().GetBrush("AnimNext.TraitDetails.Background"))
						[
							SNew(STextBlock)
							.TextStyle(FUAFStyle::Get(), "AnimNext.TraitDetails.Label")
							.ColorAndOpacity(FStyleColors::Foreground)
							.Text(CommonType.GetValue()->GetDisplayNameText())
						]
					];

					const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
					const FTrait* Trait = TraitRegistry.Find(CommonType.GetValue());
					if (Trait)
					{
						// If this is an additive trait, allow it to be removed
						if (Trait->GetTraitMode() == ETraitMode::Additive)
						{
							CustomWidget
							.ValueContent()
							[
								SNew(SCheckBox)
								.ToolTipText(LOCTEXT("EnableOptionalTrait", "Enable/disable this optional trait"))
								.IsChecked(ECheckBoxState::Checked)
								.OnCheckStateChanged_Lambda([ChildIndex, WeakTraitDescsHandle = TWeakPtr<IPropertyHandle>(TraitDescsHandle), WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)](ECheckBoxState InState)
								{
									TSharedPtr<IPropertyHandle> PinnedTraitDescsHandle = WeakTraitDescsHandle.Pin();
									TSharedPtr<IPropertyHandle> PinnedPropertyHandle = WeakPropertyHandle.Pin();
									if (PinnedTraitDescsHandle.IsValid() && PinnedPropertyHandle.IsValid())
									{
										PinnedTraitDescsHandle->AsArray()->DeleteItem(ChildIndex);
										PinnedPropertyHandle->RequestRebuildChildren();
									}
								})
							];
						}
					}
				}
			}
		}
	}

	if (CommonDefaults.IsSet() && CommonDefaults.GetValue().Builder.GetNumStacks() > 0)
	{
		// Now add any entries for defaults we didnt find above to allow them to be added
		for (const FAnimNextSimpleAnimGraphBuilderTraitDesc& DefaultDesc : CommonDefaults.GetValue().Builder.Stacks[0].TraitDescs)
		{
			if (!AddedStructs.Contains(DefaultDesc.TraitData.GetScriptStruct()))
			{
				InChildBuilder.AddCustomRow(DefaultDesc.TraitData.GetScriptStruct()->GetDisplayNameText())
				.NameContent()
				[
					SNew(SBorder)
					.Padding(FMargin(6.0f, 2.0f))
					.BorderImage(FUAFStyle::Get().GetBrush("AnimNext.TraitDetails.Background"))
					[
						SNew(STextBlock)
						.TextStyle(FUAFStyle::Get(), "AnimNext.TraitDetails.Label")
						.ColorAndOpacity(FStyleColors::Foreground)
						.Text(DefaultDesc.TraitData.GetScriptStruct()->GetDisplayNameText())
					]
				]
				.ValueContent()
				[
					SNew(SCheckBox)
					.ToolTipText(LOCTEXT("EnableOptionalTrait", "Enable/disable this optional trait"))
					.IsChecked(ECheckBoxState::Unchecked)
					.OnCheckStateChanged_Lambda([DefaultDesc, WeakTraitDescsHandle = TWeakPtr<IPropertyHandle>(TraitDescsHandle), WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)](ECheckBoxState InState)
					{
						TSharedPtr<IPropertyHandle> PinnedTraitDescsHandle = WeakTraitDescsHandle.Pin();
						TSharedPtr<IPropertyHandle> PinnedPropertyHandle = WeakPropertyHandle.Pin();
						if (PinnedTraitDescsHandle.IsValid() && PinnedPropertyHandle.IsValid())
						{
							FScopedTransaction Transaction(LOCTEXT("Enable Trait","Enable Trait"));
							
							TSharedPtr<IPropertyHandleArray> ArrayHandle = PinnedTraitDescsHandle->AsArray();
							FPropertyHandleItemAddResult Result = ArrayHandle->AddItem();
							TSharedPtr<IPropertyHandle> NewEntryHandle = PinnedTraitDescsHandle->GetChildHandle(Result.GetIndex());
							NewEntryHandle->EnumerateRawData([&DefaultDesc](void* RawData, const int32 DataIndex, const int32 NumDatas)
							{
								FAnimNextSimpleAnimGraphBuilderTraitDesc* TraitDesc = static_cast<FAnimNextSimpleAnimGraphBuilderTraitDesc*>(RawData);
								*TraitDesc = DefaultDesc;
								return true;
							});
							PinnedPropertyHandle->RequestRebuildChildren();
						}
					})
				];
			}
		}
	}

	if (!bIsInvalidOrAnimGraph)
	{
		// Add an 'add' widget for any custom behaviors users want to opt into
		InChildBuilder.AddCustomRow(LOCTEXT("AddTraitWidget", "Add Trait"))
		.NameContent()
		[
			SNew(SComboButton)
			.ToolTipText(LOCTEXT("AddTraitTooltip", "Add a trait to modify how this asset is played"))
			.HasDownArrow(false)
			.ComboButtonStyle(FUAFStyle::Get(), "AnimNext.TraitDetails.AddButton")
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 3.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Plus"))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 2.0f)
				[
					SNew(STextBlock)
					.TextStyle(FUAFStyle::Get(), "AnimNext.TraitDetails.Label")
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(LOCTEXT("AddTraitButton", "Add"))
				]
			]
			.OnGetMenuContent_Lambda([WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)]()
			{
				FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

				class FStructFilter : public IStructViewerFilter
				{
				public:
					virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
					{
						if (InStruct->HasMetaData(TEXT("Hidden")))
						{
							return false;
						}

						if (!InStruct->IsChildOf(FAnimNextTraitSharedData::StaticStruct()))
						{
							return false;
						}
						
						const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
						const FTrait* Trait = TraitRegistry.Find(InStruct);
						return Trait && Trait->GetTraitMode() == ETraitMode::Additive;
					}

					virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs)
					{
						return false;
					};
				};

				FStructViewerInitializationOptions Options;
				Options.StructFilter = MakeShared<FStructFilter>();
				Options.Mode = EStructViewerMode::StructPicker;
				Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
				Options.bShowNoneOption = false;

				return
					SNew(SBox)
					.WidthOverride(400.0f)
					.HeightOverride(400.0f)
					[
						StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([WeakPropertyHandle](const UScriptStruct* InStruct)
						{
							TSharedPtr<IPropertyHandle> PinnedPropertyHandle = WeakPropertyHandle.Pin();
							if (PinnedPropertyHandle.IsValid())
							{
								FScopedTransaction Transaction(LOCTEXT("Add Trait", "Add Trait"));
								
								TSharedPtr<IPropertyHandle> BuilderHandle = PinnedPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextFactoryParams, Builder));
								TSharedPtr<IPropertyHandle> StacksHandle = BuilderHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextSimpleAnimGraphBuilder, Stacks));
								TSharedPtr<IPropertyHandle> StackHandle = StacksHandle->GetChildHandle(0);
								if (!StackHandle.IsValid())
								{
									StacksHandle->AsArray()->AddItem();
									StackHandle = StacksHandle->GetChildHandle(0);
								}

								TSharedPtr<IPropertyHandle> TraitDescsHandle = StackHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextSimpleAnimGraphBuilderTraitStackDesc, TraitDescs));
								TSharedPtr<IPropertyHandleArray> ArrayHandle = TraitDescsHandle->AsArray();
								FPropertyHandleItemAddResult Result = ArrayHandle->AddItem();
								TSharedPtr<IPropertyHandle>  NewEntryHandle = TraitDescsHandle->GetChildHandle(Result.GetIndex());
								NewEntryHandle->EnumerateRawData([InStruct](void* RawData, const int32 DataIndex, const int32 NumDatas)
								{
									FAnimNextSimpleAnimGraphBuilderTraitDesc* TraitDesc = static_cast<FAnimNextSimpleAnimGraphBuilderTraitDesc*>(RawData);
									TraitDesc->TraitData.InitializeAsScriptStruct(InStruct);
									return true;
								});
								PinnedPropertyHandle->RequestRebuildChildren();
							}
						}))
					];
			})
		];
	}
}

}

#undef LOCTEXT_NAMESPACE