// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphEntryAssetDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraphEntryAssetDefinitions)

#define LOCTEXT_NAMESPACE "AnimNextAnimGraphEntryAssetDefinitions"

FText UAssetDefinition_AnimNextAnimationGraphEntry::GetObjectDisplayNameText(UObject* Object) const
{
	UUAFRigVMAssetEntry* Parameter = CastChecked<UUAFRigVMAssetEntry>(Object);
	return Parameter->GetDisplayName();
}

#undef LOCTEXT_NAMESPACE
