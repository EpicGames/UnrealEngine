// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "AnimNextAnimGraphEntryAssetDefinitions.generated.h"

#define LOCTEXT_NAMESPACE "AnimNextAnimGraphEntryAssetDefinitions"

UCLASS()
class UAssetDefinition_AnimNextAnimationGraphEntry : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override { return LOCTEXT("AnimNextAnimationGraphEntry", "Animation Graph"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimNextAnimationGraphEntry::StaticClass(); }
	virtual FText GetObjectDisplayNameText(UObject* Object) const override;
};

#undef LOCTEXT_NAMESPACE