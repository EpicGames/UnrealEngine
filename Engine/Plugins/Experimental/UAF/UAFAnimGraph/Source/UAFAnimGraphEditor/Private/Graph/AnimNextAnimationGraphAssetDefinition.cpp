// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimationGraphAssetDefinition.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "ContentBrowserMenuContexts.h"
#include "FileHelpers.h"
#include "IWorkspaceEditorModule.h"
#include "ToolMenus.h"
#include "UncookedOnlyUtils.h"
#include "Workspace/AnimNextWorkspaceFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimationGraphAssetDefinition)

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

EAssetCommandResult UAssetDefinition_AnimNextAnimationGraph::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::UAF::Editor;
	using namespace UE::Workspace;

	for (UUAFAnimGraph* Asset : OpenArgs.LoadObjects<UUAFAnimGraph>())
	{
		IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceEditorModule.OpenWorkspaceForObject(Asset, EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass());
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
