// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextAnimationGraphFactory.h"

#include "UAFCompilationScope.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "UncookedOnlyUtils.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimationGraphFactory)

UUAFAnimGraphFactory::UUAFAnimGraphFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUAFAnimGraph::StaticClass();
}

bool UUAFAnimGraphFactory::ConfigureProperties()
{
	return true;
}

UObject* UUAFAnimGraphFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if(InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	UUAFAnimGraph* NewAnimationGraph = NewObject<UUAFAnimGraph>(InParent, Class, Name, FlagsToUse);

	// Create internal editor data
	UUAFAnimGraph_EditorData* EditorData = NewObject<UUAFAnimGraph_EditorData>(NewAnimationGraph, TEXT("EditorData"), RF_Transactional);
	NewAnimationGraph->EditorData = EditorData;
	EditorData->Initialize(/*bRecompileVM*/false);

	// Add a single internal graph entry
	UAnimNextAnimationGraphEntry* NewGraph = EditorData->AddAnimationGraph(FRigUnit_AnimNextGraphRoot::DefaultEntryPoint, false, false);

	// Hide in the outliner - the asset itself represents the graph to users
	NewGraph->SetHiddenInOutliner(true);

	// Compile the initial skeleton
	UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(NewAnimationGraph);

	check(!EditorData->bErrorsDuringCompilation);

	return NewAnimationGraph;
}
