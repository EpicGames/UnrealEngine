// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimationGraphItemDetails.h"

#include "AnimNextAnimGraphWorkspaceAssetUserData.h"
#include "ClassIconFinder.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "WorkspaceItemMenuContext.h"
#include "IWorkspaceEditor.h"
#include "ScopedTransaction.h"
#include "RigVMModel/RigVMClient.h"
#include "ToolMenus.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"

#define LOCTEXT_NAMESPACE "FAnimNextAnimationGraphItemDetails"

namespace UE::UAF::Editor
{
const FSlateBrush* FAnimNextAnimationGraphItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
{
	return FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_24x"));
}

}

#undef LOCTEXT_NAMESPACE // "FAnimNextGraphItemDetails"