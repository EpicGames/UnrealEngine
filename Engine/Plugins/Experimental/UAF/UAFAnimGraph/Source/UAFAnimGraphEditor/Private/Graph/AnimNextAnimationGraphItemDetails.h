// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/AnimNextAssetItemDetails.h"

namespace UE::UAF::Editor
{

class FAnimNextAnimationGraphItemDetails : public FAnimNextAssetItemDetails
{
public:
	FAnimNextAnimationGraphItemDetails() = default;
	
	UAFANIMGRAPHEDITOR_API virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override;

};

}
