// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimationGraphMenuExtensions.h"

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextEdGraphNode.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ToolMenus.h"
#include "UncookedOnlyUtils.h"
#include "EdGraph/EdGraphNode.h"
#include "Framework/Application/SlateApplication.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Graph/RigUnit_AnimNextRunAnimationGraph_v1.h"
#include "Graph/RigUnit_AnimNextRunAnimationGraph_v2.h"
#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "TraitCore/TraitRegistry.h"
#include "AnimNextController.h"
#include "AnimNextEdGraph.h"
#include "AnimNextTraitStackUnitNode.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "BlueprintCompilationManager.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Common/UAFVariableDragDropContext.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "RigVMModel/RigVMClient.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/GraphNodeTemplateRegistry.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Variables/AnimNextSharedVariables.h"

#define LOCTEXT_NAMESPACE "FAnimationGraphMenuExtensions"

namespace UE::UAF::Editor::Private
{
	static const FLazyName VariablesTraitBaseName = TEXT("Variables");
}

namespace UE::UAF::Editor
{

void FAnimationGraphMenuExtensions::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(TEXT("FAnimNextAnimationGraphItemDetails"));
	UToolMenu* NodeMenu = UToolMenus::Get()->ExtendMenu("GraphEditor.GraphNodeContextMenu.AnimNextEdGraphNode");
	if (NodeMenu == nullptr)
	{
		return;
	}

	NodeMenu->AddDynamicSection(TEXT("AnimNextEdGraphNode"), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		UGraphNodeContextMenuContext* Context = InMenu->FindContext<UGraphNodeContextMenuContext>();
		if(Context == nullptr)
		{
			return;
		}

		const UAnimNextEdGraphNode* AnimNextEdGraphNode = Cast<UAnimNextEdGraphNode>(Context->Node);
		if(AnimNextEdGraphNode == nullptr)
		{
			return;
		}

		URigVMNode* ModelNode = AnimNextEdGraphNode->GetModelNode();
		if(ModelNode == nullptr)
		{
			return;
		}

		const UAnimNextEdGraph* AnimNextEdGraph = Cast<UAnimNextEdGraph>(Context->Graph);
		if(AnimNextEdGraph == nullptr)
		{
			return;
		}

		URigVMGraph* Model = AnimNextEdGraph->GetModel();
		if(Model == nullptr)
		{
			return;
		}

		auto IsRunGraphNode = [](URigVMNode* InModelNode)
		{
			if (const URigVMUnitNode* VMNode = Cast<URigVMUnitNode>(InModelNode))
			{
				const UScriptStruct* ScriptStruct = VMNode->GetScriptStruct();
				return ScriptStruct == FRigUnit_AnimNextRunAnimationGraph_v1::StaticStruct() || ScriptStruct == FRigUnit_AnimNextRunAnimationGraph_v2::StaticStruct();
			}

			return false;
		};
		
		if (UncookedOnly::FAnimGraphUtils::IsTraitStackNode(ModelNode))
		{
			// Cast test to deal with legacy content
			if (UAnimNextTraitStackUnitNode* TraitStackNode = Cast<UAnimNextTraitStackUnitNode>(ModelNode))
			{
				FToolMenuSection& Section = InMenu->AddSection("AnimNextTemplateNodeActions", LOCTEXT("AnimNextTemplateNodeActionsMenuHeader", "Template"));

				auto CreateNewTemplate = [TraitStackNode, Model]() -> UClass*
				{
					UPackage* Outer = Model->GetPackage();
					if(Outer == nullptr)
					{
						return nullptr;
					}

					FSaveAssetDialogConfig SaveAssetDialogConfig;
					SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
					SaveAssetDialogConfig.DefaultPath = FPaths::GetPath(Outer->GetPathName());
					SaveAssetDialogConfig.DefaultAssetName = TEXT("NewNodeTemplate");
					SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
					
					const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
					if (SaveObjectPath.IsEmpty())
					{
						return nullptr;
					}

					const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
					const FString SavePackagePath = FPaths::GetPath(SavePackageName);
					const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);

					UPackage* Package = CreatePackage(*SavePackageName);
					ensure(Package);
					if (Package == nullptr)
					{
						return nullptr;
					}

					UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprint(UUAFGraphNodeTemplate::StaticClass(), Package, *SaveAssetName, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), NAME_None);
					if (Blueprint == nullptr)
					{
						return nullptr;
					}

					UClass* GeneratedClass = Blueprint->GeneratedClass;
					if (GeneratedClass == nullptr)
					{
						return nullptr;
					}
					
					UUAFGraphNodeTemplate* CDO = GeneratedClass->GetDefaultObject<UUAFGraphNodeTemplate>();
					if (CDO == nullptr)
					{
						return nullptr;
					}

					CDO->InitializeTemplateFromNode(TraitStackNode);

					// Recompile to take on the CDO modifications
					const EBlueprintCompileOptions CompileOptions =
						EBlueprintCompileOptions::SkipGarbageCollection |
						EBlueprintCompileOptions::SkipDefaultObjectValidation |
						EBlueprintCompileOptions::SkipFiBSearchMetaUpdate;

					FBlueprintCompilationManager::CompileSynchronously(
						FBPCompileRequest(Blueprint, CompileOptions, nullptr)
					);

					// Open asset
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(Blueprint);
					
					return GeneratedClass;
				};

				Section.AddMenuEntry(
					TEXT("CreateTemplate"),
					LOCTEXT("CreateTemplate", "Create Template"),
					LOCTEXT("CreateTemplateTooltip", "Create a new template Blueprint from this node"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda(
						[CreateNewTemplate]()
						{
							CreateNewTemplate();
						})));

				Section.AddSubMenu(
					TEXT("AssignTemplate"),
					LOCTEXT("AssignTemplate", "Assign Template"),
					LOCTEXT("AssignTemplateTooltip", "Assign a new template to this node and refresh the node according to the template"),
					FNewToolMenuDelegate::CreateLambda([AnimNextEdGraphNode, TraitStackNode](UToolMenu* InSubMenu)
					{
						class FClassFilter : public IClassViewerFilter
						{
						public:
							virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
							{
								return InClass->IsChildOf(UUAFGraphNodeTemplate::StaticClass());
							}

							virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
							{
								return InUnloadedClassData->IsChildOf(UUAFGraphNodeTemplate::StaticClass());
							}
						};
						
						FClassViewerInitializationOptions Options;
						Options.bShowUnloadedBlueprints = true;
						Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
						Options.ClassFilters.Add(MakeShared<FClassFilter>());
						Options.bShowNoneOption = false;

						FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

						TSharedRef<SWidget> ClassViewer = ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateLambda([AnimNextEdGraphNode, TraitStackNode](UClass* InClass)
						{
							FSlateApplication::Get().DismissAllMenus();

							UAnimNextController* VMController = CastChecked<UAnimNextController>(AnimNextEdGraphNode->GetController());

							FScopedTransaction Transaction(LOCTEXT("AssignTemplate", "Assign Template"));
							TraitStackNode->Template = InClass;
							TraitStackNode->Template->GetDefaultObject<UUAFGraphNodeTemplate>()->RefreshNodeFromTemplate(VMController, TraitStackNode);
						}));

						FToolMenuEntry Entry = FToolMenuEntry::InitWidget(
							TEXT("GraphTemplateClassPicker"),
							SNew(SBox)
							.WidthOverride(300.0f)
							.HeightOverride(400.0f)
							[
								ClassViewer
							],
							FText::GetEmpty(),
							true,
							false,
							false,
							LOCTEXT("GraphTemplateClassPickerTooltip", "Choose a template class to assign")
						);

						InSubMenu->AddMenuEntry(NAME_None, Entry);
					}));
				
				Section.AddMenuEntry(
					TEXT("CreateAndAssignTemplate"),
					LOCTEXT("CreateAndAssignTemplate", "Create and Assign Template"),
					LOCTEXT("CreateAndAssignTemplateTooltip", "Create a new template Blueprint from this node and assign it to this node"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda(
						[TraitStackNode, CreateNewTemplate]()
						{
							UClass* Class = CreateNewTemplate();
							if (Class == nullptr)
							{
								return;
							}
							
							FScopedTransaction Transaction(LOCTEXT("AssignTemplate", "Assign Template"));
							TraitStackNode->Template = Class;
						})));

				Section.AddMenuEntry(
					TEXT("EditTemplate"),
					LOCTEXT("EditTemplate", "Edit Template"),
					LOCTEXT("EditTemplateTooltip", "Edit the template used to create this node"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
						[TraitStackNode]()
						{
							if (TraitStackNode->Template == nullptr)
							{
								return;
							}
							
							UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(TraitStackNode->Template);
							if (BPClass == nullptr)
							{
								return;
							}

							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(BPClass->ClassGeneratedBy);
						}),
						FCanExecuteAction::CreateLambda([TraitStackNode]()
						{
							if (TraitStackNode->Template == nullptr)
							{
								return false;
							}

							UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(TraitStackNode->Template);
							if (BPClass == nullptr)
							{
								return false;
							}

							return true;
						})));

				Section.AddMenuEntry(
					TEXT("RefreshFromTemplate"),
					LOCTEXT("RefreshFromTemplate", "Refresh From Template"),
					LOCTEXT("RefreshFromTemplateTooltip", "Refresh this node, applying any customizations from its template"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda(
						[TraitStackNode, Model, AnimNextEdGraphNode]()
						{
							if (TraitStackNode->Template == nullptr)
							{
								return;
							}

							UAnimNextController* VMController = CastChecked<UAnimNextController>(AnimNextEdGraphNode->GetController());

							FScopedTransaction Transaction(LOCTEXT("RefreshFromTemplate", "Refresh From Template"));
							TraitStackNode->Template->GetDefaultObject<UUAFGraphNodeTemplate>()->RefreshNodeFromTemplate(VMController, TraitStackNode);
						}),
						FCanExecuteAction::CreateLambda([TraitStackNode]()
						{
							if (TraitStackNode->Template == nullptr)
							{
								return false;
							}

							return true;
						})));
			}
		}
		else if(IsRunGraphNode(ModelNode))
		{
			FToolMenuSection& Section = InMenu->AddSection("AnimNextRunAnimGraphNodeActions", LOCTEXT("AnimNextAnimGraphNodeActionsMenuHeader", "Animation Graph"));
	
			URigVMController* VMController = AnimNextEdGraphNode->GetController();
			URigVMPin* VMPin = Context->Pin != nullptr ? AnimNextEdGraphNode->FindModelPinFromGraphPin(Context->Pin) : nullptr;
	
			if(VMPin != nullptr && ModelNode->FindTrait(VMPin))
			{
				Section.AddMenuEntry(
					TEXT("RemoveExposedVariables"),
					LOCTEXT("RemoveExposedVariablesMenu", "Remove Exposed Variables"),
					LOCTEXT("RemoveExposeVariablesMenuTooltip", "Remove the exposed variable trait from this node"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([VMController, ModelNode, Name = VMPin->GetFName()]()
						{
							VMController->RemoveTrait(ModelNode->GetFName(), Name, true, true);
						})
					));
			}
			else
			{
				auto BuildExposeVariablesContextMenu = [AnimNextEdGraphNode, ModelNode](UToolMenu* InSubMenu)
				{
					URigVMController* VMController = AnimNextEdGraphNode->GetController();

					FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

					FAssetPickerConfig AssetPickerConfig;
					AssetPickerConfig.Filter.ClassPaths.Add(UUAFSharedVariables::StaticClass()->GetClassPathName());
					AssetPickerConfig.Filter.bRecursiveClasses = true;
					AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
					AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoAssetsWithPublicVariablesMessage", "No animation graphs with public variables found");
					AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([VMController, ModelNode](const FAssetData& InAssetData)
					{
						FSlateApplication::Get().DismissAllMenus();

						FString DefaultValue;
						FRigVMTrait_AnimNextPublicVariables DefaultTrait;
						FRigVMTrait_AnimNextPublicVariables NewTrait;
						UUAFSharedVariables* Asset = CastChecked<UUAFSharedVariables>(InAssetData.GetAsset());
						NewTrait.InternalAsset = Asset;
						const FInstancedPropertyBag& VariableDefaults = Asset->GetVariableDefaults();
						if (VariableDefaults.GetPropertyBagStruct())
						{
							TConstArrayView<FPropertyBagPropertyDesc> Descs = VariableDefaults.GetPropertyBagStruct()->GetPropertyDescs();
							NewTrait.InternalVariableNames.Reserve(Descs.Num());
							for(const FPropertyBagPropertyDesc& Desc : Descs)
							{
								if (Desc.CachedProperty->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic))
								{
									NewTrait.InternalVariableNames.Add(Desc.Name);
								}
							}
							FRigVMTrait_AnimNextPublicVariables::StaticStruct()->ExportText(DefaultValue, &NewTrait, &DefaultTrait, nullptr, PPF_SerializedAsImportText, nullptr);

							const FName ValidTraitName = URigVMSchema::GetUniqueName(Private::VariablesTraitBaseName, [ModelNode](const FName& InName)
							{
								return ModelNode->FindPin(InName.ToString()) == nullptr;
							}, false, false);

							VMController->AddTrait(ModelNode->GetFName(), *FRigVMTrait_AnimNextPublicVariables::StaticStruct()->GetPathName(), ValidTraitName, DefaultValue, INDEX_NONE, true, true);
						}
					});

					AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([](const FAssetData& InAssetData)
					{
						// Filter to only show assets with public variables
						FAnimNextAssetRegistryExports Exports;
						if(UncookedOnly::FUtils::GetExportedVariablesForAsset(InAssetData, Exports))
						{
							for(const FAnimNextExport& Export : Exports.Exports)
							{
								if (const FAnimNextVariableDeclarationData* VariableDeclaration = Export.Data.GetPtr<FAnimNextVariableDeclarationData>())
								{
									if (EnumHasAllFlags(static_cast<EAnimNextExportedVariableFlags>(VariableDeclaration->Flags),EAnimNextExportedVariableFlags::Public | EAnimNextExportedVariableFlags::Declared))
									{
										return false;
									}
								}
							}
						}
						return true;
					});

					FToolMenuEntry Entry = FToolMenuEntry::InitWidget(
						TEXT("AnimationGraphPicker"),
						SNew(SBox)
						.WidthOverride(300.0f)
						.HeightOverride(400.0f)
						[
							ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
						],
						FText::GetEmpty(),
						true,
						false,
						false,
						LOCTEXT("AnimationGraphPickerTooltip", "Choose an animation graph with public variables to expose")
					);

					InSubMenu->AddMenuEntry(NAME_None, Entry);
				};
				
				Section.AddSubMenu(
					TEXT("ExposeVariables"),
					LOCTEXT("ExposeVariablesMenu", "Expose Variables"),
					LOCTEXT("ExposeVariablesMenuTooltip", "Expose the variables of a selected animation graph as pins on this node"),
					FNewToolMenuDelegate::CreateLambda(BuildExposeVariablesContextMenu));
			}
		}
	}));

	if (UToolMenu* DragDropMenu = UToolMenus::Get()->ExtendMenu("UAFVariableDragDropMenu"))
	{
		DragDropMenu->AddDynamicSection("AddTemplateNode", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			const UUAFVariableDragDropContext* Context = InMenu->FindContext<UUAFVariableDragDropContext>();
			if (Context == nullptr)
			{
				return;
			}

			FAnimNextParamType VariableType = FAnimNextParamType::FromString(Context->TypeName);
			if (!VariableType.IsValid())
			{
				return;
			}

			TConstArrayView<FGraphNodeTemplateInfo> Handlers = FGraphNodeTemplateRegistry::GetDragDropHandlersForVariable(VariableType);
			for (const FGraphNodeTemplateInfo& Handler : Handlers)
			{
				FSoftObjectPath SoftClassPath(Handler.ClassPath);
				UClass* Class = Cast<UClass>(SoftClassPath.TryLoad());
				if (Class == nullptr)
				{
					continue;
				}

				UUAFGraphNodeTemplate* Template = Class->GetDefaultObject<UUAFGraphNodeTemplate>();

				FEditorScriptExecutionGuard AllowScripts;
				FText MenuDesc = FText::Format(LOCTEXT("TemplateLabelFormat", "Add {0}"), Template->GetMenuDescription());
				FText Tooltip = Template->GetTooltipText();

				InMenu->AddMenuEntry("AddTemplateNode",
					FToolMenuEntry::InitMenuEntry(
					"",
					MenuDesc,
					Tooltip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([Template, Context, VariableType]()
						{
							URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(Context->ParentGraph);
							if (EdGraph == nullptr)
							{
								return;
							}

							UAnimNextController* Controller = Cast<UAnimNextController>(EdGraph->GetController());
							if (Controller == nullptr)
							{
								return;
							}

							UObject* SourceObject = nullptr;
							if (!Context->SourceObjectPath.IsEmpty())
							{
								SourceObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(Context->SourceObjectPath);
							}

							if (SourceObject == nullptr)
							{
								return;
							}

							Controller->OpenUndoBracket(TEXT("Drop variables on graph"));

							URigVMUnitNode* NewNode = Template->CreateNewNode(Controller, FVector2D(Context->Location));
							if (NewNode == nullptr)
							{
								Controller->CancelUndoBracket();
								return;
							}

							{
								FEditorScriptExecutionGuard AllowScripts;
								Template->HandleVariableDropped(Controller, NewNode, FAnimNextVariableReference::FromName(Context->Name, SourceObject));
							}

							Controller->CloseUndoBracket();
						}))
					));
				}
		}));
	}
}

void FAnimationGraphMenuExtensions::UnregisterMenus()
{
	if(UToolMenus* ToolMenus = UToolMenus::Get())
	{
		ToolMenus->UnregisterOwnerByName("FAnimNextAnimationGraphItemDetails");
	}
}

}

#undef LOCTEXT_NAMESPACE
