// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UAnimNextEdGraphNode;
class URigVMNode;
class UToolMenu;
struct FToolMenuSection;

namespace UE::UAF::Editor
{

struct FAnimationGraphMenuExtensions
{
	static void RegisterMenus();
	static void UnregisterMenus();
};

}

