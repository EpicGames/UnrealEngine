// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextGraphDetails.h"

#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Graph/AnimNextAnimGraph.h"

namespace UE::UAF::Editor
{

void FAnimNextGraphDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// We dont customize the header here, so our children our added inline
}

void FAnimNextGraphDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedRef<IPropertyHandle> AssetPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextAnimGraph, Asset)).ToSharedRef();
	InChildBuilder.AddProperty(AssetPropertyHandle)
	.DisplayName(InPropertyHandle->GetPropertyDisplayName());

	IDetailCategoryBuilder& CategoryBuilder = InChildBuilder.GetParentCategory();
	TSharedRef<IPropertyHandle> HostGraphPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextAnimGraph, HostGraph)).ToSharedRef();
	CategoryBuilder.AddProperty(HostGraphPropertyHandle, EPropertyLocation::Advanced);
}

}
