// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace UE::UAF::Editor
{

class FAnimNextGraphDetails : public IPropertyTypeCustomization
{
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
};

}