// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextGraphPanelNodeFactory.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextEdGraphNode.h"
#include "Graph/SAnimNextGraphNode.h"

TSharedPtr<SGraphNode> FAnimNextGraphPanelNodeFactory::CreateNode(UEdGraphNode* Node) const
{
	if (UAnimNextEdGraphNode* AnimNextGraphNode = Cast<UAnimNextEdGraphNode>(Node))
	{
		if(UE::UAF::UncookedOnly::FAnimGraphUtils::IsTraitStackNode(AnimNextGraphNode->GetModelNode()))
		{
			TSharedPtr<SGraphNode> GraphNode =
				SNew(SAnimNextGraphNode)
				.GraphNodeObj(AnimNextGraphNode);

			GraphNode->SlatePrepass();
			AnimNextGraphNode->SetDimensions(GraphNode->GetDesiredSize());
			return GraphNode;
		}
	}

	return nullptr;
}
