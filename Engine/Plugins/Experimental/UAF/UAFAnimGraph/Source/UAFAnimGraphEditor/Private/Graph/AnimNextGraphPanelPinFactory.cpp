// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextGraphPanelPinFactory.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextEdGraphNode.h"
#include "Graph/SAnimNextGraphNode.h"
#include "TraitCore/TraitHandle.h"
#include "STraitHandlePin.h"

TSharedPtr<SGraphPin> FAnimNextGraphPanelPinFactory::CreatePin(UEdGraphPin* Pin) const
{
	UAnimNextEdGraphNode* AnimNextGraphNode = Cast<UAnimNextEdGraphNode>(Pin->GetOwningNode());
	if (AnimNextGraphNode == nullptr)
	{
		return nullptr;
	}

	URigVMPin* ModelPin = AnimNextGraphNode->GetModelPinFromPinPath(Pin->GetName());
	if (ModelPin == nullptr)
	{
		return nullptr;
	}

	UScriptStruct* Struct = ModelPin->GetScriptStruct();
	if (Struct == nullptr)
	{
		return nullptr;
	}

	if (!Struct->IsChildOf<FAnimNextTraitHandle>())
	{
		return nullptr;
	}

	return SNew(UE::UAF::STraitHandlePin, Pin);
}
