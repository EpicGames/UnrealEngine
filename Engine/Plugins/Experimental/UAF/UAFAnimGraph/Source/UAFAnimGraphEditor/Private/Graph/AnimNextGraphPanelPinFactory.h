// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

class FAnimNextGraphPanelPinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(UEdGraphPin* Pin) const override;
};
