// Copyright Epic Games, Inc. All Rights Reserved.


#include "GraphNodeTemplateDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "SActionButton.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "String/ParseTokens.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Widgets/SRigVMNodeLayoutWidget.h"

#define LOCTEXT_NAMESPACE "GraphNodeTemplateDetails"

namespace UE::UAF::Editor
{

void FGraphNodeTemplateDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We expect to edit the CDO here
	TArray<TWeakObjectPtr<UObject>> Objects = DetailBuilder.GetSelectedObjects();
	if (Objects.Num() != 1)
	{
		return;
	}

	WeakTemplate = Cast<UUAFGraphNodeTemplate>(Objects[0].Get());
	if (WeakTemplate == nullptr || !WeakTemplate->HasAllFlags(RF_ClassDefaultObject))
	{
		return;
	}

	IDetailCategoryBuilder& TemplateCategory = DetailBuilder.EditCategory("Template", LOCTEXT("TemplateCategory", "Template"), ECategoryPriority::Default);
	
	IDetailPropertyRow& Row = TemplateCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UUAFGraphNodeTemplate, NodeLayout));

	Row
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.CustomWidget()
	.NameContent()
	[
		Row.GetPropertyHandle()->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SComboButton)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EditLayout", "Edit Layout"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.HasDownArrow(true)
			.OnGetMenuContent_Lambda([this]()
			{
				return
					SNew(SBox)
					.WidthOverride(300.0f)
					.HeightOverride(300.0f)
					[
						SNew(SRigVMNodeLayoutWidget)
						.MaxScrollBoxSize(FOptionalSize())
						.OnGetUncategorizedPins(this, &FGraphNodeTemplateDetails::GetUncategorizedPins)
						.OnGetCategories(this, &FGraphNodeTemplateDetails::GetPinCategories)
						.OnGetElementCategory(this, &FGraphNodeTemplateDetails::GetPinCategory)
						.OnGetElementIndexInCategory(this, &FGraphNodeTemplateDetails::GetPinIndexInCategory)
						.OnGetElementLabel(this, &FGraphNodeTemplateDetails::GetPinLabel)
						.OnGetElementColor(this, &FGraphNodeTemplateDetails::GetPinColor)
						.OnGetElementIcon(this, &FGraphNodeTemplateDetails::GetPinIcon)
						.OnCategoryAdded(this, &FGraphNodeTemplateDetails::HandleCategoryAdded)
						.OnCategoryRemoved(this, &FGraphNodeTemplateDetails::HandleCategoryRemoved)
						.OnCategoryRenamed(this, &FGraphNodeTemplateDetails::HandleCategoryRenamed)
						.OnElementCategoryChanged(this, &FGraphNodeTemplateDetails::HandlePinCategoryChanged)
						.OnElementLabelChanged(this, &FGraphNodeTemplateDetails::HandlePinLabelChanged)
						.OnElementIndexInCategoryChanged(this, &FGraphNodeTemplateDetails::HandlePinIndexInCategoryChanged)
						.OnValidateCategoryName(this, &FGraphNodeTemplateDetails::HandleValidateCategoryName)
						.OnValidateElementName(this, &FGraphNodeTemplateDetails::HandleValidatePinDisplayName)
						.OnGetStructuralHash(this, &FGraphNodeTemplateDetails::GetNodeLayoutHash)
					];
			})
		]
	];
}

const FRigVMNodeLayout* FGraphNodeTemplateDetails::GetNodeLayout() const
{
	if(const UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		return &Template->NodeLayout;
	}
	return nullptr;
}

TArray<FString> FGraphNodeTemplateDetails::GetUncategorizedPins() const
{
	if(const UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		TArray<FString> PinPaths;
		for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Template->Traits)
		{
			const FString TraitName = Trait.GetScriptStruct()->GetName();
			if (Template->NodeLayout.FindCategory(TraitName) == nullptr)
			{
				PinPaths.Add(TraitName);
			}

			for (TFieldIterator<FProperty> It(Trait.GetScriptStruct()); It; ++It)
			{
				if (It->HasMetaData(TEXT("Hidden")) || It->HasMetaData(TEXT("Output")))
				{
					continue;
				}

				TStringBuilder<256>  PathBuilder;
				PathBuilder.Append(TraitName);
				PathBuilder.Append(TEXT("."));
				PathBuilder.Append(It->GetName());

				FString PinPath = PathBuilder.ToString();
				if (Template->NodeLayout.FindCategory(PinPath) == nullptr)
				{
					PinPaths.Add(PinPath);
				}
			}
		}
		return PinPaths;
	}
	return TArray<FString>();
}

TArray<FRigVMPinCategory> FGraphNodeTemplateDetails::GetPinCategories() const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		return NodeLayout->Categories;
	}
	return TArray<FRigVMPinCategory>();
}

FString FGraphNodeTemplateDetails::GetPinCategory(FString InPinPath) const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		if (const FString* Category = NodeLayout->FindCategory(InPinPath))
		{
			return *Category;
		}
	}
	return FString();
}

int32 FGraphNodeTemplateDetails::GetPinIndexInCategory(FString InPinPath) const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		if (const int32* IndexPtr = NodeLayout->PinIndexInCategory.Find(InPinPath))
		{
			return *IndexPtr;
		}
	}
	return INDEX_NONE;
}

FString FGraphNodeTemplateDetails::GetPinLabel(FString InPinPath) const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		if(const FString* DisplayName = NodeLayout->FindDisplayName(InPinPath))
		{
			return *DisplayName;
		}
	}
	return FString();
}

FLinearColor FGraphNodeTemplateDetails::GetPinColor(FString InPinPath) const
{
	if(const UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Template->Traits)
		{
			for (TFieldIterator<FProperty> It(Trait.GetScriptStruct()); It; ++It)
			{
				TStringBuilder<256>  PathBuilder;
				PathBuilder.Append(Trait.GetScriptStruct()->GetName());
				PathBuilder.Append(TEXT("."));
				PathBuilder.Append(It->GetName());
				if (InPinPath == PathBuilder.ToString())
				{
					FEdGraphPinType PinType;
					GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(*It, PinType);
					return GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
				}
			}
		}
	}

	return FLinearColor::White;
}

const FSlateBrush* FGraphNodeTemplateDetails::GetPinIcon(FString InPinPath) const
{
	if(const UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Template->Traits)
		{
			for (TFieldIterator<FProperty> It(Trait.GetScriptStruct()); It; ++It)
			{
				TStringBuilder<256>  PathBuilder;
				PathBuilder.Append(Trait.GetScriptStruct()->GetName());
				PathBuilder.Append(TEXT("."));
				PathBuilder.Append(It->GetName());
				if (InPinPath == PathBuilder.ToString())
				{
					FEdGraphPinType PinType;
					GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(*It, PinType);
					return FBlueprintEditorUtils::GetIconFromPin(PinType, /* bIsLarge = */ false);
				}
			}
		}
	}

	return nullptr;
}

void FGraphNodeTemplateDetails::HandleCategoryAdded(FString InCategory)
{
	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		Template->Modify();

		FString Category = InCategory;
		FRigVMPinCategory* ExistingCategory = nullptr;
		int32 Suffix = 0;
		do
		{
			ExistingCategory = Template->NodeLayout.Categories.FindByPredicate([&Category](const FRigVMPinCategory& InPinCategory)
			{
				return InPinCategory.Path == Category;
			});

			if (ExistingCategory)
			{
				Category = FString::Printf(TEXT("%s_%d"), *InCategory, Suffix++);
			}
		}
		while (ExistingCategory != nullptr);

		FRigVMPinCategory& PinCategory = Template->NodeLayout.Categories.AddDefaulted_GetRef();
		PinCategory.Path = Category;
	}
}

void FGraphNodeTemplateDetails::HandleCategoryRemoved(FString InCategory)
{
	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		Template->Modify();
		Template->NodeLayout.Categories.RemoveAll([&InCategory](const FRigVMPinCategory& InPinCategory)
		{
			return InPinCategory.Path == InCategory;
		});
	}
}

void FGraphNodeTemplateDetails::HandleCategoryRenamed(FString InOldCategory, FString InNewCategory)
{
	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		Template->Modify();
		
		FRigVMPinCategory* ExistingCategory = Template->NodeLayout.Categories.FindByPredicate([&InOldCategory](const FRigVMPinCategory& InPinCategory)
		{
			return InPinCategory.Path == InOldCategory;
		});

		if (ExistingCategory)
		{
			ExistingCategory->Path = InNewCategory;
		}
	}
}

void FGraphNodeTemplateDetails::HandlePinCategoryChanged(FString InPinPath, FString InCategory)
{
	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		Template->Modify();

		FRigVMPinCategory* ExistingOldCategory = Template->NodeLayout.Categories.FindByPredicate([&InPinPath](const FRigVMPinCategory& InPinCategory)
		{
			for (const FString& PinPath : InPinCategory.Elements)
			{
				if (PinPath == InPinPath)
				{
					return true;
				}
			}
			return false;
		});

		if (ExistingOldCategory)
		{
			ExistingOldCategory->Elements.Remove(InPinPath);
		}
		
		FRigVMPinCategory* ExistingNewCategory = Template->NodeLayout.Categories.FindByPredicate([&InCategory](const FRigVMPinCategory& InPinCategory)
		{
			return InCategory == InPinCategory.Path;
		});

		if (ExistingNewCategory)
		{
			ExistingNewCategory->Elements.AddUnique(InPinPath);
		}
		else
		{
			FRigVMPinCategory& NewCategory = Template->NodeLayout.Categories.AddDefaulted_GetRef();
			NewCategory.Path = InCategory;
			NewCategory.Elements.Add(InPinPath);
		}
	}
}

void FGraphNodeTemplateDetails::HandlePinLabelChanged(FString InPinPath, FString InNewLabel)
{
	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		Template->Modify();
		Template->NodeLayout.DisplayNames.Add(InPinPath, InNewLabel);
	}
}

void FGraphNodeTemplateDetails::HandlePinIndexInCategoryChanged(FString InPinPath, int32 InIndexInCategory)
{
	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		Template->Modify();
		Template->NodeLayout.PinIndexInCategory.Add(InPinPath, InIndexInCategory);
	}
}

bool FGraphNodeTemplateDetails::ValidateName(FString InNewName, FText& OutErrorMessage)
{
	if(InNewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNamesAreNotAllowed", "Empty names are not allowed.");
		return false;
	}

	if(FChar::IsDigit(InNewName[0]))
	{
		OutErrorMessage = LOCTEXT("NamesCannotStartWithADigit", "Names cannot start with a digit.");
		return false;
	}

	for (int32 i = 0; i < InNewName.Len(); ++i)
	{
		TCHAR& C = InNewName[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||					 // Any letter
			(C == '_') || (C == '-') || (C == ' ') ||				 // _  - space anytime
			FChar::IsDigit(C);										 // 0-9 anytime

		if (!bGoodChar)
		{
			const FText Character = FText::FromString(InNewName.Mid(i, 1));
			OutErrorMessage = FText::Format(LOCTEXT("CharacterNotAllowedFormat", "'{0}' not allowed."), Character);
			return false;
		}
	}

	if (InNewName.Len() > 100)
	{
		OutErrorMessage = LOCTEXT("NameIsTooLong", "Name is too long.");
		return false;
	}

	return true;
}

bool FGraphNodeTemplateDetails::HandleValidateCategoryName(FString InCategoryPath, FString InNewName, FText& OutErrorMessage)
{
	if(!ValidateName(InNewName, OutErrorMessage))
	{
		return false;
	}

	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		// TODO: Possibly too strict here
		if (Template->NodeLayout.Categories.ContainsByPredicate([&InNewName](const FRigVMPinCategory& InPinCategory)
		{
			bool bFound = false;
			UE::String::ParseTokens(InPinCategory.Path, TEXT('|'), [&bFound, &InNewName](FStringView InView)
			{
				if (InView.Equals(InNewName, ESearchCase::IgnoreCase))
				{
					bFound = true;
				}
			});
			return bFound;
		}))
		{
			OutErrorMessage = LOCTEXT("NameIsAlreadyUsed", "Duplicate name.");
			return false;
		}
	}

	return true;
}

bool FGraphNodeTemplateDetails::HandleValidatePinDisplayName(FString InPinPath, FString InNewName, FText& OutErrorMessage)
{
	if(!ValidateName(InNewName, OutErrorMessage))
	{
		return false;
	}

	if(UUAFGraphNodeTemplate* Template = WeakTemplate.Get())
	{
		const TArray<FString> UncategorizedPins = GetUncategorizedPins();

		if (UncategorizedPins.ContainsByPredicate([&InNewName](const FString& InPinName)
		{
			bool bFound = false;
			UE::String::ParseTokens(InPinName, TEXT('.'), [&bFound, &InNewName](FStringView InView)
			{
				if (InView.Equals(InNewName, ESearchCase::IgnoreCase))
				{
					bFound = true;
				}
			});
			return bFound;
		}))
		{
			OutErrorMessage = LOCTEXT("NameIsAlreadyUsedWithinPin", "Duplicate named pin.");
			return false;
		}

		for (const TPair<FString, FString>& StringPair : Template->NodeLayout.DisplayNames)
		{
			if (StringPair.Value.Equals(InNewName, ESearchCase::IgnoreCase))
			{
				OutErrorMessage = LOCTEXT("NameIsAlreadyUsedWithinPin", "Duplicate named pin.");
				return false;
			}
		}
	}

	return true;
}

uint32 FGraphNodeTemplateDetails::GetNodeLayoutHash() const
{
	uint32 Hash = 0;
	if(const FRigVMNodeLayout* Layout = GetNodeLayout())
	{
		Hash = HashCombine(Hash, GetTypeHash(*Layout));
	}
	const TArray<FString> UncategorizedPins = GetUncategorizedPins();
	for(const FString& UncategorizedPin : UncategorizedPins)
	{
		Hash = HashCombine(Hash, GetTypeHash(UncategorizedPin));
	}
	return Hash;
}

}

#undef LOCTEXT_NAMESPACE