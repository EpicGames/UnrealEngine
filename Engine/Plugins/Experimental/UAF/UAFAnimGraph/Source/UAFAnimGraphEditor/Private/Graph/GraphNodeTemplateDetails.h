// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/UAFGraphNodeTemplate.h"

struct FSlateBrush;
struct FRigVMPinCategory;
struct FRigVMNodeLayout;

namespace UE::UAF::Editor
{

class FGraphNodeTemplateDetails : public IDetailCustomization
{
	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
	const FRigVMNodeLayout* GetNodeLayout() const;
	TArray<FString> GetUncategorizedPins() const;
	TArray<FRigVMPinCategory> GetPinCategories() const;
	FString GetPinCategory(FString InPinPath) const;
	int32 GetPinIndexInCategory(FString InPinPath) const;
	FString GetPinLabel(FString InPinPath) const;
	FLinearColor GetPinColor(FString InPinPath) const;
	const FSlateBrush* GetPinIcon(FString InPinPath) const;
	void HandleCategoryAdded(FString InCategory);
	void HandleCategoryRemoved(FString InCategory);
	void HandleCategoryRenamed(FString InOldCategory, FString InNewCategory);
	void HandlePinCategoryChanged(FString InPinPath, FString InCategory);
	void HandlePinLabelChanged(FString InPinPath, FString InNewLabel);
	void HandlePinIndexInCategoryChanged(FString InPinPath, int32 InIndexInCategory);
	static bool ValidateName(FString InNewName, FText& OutErrorMessage);
	bool HandleValidateCategoryName(FString InCategoryPath, FString InNewName, FText& OutErrorMessage);
	bool HandleValidatePinDisplayName(FString InPinPath, FString InNewName, FText& OutErrorMessage);
	uint32 GetNodeLayoutHash() const;

	TWeakObjectPtr<UUAFGraphNodeTemplate> WeakTemplate;

	struct FCachedPinInfo
	{
		FString PinName;
		FString PinCategory;
	};
	
	TArray<TUniquePtr<FCachedPinInfo>> CachedPinInfo;
};

}