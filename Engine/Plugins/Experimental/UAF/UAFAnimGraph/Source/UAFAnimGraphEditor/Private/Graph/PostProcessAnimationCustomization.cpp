// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcessAnimationCustomization.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/PostProcessAnimationAssetUserData.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "UAF/UAFAssetFactory.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::UAF::Editor
{
	bool FPostProcessAnimationCustomization::OnShouldFilterPostProcessAnimation(const FAssetData& AssetData)
	{
		static TArray<UClass*> AllowedClasses(FAssetDataFactory::GetRegisteredObjectClasses());
		return (AllowedClasses.Contains(AssetData.GetClass()) == false);
	}

	FString FPostProcessAnimationCustomization::GetCurrentPostProcessAnimationPath(USkeletalMesh* SkeletalMesh)
	{
		if (SkeletalMesh)
		{
			UPostProcessAnimationUserAssetData* UserAssetData = Cast<UPostProcessAnimationUserAssetData>(SkeletalMesh->GetAssetUserDataOfClass(UPostProcessAnimationUserAssetData::StaticClass()));
			if (UserAssetData)
			{
				return UserAssetData->AnimationAsset.GetPath();
			}
		}

		return {};
	}

	void FPostProcessAnimationCustomization::OnSetPostProcessAnimation(const FAssetData& AssetData, TStrongObjectPtr<USkeletalMesh> SkeletalMesh)
	{
		if (SkeletalMesh)
		{
			UPostProcessAnimationUserAssetData* UserAssetData = Cast<UPostProcessAnimationUserAssetData>(SkeletalMesh->GetAssetUserDataOfClass(UPostProcessAnimationUserAssetData::StaticClass()));

			// Did we select a new and valid new animation asset?
			UObject* NewlySelectedAnimAsset = AssetData.GetAsset();
			if (NewlySelectedAnimAsset)
			{
				// Do we have a user asset data already?
				if (UserAssetData)
				{
					// Set the newly selected anim graph on the already existing user asset data.
					UserAssetData->AnimationAsset = NewlySelectedAnimAsset;
				}
				else
				{
					// We do not have a user asset data yet, create one and set the graph on it.
					UserAssetData = NewObject<UPostProcessAnimationUserAssetData>(SkeletalMesh.Get());
					UserAssetData->AnimationAsset = NewlySelectedAnimAsset;
					SkeletalMesh->AddAssetUserData(UserAssetData);
				}
			}
			else
			{
				// The anim graph asset got cleared, remove the user asset data.
				SkeletalMesh->RemoveUserDataOfClass(UPostProcessAnimationUserAssetData::StaticClass());
			}
		}
	}

	void FPostProcessAnimationCustomization::OnCustomizeMeshDetails(IDetailLayoutBuilder& DetailLayout, TWeakObjectPtr<USkeletalMesh> SkeletalMeshWeak)
	{
		const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();

		IDetailCategoryBuilder& SkelMeshCategory = DetailLayout.EditCategory("Animation");

		const FText PropertyText = FText::FromString("Post-Process Animation");
		FDetailWidgetRow& PostProcessAnimGraphRow = SkelMeshCategory.AddCustomRow(PropertyText);
		PostProcessAnimGraphRow.NameContent()
			[
				SNew(STextBlock)
					.Text(PropertyText)
					.Font(DetailFontInfo)
			];

		TSharedPtr<SObjectPropertyEntryBox> PostProcessAnimGraphWidget = SNew(SObjectPropertyEntryBox)
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateStatic(&FPostProcessAnimationCustomization::OnShouldFilterPostProcessAnimation))
			.ObjectPath_Lambda([SkeletalMeshWeak]()
				{
					return GetCurrentPostProcessAnimationPath(SkeletalMeshWeak.Get());
				})
			.OnObjectChanged_Lambda([SkeletalMeshWeak](const FAssetData& AssetData)
				{
					OnSetPostProcessAnimation(AssetData, SkeletalMeshWeak.Pin());
				})
			;

		PostProcessAnimGraphRow.ValueContent()
			[
				PostProcessAnimGraphWidget.ToSharedRef()
			];
	}
}