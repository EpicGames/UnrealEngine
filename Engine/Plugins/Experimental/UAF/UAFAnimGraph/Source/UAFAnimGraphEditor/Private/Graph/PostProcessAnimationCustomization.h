// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Engine/SkeletalMesh.h"
#include "Graph/AnimNextAnimGraph.h"
#include "DetailCategoryBuilder.h"

namespace UE::UAF::Editor
{
	class FPostProcessAnimationCustomization
	{
	public:
		static bool OnShouldFilterPostProcessAnimation(const FAssetData& AssetData);
		static FString GetCurrentPostProcessAnimationPath(USkeletalMesh* SkeletalMesh);
		static void OnSetPostProcessAnimation(const FAssetData& AssetData, TStrongObjectPtr<USkeletalMesh> SkeletalMesh);
		static void OnCustomizeMeshDetails(IDetailLayoutBuilder& DetailLayout, TWeakObjectPtr<USkeletalMesh> SkeletalMeshWeak);
	};
}
