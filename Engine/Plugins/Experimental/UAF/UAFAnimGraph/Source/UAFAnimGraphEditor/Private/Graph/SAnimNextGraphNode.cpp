// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/SAnimNextGraphNode.h"
#include "AnimNextEdGraphNode.h"
#include "AnimNextTraitStackUnitNode.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SAnimNextGraphNode"

void SAnimNextGraphNode::Construct( const FArguments& InArgs )
{
	SRigVMGraphNode::Construct(SRigVMGraphNode::FArguments().GraphNodeObj(InArgs._GraphNodeObj));
}

TSharedRef<SWidget> SAnimNextGraphNode::CreateIconWidget()
{
	UAnimNextEdGraphNode* AnimNextEdGraphNode = Cast<UAnimNextEdGraphNode>(GraphNode);
	UAnimNextTraitStackUnitNode* TraitStackNode = AnimNextEdGraphNode != nullptr ? Cast<UAnimNextTraitStackUnitNode>(AnimNextEdGraphNode->GetModelNode()) : nullptr;
	if (TraitStackNode == nullptr)
	{
		return SRigVMGraphNode::CreateIconWidget();
	}

	// Get node icon tint
	IconColor = FLinearColor::White;
	GraphNode->GetIconAndTint(IconColor);

	return
		SNew(SBox)
		.WidthOverride(16.0f)
		.HeightOverride(16.0f)
		[
			SNew(SImage)
			.Image(TraitStackNode->GetDefaultNodeIconBrush())
			.ColorAndOpacity(this, &SAnimNextGraphNode::GetNodeTitleIconColor)
		];
}

#undef LOCTEXT_NAMESPACE
