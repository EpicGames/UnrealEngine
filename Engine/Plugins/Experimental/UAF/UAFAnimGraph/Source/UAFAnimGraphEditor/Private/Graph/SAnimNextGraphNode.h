// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SRigVMGraphNode.h"

class UAnimNextEdGraphNode;

class SAnimNextGraphNode : public SRigVMGraphNode
{
public:
	SLATE_BEGIN_ARGS(SAnimNextGraphNode)
		: _GraphNodeObj(nullptr)
		{}

	SLATE_ARGUMENT(UAnimNextEdGraphNode*, GraphNodeObj)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	// SGraphNode interface
	virtual TSharedRef<SWidget> CreateIconWidget() override;
};
