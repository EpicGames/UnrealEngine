// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraitEditorView.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SAssetDropTarget.h"
#include "ScopedTransaction.h"
#include "WorkspaceSchema.h"
#include "Framework/Commands/GenericCommands.h"
#include "TraitCore/TraitInterfaceRegistry.h"
#include "TraitCore/TraitRegistry.h"
#include "ObjectEditorUtils.h"
#include "AnimNextEdGraphNode.h"
#include "IAnimNextEditorModule.h"
#include "STraitListView.h"
#include "STraitStackView.h"
#include "MessageLogModule.h"
#include "IWorkspaceEditor.h"
#include "IMessageLogListing.h"
#include "AnimNextController.h"
#include "ITraitStackEditor.h"
#include "UncookedOnlyUtils.h"
#include "Logging/MessageLog.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "TraitListEditor"

namespace UE::UAF::Editor
{

STraitEditorView::STraitEditorView()
	: TraitEditorSharedData(MakeShared<FTraitEditorSharedData>())
{
	TraitEditorSharedData->CurrentTraitsDataShared = MakeShared<TArray<TSharedPtr<FTraitDataEditorDef>>>();
}

void STraitEditorView::Construct(const FArguments& InArgs, TWeakPtr<UE::Workspace::IWorkspaceEditor> InWorkspaceEditorWeak)
{
	WorkspaceEditorWeak = MoveTemp(InWorkspaceEditorWeak);

	TWeakObjectPtr<UAnimNextEdGraphNode> EdGraphNodeWeak = TraitEditorSharedData->EdGraphNodeWeak;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0)
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					GetOptionsMenuWidget()
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0)
				.Padding(0.f, 10.f, 0.f, 0.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					[
						SAssignNew(TraitListWidget, STraitListView, TraitEditorSharedData)
						.OnTraitClicked(STraitListView::FOnTraitClicked::CreateSP(this, &STraitEditorView::OnTraitClicked))
						.OnGetSelectedTraitData(STraitListView::FOnGetSelectedTraitData::CreateSP(this, &STraitEditorView::OnGetSelectedTraitData))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					[
						SAssignNew(TraitStackWidget, STraitStackView, TraitEditorSharedData)
						.OnTraitDeleteRequest(STraitStackView::FOnStackTraitDeleteRequest::CreateSP(this, &STraitEditorView::OnTraitDeleteRequest))
						.OnStatckTraitSelectionChanged(STraitStackView::FOnStatckTraitSelectionChanged::CreateSP(this, &STraitEditorView::OnStatckTraitSelectionChanged))
						.OnStackTraitDragAccepted(STraitStackView::FOnStackTraitDragAccepted::CreateSP(this, &STraitEditorView::OnStackTraitDragAccepted))
					]
				]
			]
		]
	];
}

void STraitEditorView::SetTraitData(const FTraitStackData& InTraitStackData)
{
	UAnimNextEdGraphNode* AnimNextEdGraphNode = InTraitStackData.EdGraphNodeWeak.Get();

	// If this is not a trait stack node, clear the shared data
	if(AnimNextEdGraphNode && !UE::UAF::UncookedOnly::FAnimGraphUtils::IsTraitStackNode(AnimNextEdGraphNode->GetModelNode()))
	{
		TraitEditorSharedData->EdGraphNodeWeak = nullptr;
	}
	else
	{
		TraitEditorSharedData->EdGraphNodeWeak = InTraitStackData.EdGraphNodeWeak;
	}

	GenerateTraitStackData(TraitEditorSharedData->EdGraphNodeWeak, TraitEditorSharedData);

	Refresh();
}

FReply STraitEditorView::OnTraitClicked(const FTraitUID InClickedTraitUID)
{
	FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
	if (const FTrait* Trait = TraitRegistry.Find(InClickedTraitUID))
	{
		if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(TraitEditorSharedData->EdGraphNodeWeak))
		{
			if (UAnimNextController* Controller = Cast<UAnimNextController>(EdGraphNode->GetController()))
			{
				int32 TraitIndex = INDEX_NONE;
				TSharedPtr<FTraitDataEditorDef> SwapTraitData = FTraitEditorUtils::FindTraitInCurrentStackData(SelectedTraitUID, TraitEditorSharedData->CurrentTraitsDataShared, &TraitIndex);

				const bool bIsAddingMissingBaseTrait = Trait->GetTraitMode() == ETraitMode::Base && TraitIndex == 0 && (*TraitEditorSharedData->CurrentTraitsDataShared)[0]->TraitUID == FTraitUID();
				const int32 PinIndex = GetTraitPinIndex(EdGraphNode, SwapTraitData, TraitIndex);

				const FName NewTraitTypeName = *Trait->GetTraitName();
				if (TraitIndex == INDEX_NONE || bIsAddingMissingBaseTrait)
				{
					Controller->AddTraitByName(EdGraphNode->GetFName(), NewTraitTypeName, PinIndex,TEXT(""), true, true);
				}
				else
				{
					Controller->SwapTraitByName(EdGraphNode->GetFName(), SwapTraitData->TraitName, PinIndex, NewTraitTypeName, TEXT(""), true, true);
				}
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TWeakPtr<FTraitDataEditorDef> STraitEditorView::OnGetSelectedTraitData() const
{
	return StackSelectedTrait.ToWeakPtr();
}

FReply STraitEditorView::OnTraitDeleteRequest(const FTraitUID InTraitUIDToDelete)
{
	if (TArray<TSharedPtr<FTraitDataEditorDef>>* CurrentTraitsData = TraitEditorSharedData->CurrentTraitsDataShared.Get())
	{
		const int32 Num = CurrentTraitsData->Num();
		for (int32 i = Num - 1; i >= 0; --i)
		{
			TSharedPtr<FTraitDataEditorDef>& TraitDataEditorDef = (*CurrentTraitsData)[i];
			if (TraitDataEditorDef->TraitUID == InTraitUIDToDelete)
			{
				if (TraitEditorSharedData->EdGraphNodeWeak.IsValid())
				{
					if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(TraitEditorSharedData->EdGraphNodeWeak.Get()))
					{
						if (UAnimNextController* Controller = Cast<UAnimNextController>(EdGraphNode->GetController()))
						{
							Controller->RemoveTraitByName(EdGraphNode->GetFName(), TraitDataEditorDef->TraitName, true, true);
						}
					}
				}

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply STraitEditorView::OnStackTraitDragAccepted(const FTraitUID DraggedTraitUID, const FTraitUID TargetTraitUID, EItemDropZone DropZone)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSPLambda(this, [this, DraggedTraitUID, TargetTraitUID, DropZone]()
	{
		ExecuteTraitDrag(DraggedTraitUID, TargetTraitUID, DropZone);
	}));

	return FReply::Handled();
}

void STraitEditorView::ExecuteTraitDrag(const FTraitUID DraggedTraitUID, const FTraitUID TargetTraitUID, EItemDropZone DropZone)
{
	FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

	const FTrait* DraggedTrait = TraitRegistry.Find(DraggedTraitUID);
	
	if (DraggedTrait != nullptr)
	{
		if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(TraitEditorSharedData->EdGraphNodeWeak))
		{
			if (UAnimNextController* Controller = Cast<UAnimNextController>(EdGraphNode->GetController()))
			{
				int32 TargetTraitIndex = INDEX_NONE;
				TSharedPtr<FTraitDataEditorDef> SwapTraitData = FTraitEditorUtils::FindTraitInCurrentStackData(TargetTraitUID, TraitEditorSharedData->CurrentTraitsDataShared, &TargetTraitIndex);
				const int32 PinIndex = GetTraitPinIndex(EdGraphNode, SwapTraitData, TargetTraitIndex);
				const bool bIsAddingMissingBaseTrait = DraggedTrait->GetTraitMode() == ETraitMode::Base && TargetTraitIndex == 0 && (*TraitEditorSharedData->CurrentTraitsDataShared)[0]->TraitUID == FTraitUID();

				if (TargetTraitIndex != INDEX_NONE || bIsAddingMissingBaseTrait)
				{
					const FName DraggedTraitTypeName = *DraggedTrait->GetTraitName();

					if (DropZone == EItemDropZone::OntoItem)
					{
						if(bIsAddingMissingBaseTrait)
						{
							Controller->AddTraitByName(EdGraphNode->GetFName(), DraggedTraitTypeName, PinIndex, TEXT(""), true, true);
						}
						else
						{
							Controller->SwapTraitByName(EdGraphNode->GetFName(), SwapTraitData->TraitName, PinIndex, DraggedTraitTypeName, TEXT(""), true, true);
						}
					}
					else
					{
						int32 DraggedTraitIndex = INDEX_NONE;
						TSharedPtr<FTraitDataEditorDef> DraggedTraitData = FTraitEditorUtils::FindTraitInCurrentStackData(DraggedTraitUID, TraitEditorSharedData->CurrentTraitsDataShared, &DraggedTraitIndex);

						if (DraggedTraitIndex != INDEX_NONE)
						{
							Controller->SetTraitPinIndex(EdGraphNode->GetFName(), DraggedTraitData->TraitName, PinIndex + 1, true, true);
						}
						else
						{
							Controller->AddTraitByName(EdGraphNode->GetFName(), DraggedTraitTypeName, PinIndex + 1, TEXT(""), true, true);
						}
					}
				}
			}
		}
	}
}

void STraitEditorView::OnStatckTraitSelectionChanged(const FTraitUID InTraitSelected)
{
	SelectedTraitUID = InTraitSelected;

	StackSelectedTrait = InTraitSelected != FTraitUID() ? FTraitEditorUtils::FindTraitInCurrentStackData(InTraitSelected, TraitEditorSharedData->CurrentTraitsDataShared) : nullptr;
}

void STraitEditorView::Refresh()
{
	RefreshTraitStackTraitsStatus();
	RefreshWidgets();
}

void STraitEditorView::RefreshWidgets()
{
	TraitListWidget->RefreshList();
	TraitStackWidget->RefreshList();
}

void STraitEditorView::RefreshTraitStack()
{
	GenerateTraitStackData(TraitEditorSharedData->EdGraphNodeWeak, TraitEditorSharedData);
	RefreshTraitStackTraitsStatus();

	TraitStackWidget->RefreshList();
}

void STraitEditorView::OnRequestRefresh()
{
	Refresh();
}

void STraitEditorView::RefreshTraitStackTraitsStatus()
{
	if (TraitEditorSharedData->CurrentTraitsDataShared.IsValid())
	{
		if (TraitEditorSharedData->EdGraphNodeWeak.IsValid())
		{
			TraitEditorSharedData->bStackContainsErrors = false;

			TArray<TSharedPtr<FTraitDataEditorDef>>& CurrentTraitsData = *(TraitEditorSharedData->CurrentTraitsDataShared.Get());

			TArray<TSharedRef<FTokenizedMessage>> Messages;

			const int32 NumTraits = CurrentTraitsData.Num();
			for (int32 i = NumTraits - 1; i >= 0; i--)
			{
				TSharedPtr<FTraitDataEditorDef>& TraitData = CurrentTraitsData[i];
				UpdateTraitStatusInStack(CurrentTraitsData, i, TraitData);
				
				for (FTraitInterfaceUID TtaitInterfaceUID : TraitData->StackStatus.MissingInterfaces)
				{
					TraitEditorSharedData->StackMissingInterfaces.AddUnique(TtaitInterfaceUID);
					
					const int32 StackUsedInterfacesIndex = TraitEditorSharedData->StackUsedInterfaces.Find(TtaitInterfaceUID);
					ensure(StackUsedInterfacesIndex != INDEX_NONE);
					TraitEditorSharedData->StackUsedInteraceMissingIndexes.AddUnique(StackUsedInterfacesIndex);
				}

				if (TraitData->StackStatus.StatusMessages.Num() > 0)
				{
					TraitEditorSharedData->bStackContainsErrors = true;

					for (const FTraitStackTraitStatus::FStatusMessage& StatusMessage : TraitData->StackStatus.StatusMessages)
					{
						const EMessageSeverity::Type Severity = (StatusMessage.Status == FTraitStackTraitStatus::EStackStatus::Warning) ? EMessageSeverity::Warning : EMessageSeverity::Error;
						const FText& MessageText = StatusMessage.MessageText;
						Messages.Add(FTokenizedMessage::Create(Severity, FText::Format(LOCTEXT("TraitEditorLogTraitNameErrorFormat", "{0}: {1}"), TraitData->TraitDisplayName, MessageText)));
					}

					// Open tab to display errors
					if (TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WorkspaceEditorWeak.Pin())
					{
						WorkspaceEditor->GetTabManager()->TryInvokeTab(FTabId(UE::UAF::Editor::CompilerResultsTabName));
					}
				}
			}

			if(Messages.Num())
			{
				FMessageLog Log("AnimNextCompilerResults");
				Log.NewPage(LOCTEXT("TraitStackCompileResults", "Trait Stack Compilation"));
				Log.AddMessages(Messages);
			}
		}
		else
		{
			TraitEditorSharedData->CurrentTraitsDataShared->Reset();
		}
	}
}

void STraitEditorView::UpdateTraitStatusInStack(const TArray<TSharedPtr<FTraitDataEditorDef>>& CurrentTraitsData, int32 TraitIndex, TSharedPtr<FTraitDataEditorDef>& TraitData)
{
	switch (TraitData->TraitMode)
	{
		case ETraitMode::Base:
		{
			if (TraitData->TraitUID == FTraitUID())
			{
				TraitData->StackStatus.StatusMessages.Add(FTraitStackTraitStatus::FStatusMessage(FTraitStackTraitStatus::EStackStatus::Error, FText(LOCTEXT("TraitStatusInStack_InvalidBaseTrait", "Base Trait Data is Invalid. Please, select a new Base Trait."))));
				return;
			}
			if (CurrentTraitsData[0]->TraitUID != TraitData->TraitUID)
			{
				TraitData->StackStatus.StatusMessages.Add(FTraitStackTraitStatus::FStatusMessage(FTraitStackTraitStatus::EStackStatus::Error, FText(LOCTEXT("TraitStatusInStack_BaseNotAtTop", "Base Traits should be the first Trait in the Stack."))));
				return;
			}
			break;
		}
		case ETraitMode::Additive:
		{
			if (TraitData->TraitUID == FTraitUID())
			{
				TraitData->StackStatus.StatusMessages.Add(FTraitStackTraitStatus::FStatusMessage(FTraitStackTraitStatus::EStackStatus::Error, FText(LOCTEXT("TraitStatusInStack_InvalidAdditiveTrait", "Additive Trait Data is Invalid. Please, fix the Stack."))));
				return;
			}
			if (CurrentTraitsData[0]->TraitUID == TraitData->TraitUID)
			{
				TraitData->StackStatus.StatusMessages.Add(FTraitStackTraitStatus::FStatusMessage(FTraitStackTraitStatus::EStackStatus::Error, FText(LOCTEXT("TraitStatusInStack_AdditiveAtTop", "Additive Traits can not be at the Top of the Stack."))));
				return;
			}
			break;
		}
		default:
		{
			TraitData->StackStatus.StatusMessages.Add(FTraitStackTraitStatus::FStatusMessage(FTraitStackTraitStatus::EStackStatus::Error, FText(LOCTEXT("TraitStatusInStack_InvalidTraitData", "Trait data is invalid, please correct the Stack."))));
			break;
		}
	}

	const TArray<FTraitInterfaceUID>& RequiredInterfaces = TraitData->RequiredInterfaces;

	const int32 NumRequired = RequiredInterfaces.Num();
	if (NumRequired > 0)
	{
		for (int32 RequiredIndex = 0; RequiredIndex < NumRequired; RequiredIndex++)
		{
			const FTraitInterfaceUID& RequiredInterface = RequiredInterfaces[RequiredIndex];

			bool bFound = false;
			const int32 StartIndex = (TraitData->TraitMode == ETraitMode::Base) 
				? CurrentTraitsData.Num() - 1 // Base traits scan all stack to find a valid interface
				: TraitIndex; // Additive traits start search from current trait to enable traits that inherit from a trait with a required interface that is implemented in the derived class
			
			for (int32 SearchStartIndex = StartIndex; SearchStartIndex >= 0 && !bFound; SearchStartIndex--)
			{
				const TSharedPtr<FTraitDataEditorDef>& ParentTraitData = CurrentTraitsData[SearchStartIndex];

				const TArray<FTraitInterfaceUID>& ParentImplementedInterfaces = ParentTraitData->ImplementedInterfaces;

				for (const FTraitInterfaceUID& ParentImplementedInterface : ParentImplementedInterfaces)
				{
					if (ParentImplementedInterface == RequiredInterface)
					{
						bFound = true;
						break;
					}
				}
			}
			if (!bFound)
			{
				TraitData->StackStatus.MissingInterfaces.Add(RequiredInterface);
			}
		}

		for (const FTraitInterfaceUID& MissingInterface : TraitData->StackStatus.MissingInterfaces)
		{
			if (const ITraitInterface* TraitInterface = FTraitInterfaceRegistry::Get().Find(MissingInterface))
			{
				const FText InterfaceName(TraitInterface->GetDisplayName());
				const FText MissingError = FText::Format(LOCTEXT("TraitStatusInStack_MissingInterface", "Trait {0} requires a parent implementing interface {1}"), TraitData->TraitDisplayName, InterfaceName);
				TraitData->StackStatus.StatusMessages.Add(FTraitStackTraitStatus::FStatusMessage(FTraitStackTraitStatus::EStackStatus::Warning, MissingError));
			}
		}
	}

	TraitData->StackStatus.TraitStatus = TraitData->StackStatus.HasErrors()
											? FTraitStackTraitStatus::EStackStatus::Error 
											: TraitData->StackStatus.HasWarnings() 
												? FTraitStackTraitStatus::EStackStatus::Warning
												: FTraitStackTraitStatus::EStackStatus::Ok;
}

TSharedRef<SWidget> STraitEditorView::GetOptionsMenuWidget()
{
	TSharedPtr<SImage> FilterImage = SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
		.ColorAndOpacity(FSlateColor::UseForeground());

	FMenuBuilder DetailViewOptions(true, nullptr);

	TSharedPtr<FTraitEditorSharedData>& TraitEditorSharedDataLocal = TraitEditorSharedData;

	DetailViewOptions.AddMenuEntry(
		LOCTEXT("TraitEditor_ShowTraitInterfaces", "Show Trait Interfaces"),
		LOCTEXT("TraitEditor_ShowTraitInterfaces_ToolTip", "Displays Trait Implemented and Required interfaces"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, TraitEditorSharedDataLocal]()
			{
				if (TraitEditorSharedDataLocal.IsValid())
				{
					// TODO zzz : Store bShowTraitInterfaces in a serialized settings class somewhere
					TraitEditorSharedDataLocal->bShowTraitInterfaces = !TraitEditorSharedDataLocal->bShowTraitInterfaces;
					TraitEditorSharedDataLocal->bShowTraitInterfacesIfWarningsOrErrors = false;
					RefreshWidgets();
				}
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([TraitEditorSharedDataLocal]()
			{
				return (TraitEditorSharedDataLocal.IsValid()) ? TraitEditorSharedDataLocal->bShowTraitInterfaces : false;
			})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	DetailViewOptions.AddMenuEntry(
		LOCTEXT("TraitEditor_ShowTraitInterfaces_Errors", "Show Trait Interfaces If Warnings / Errors"),
		LOCTEXT("TraitEditor_ShowTraitInterfaces_Errors_ToolTip", "Displays Trait Implemented and Required interfaces if is there any Warning or Error on the Stack"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, TraitEditorSharedDataLocal]()
				{
					if (TraitEditorSharedDataLocal.IsValid())
					{
						// TODO zzz : Store bShowTraitInterfacesIfWarningsOrErrors in a serialized settings class somewhere
						TraitEditorSharedDataLocal->bShowTraitInterfacesIfWarningsOrErrors = !TraitEditorSharedDataLocal->bShowTraitInterfacesIfWarningsOrErrors;
						TraitEditorSharedDataLocal->bShowTraitInterfaces = false;
						RefreshWidgets(); //RefreshTraitStack();
					}
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([TraitEditorSharedDataLocal]()
				{
					return (TraitEditorSharedDataLocal.IsValid()) ? TraitEditorSharedDataLocal->bShowTraitInterfacesIfWarningsOrErrors : false;
				})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	DetailViewOptions.AddMenuEntry(
		LOCTEXT("TraitEditor_AdvancedView", "Advanced View"),
		LOCTEXT("TraitEditor_AdvancedView_ToolTip", "Displays all Traits, including hidden ones"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, TraitEditorSharedDataLocal]()
				{
					if (TraitEditorSharedDataLocal.IsValid())
					{
						// TODO zzz : Store bShowTraitInterfacesIfWarningsOrErrors in a serialized settings class somewhere
						TraitEditorSharedDataLocal->bAdvancedView = !TraitEditorSharedDataLocal->bAdvancedView;
						RefreshWidgets(); //RefreshTraitStack();
					}
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([TraitEditorSharedDataLocal]()
				{
					return (TraitEditorSharedDataLocal.IsValid()) ? TraitEditorSharedDataLocal->bAdvancedView : false;
				})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);


	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0)
		.HAlign(HAlign_Right)
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.ContentPadding(0.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
			.MenuContent()
			[
				DetailViewOptions.MakeWidget()
			]
			.ButtonContent()
			[
				FilterImage.ToSharedRef()
			]
		];
}

int32 STraitEditorView::GetTraitPinIndex(UAnimNextEdGraphNode* InEdGraphNode, const TSharedPtr<FTraitDataEditorDef>& InTraitData, int32 InTraitIndex)
{
	if (InEdGraphNode == nullptr || !InTraitData.IsValid())
	{
		return INDEX_NONE;
	}

	if (URigVMNode* ModelNode = InEdGraphNode->GetModelNode())
	{
		// Special case for a stack that has no Base Trait but has Additive traits (we have an empty trait at index 0)
		if (InTraitIndex == 0 && TraitEditorSharedData->CurrentTraitsDataShared.IsValid() && TraitEditorSharedData->CurrentTraitsDataShared->Num() > 0 && (*TraitEditorSharedData->CurrentTraitsDataShared)[0]->TraitUID == FTraitUID())
		{
			// Get first index that is a decorator pin, skipping non decorator pins and execute pins
			const TArray<URigVMPin*> Pins = ModelNode->GetPins();
			int32 Index = 0;
			for (const URigVMPin* Pin : Pins)
			{
				if (!Pin->IsTraitPin() || Pin->IsExecuteContext())
				{
					Index++;
					continue;
				}

				break;
			}
			return Index;
		}
		else
		{
			// Obtain the pins from the stack
			const TArray<URigVMPin*> TraitPins = ModelNode->GetTraitPins();
			for (const URigVMPin* TraitPin : TraitPins)
			{
				if (TraitPin->IsExecuteContext())
				{
					continue;
				}

				if (TraitPin->GetFName() == InTraitData->TraitName)
				{
					return TraitPin->GetPinIndex();
				}
			}
		}
	}

	return INDEX_NONE;
}

/*static*/ void STraitEditorView::GenerateTraitStackData(const TWeakObjectPtr<UAnimNextEdGraphNode>& EdGraphNodeWeak, TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData)
{
	if (!ensure(InTraitEditorSharedData.IsValid() && InTraitEditorSharedData->CurrentTraitsDataShared.IsValid()))
	{
		return;
	}

	TArray<TSharedPtr<FTraitDataEditorDef>>& TraitsData = *InTraitEditorSharedData->CurrentTraitsDataShared.Get();
	TArray<FTraitInterfaceUID>& StackUsedInterfaces = InTraitEditorSharedData->StackUsedInterfaces;
	TArray<FTraitInterfaceUID>& StackMissingInterfaces = InTraitEditorSharedData->StackMissingInterfaces;
	
	TraitsData.Reset();
	StackUsedInterfaces.Reset();
	StackMissingInterfaces.Reset();

	if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(EdGraphNodeWeak.Get()))
	{
		if (URigVMNode* ModelNode = EdGraphNode->GetModelNode())
		{
			int32 NumBaseTraits = 0;

			// Obtain the pins from the stack
			const TArray<URigVMPin*> TraitPins = ModelNode->GetTraitPins();
			const int32 NumPins = TraitPins.Num();
			if (NumPins > 0)
			{
				FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

				TArray<int32> StackUsedInterfacesNumHits;
				StackUsedInterfacesNumHits.Reset(NumPins);

				// For each Trait (represented as a pin in the node)
				for (const URigVMPin* TraitPin : TraitPins)
				{
					if (TraitPin->IsExecuteContext())
					{
						continue;
					}

					// Create a temporary trait instance, in order to get the correct TraitSharedDataStruct
					if (TSharedPtr<FStructOnScope> ScopedTrait = ModelNode->GetTraitInstance(TraitPin->GetFName()))
					{
						if (const FRigVMTrait* RigVMTrait = (FRigVMTrait*)ScopedTrait->GetStructMemory())
						{
							if (TWeakObjectPtr<UScriptStruct> TraitStruct = RigVMTrait->GetTraitSharedDataStruct(); TraitStruct.IsValid())
							{
								if (const FTrait* Trait = TraitRegistry.Find(TraitStruct.Get()))
								{
									TConstArrayView<FTraitInterfaceUID> TraitImplementedInterfaces = Trait->GetTraitInterfaces();
									TConstArrayView<FTraitInterfaceUID> TraitRequiredInterfaces = Trait->GetTraitRequiredInterfaces();
									
									TSharedPtr<FTraitDataEditorDef>& TraitData = TraitsData.Add_GetRef(MakeShared<FTraitDataEditorDef>(*RigVMTrait->GetName()
																																		, TraitStruct->GetDisplayNameText()
																																		, Trait->GetTraitUID()
																																		, Trait->GetTraitMode()
																																		, TArray<FTraitInterfaceUID>(TraitImplementedInterfaces)
																																		, TArray<FTraitInterfaceUID>(TraitRequiredInterfaces)
																																		, Trait->MultipleInstanceSupport()));

									for (const FTraitInterfaceUID& TraitInterface : TraitImplementedInterfaces)
									{
										if (!FTraitEditorUtils::IsInternal(TraitInterface))
										{
											int32 Index = StackUsedInterfaces.Find(TraitInterface);
											if (Index != INDEX_NONE)
											{
												StackUsedInterfacesNumHits[Index]++;
											}
											else
											{
												Index = StackUsedInterfaces.Add(TraitInterface);
												StackUsedInterfacesNumHits.Add(1);
											}
										}
									}

									for (const FTraitInterfaceUID& TraitInterface : TraitRequiredInterfaces)
									{
										if (!FTraitEditorUtils::IsInternal(TraitInterface))
										{
											int32 Index = StackUsedInterfaces.Find(TraitInterface);
											if (Index != INDEX_NONE)
											{
												StackUsedInterfacesNumHits[Index]++;
											}
											else
											{
												Index = StackUsedInterfaces.Add(TraitInterface);
												StackUsedInterfacesNumHits.Add(1);
											}
										}
									}
								}
							}
						}
					}
				}

				const int32 NumUsedInterfaces = StackUsedInterfaces.Num();
				if (NumUsedInterfaces > 0)
				{
					// Generate the stack interface used indexes for each Trait
					for (TSharedPtr<FTraitDataEditorDef>& TraitData : TraitsData)
					{
						FTraitEditorUtils::GenerateStackInterfacesUsedIndexes(TraitData, InTraitEditorSharedData);
					}
				}

				for (const TSharedPtr<FTraitDataEditorDef>& TraitData : TraitsData)
				{
					if (TraitData->TraitMode == ETraitMode::Base)
					{
						NumBaseTraits++;
					}
				}
			}

			if (NumBaseTraits == 0)
			{
				TraitsData.Insert(MakeShared<FTraitDataEditorDef>(NAME_None, LOCTEXT("BaseTraitUnset", "<Base Trait Unset>"), FTraitUID(), ETraitMode::Base, TArray<FTraitInterfaceUID>(), TArray<FTraitInterfaceUID>(), false), 0);
			}
		}
	}
	else
	{
		TraitsData.Reset();
	}
}

} // end namespace UE::Workspace

#undef LOCTEXT_NAMESPACE
