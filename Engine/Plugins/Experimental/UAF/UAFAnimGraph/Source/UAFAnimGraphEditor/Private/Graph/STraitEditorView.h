// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"

#include "Graph/TraitEditorDefs.h"


class UAnimNextController;
class UAnimNextEdGraphNode;
class IMessageLogListing;

namespace UE::Workspace
{

class IWorkspaceEditor;

}

namespace UE::UAF::Editor
{
struct FTraitStackData;

class STraitListView;
class STraitStackView;

struct FTraitEditorViewEntry;

}


namespace UE::UAF::Editor
{

class STraitEditorView : public SCompoundWidget
{
public:
	STraitEditorView();

	SLATE_BEGIN_ARGS(STraitEditorView) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<UE::Workspace::IWorkspaceEditor> InWorkspaceEditorWeak);

	void SetTraitData(const FTraitStackData& InTraitStackData);

private:
	FReply OnTraitClicked(const FTraitUID InTraitClicked);
	FReply OnTraitDeleteRequest(const FTraitUID InTraitUIDToDelete);
	FReply OnStackTraitDragAccepted(const FTraitUID DraggedTraitUID, const FTraitUID TargetTraitUID, EItemDropZone DropZone);
	void ExecuteTraitDrag(const FTraitUID DraggedTraitUID, const FTraitUID TargetTraitUID, EItemDropZone DropZone);
	void OnStatckTraitSelectionChanged(const FTraitUID InTraitSelected);
	TWeakPtr<FTraitDataEditorDef> OnGetSelectedTraitData() const;

	void Refresh();
	void RefreshWidgets();
	void RefreshTraitStack();

	void OnRequestRefresh();
	void RefreshTraitStackTraitsStatus();
	void UpdateTraitStatusInStack(const TArray<TSharedPtr<FTraitDataEditorDef>>& CurrentTraitsData, int32 TraitIndex, TSharedPtr<FTraitDataEditorDef>& TraitData);

	TSharedRef<SWidget> GetOptionsMenuWidget();

	int32 GetTraitPinIndex(UAnimNextEdGraphNode* InEdGraphNode, const TSharedPtr<FTraitDataEditorDef>& InTraitData, int32 TraitIndex = INDEX_NONE);

	static void GenerateTraitStackData(const TWeakObjectPtr<UAnimNextEdGraphNode>& EdGraphNodeWeak, TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData);

	TWeakPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditorWeak;
	TSharedPtr<FTraitEditorSharedData> TraitEditorSharedData;

	TSharedPtr<STraitListView> TraitListWidget;
	TSharedPtr<STraitStackView> TraitStackWidget;

	TSharedPtr<FTraitDataEditorDef> StackSelectedTrait;

	TWeakPtr<IMessageLogListing> CompilerResultsListingWeak;

	FTraitUID SelectedTraitUID;
	bool bStackContainsErrors = false;
	bool bShowTraitInterfaces = false;
};

}
