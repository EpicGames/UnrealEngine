// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraitHandlePin.h"

#include "EdGraph/EdGraphNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SGraphPanel.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "STraitHandlePin"

namespace UE::UAF
{

void STraitHandlePin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InPin);

	CachedImg_Pin_ConnectedHovered = FAppStyle::GetBrush(TEXT("Graph.PosePin.ConnectedHovered"));
	CachedImg_Pin_Connected = FAppStyle::GetBrush(TEXT("Graph.PosePin.Connected"));
	CachedImg_Pin_DisconnectedHovered = FAppStyle::GetBrush(TEXT("Graph.PosePin.DisconnectedHovered"));
	CachedImg_Pin_Disconnected = FAppStyle::GetBrush(TEXT("Graph.PosePin.Disconnected"));
}

const FSlateBrush* STraitHandlePin::GetPinIcon() const
{
	if (IsConnected())
	{
		return IsHovered() ? CachedImg_Pin_ConnectedHovered : CachedImg_Pin_Connected;
	}
	else
	{
		return IsHovered() ? CachedImg_Pin_DisconnectedHovered : CachedImg_Pin_Disconnected;
	}
}

}

#undef LOCTEXT_NAMESPACE