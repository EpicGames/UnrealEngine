// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SGraphPin.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UEdGraphPin;
struct FSlateBrush;

enum class EAnimGraphAttributeBlend;

namespace UE::UAF
{

class STraitHandlePin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(STraitHandlePin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

protected:
	//~ Begin SGraphPin Interface
	virtual const FSlateBrush* GetPinIcon() const override;
	//~ End SGraphPin Interface

	mutable const FSlateBrush* CachedImg_Pin_ConnectedHovered = nullptr;
	mutable const FSlateBrush* CachedImg_Pin_DisconnectedHovered = nullptr;
};

}