// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraitListView.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SAssetDropTarget.h"
#include "ScopedTransaction.h"
#include "WorkspaceSchema.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Input/SSearchBox.h"
#include "ObjectEditorUtils.h"
#include "TraitCore/TraitMode.h"
#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitInterfaceRegistry.h"
#include "TraitCore/TraitRegistry.h"
#include "Common/SCategoryTableRow.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "TraitListEditor"

namespace
{

static const FName NAME_Category("Category");
static const FName NAME_DefaultCategory("Default");
static const FName Column_Trait(TEXT("Traits"));
static const FName BaseCategoryName(TEXT("Base"));
static const FName AdditiveCategoryName(TEXT("Additive"));
static const FText BaseCategoryText(LOCTEXT("TraitBaseCategoryName", "Base"));
static const FText AdditiveCategoryText(LOCTEXT("AdditiveBaseCategoryName", "Additive"));

}

namespace UE::UAF::Editor
{

// An entry displayed in the parameters view
struct FTraitListEntry
{
	FTraitListEntry() = default;
	virtual ~FTraitListEntry() = default;

	virtual bool IsCategory() const
	{
		return false;
	}

	virtual bool IsTrait() const
	{
		return false;
	}

	virtual uint8 GetDepthLevel() const
	{
		return DepthLevel;
	}

	virtual bool IsRoot() const
	{
		return DepthLevel == 0;
	}

	virtual bool HasChildren() const
	{
		return false;
	}

	virtual void GenerateChildren(const FString& InFilterText, uint8 CurrentDepth = 0)
	{
	}

	virtual void GetChildren(TArray<TSharedRef<FTraitListEntry>>& OutChildren) const
	{
	}

	virtual void GetChildrenRecursive(TArray<TSharedRef<FTraitListEntry>>& OutChildren)
	{
	}

	virtual const FName& GetCategory() const
	{
		static const FName None = NAME_None;
		return None;
	}

	virtual const FText& GetCategoryText() const
	{
		return FText::GetEmpty();
	}

	virtual const FText& GetTraitNameText() const
	{
		return FText::GetEmpty();
	}

	virtual const TSharedPtr<FTraitDataEditorDef>& GetTraitData() const
	{
		static TSharedPtr<FTraitDataEditorDef> Empty;
		return Empty;
	}

	virtual FTraitUID GetTraitUID() const
	{
		return FTraitUID();
	}

	virtual ETraitMode GetTraitMode() const
	{
		return ETraitMode::Invalid;
	}

	uint8 DepthLevel = 0;
};

struct FTraitEntry : FTraitListEntry
{
	FTraitEntry() = default;

	explicit FTraitEntry(const TSharedPtr<FTraitDataEditorDef>& InTraitDataEditorDef)
		: TraitDataEditorDef(InTraitDataEditorDef)
	{}

	virtual bool IsTrait() const override
	{
		return true;
	}

	virtual FTraitUID GetTraitUID() const override
	{
		return TraitDataEditorDef->TraitUID;
	}

	virtual ETraitMode GetTraitMode() const override
	{
		return TraitDataEditorDef->TraitMode;
	}

	virtual const FText& GetTraitNameText() const override
	{
		return TraitDataEditorDef->TraitDisplayName;
	}

	virtual const TSharedPtr<FTraitDataEditorDef>& GetTraitData() const override
	{
		return TraitDataEditorDef;
	}

	TSharedPtr<FTraitDataEditorDef> TraitDataEditorDef;
};

struct FTraitListCategoryEntry : FTraitListEntry
{
	FTraitListCategoryEntry() = default;

	explicit FTraitListCategoryEntry(const FTraitCategoryData& InTraitCategoryData)
		: Category(InTraitCategoryData.Category)
		, CategoryText(InTraitCategoryData.CategoryText)
		, TraitList(InTraitCategoryData.TraitList)
	{
	}

	FTraitListCategoryEntry(FName InCategory, FText InCategoryText)
		: Category(InCategory)
		, CategoryText(InCategoryText)
	{
	}

	virtual bool IsCategory() const override
	{
		return true;
	}

	virtual const FName& GetCategory() const override
	{
		return Category;
	}

	virtual const FText& GetCategoryText() const override
	{
		return CategoryText;
	}

	virtual bool HasChildren() const override
	{
		return Children.Num() > 0;
	}

	virtual void GetChildren(TArray<TSharedRef<FTraitListEntry>>& OutChildren) const
	{
		OutChildren = Children;
	}

	void GetChildrenRecursive(TArray< TSharedRef<FTraitListEntry> >& OutChildren)
	{
		for (TSharedRef<FTraitListEntry>& Entry : Children)
		{
			OutChildren.Add(Entry);
			Entry->GetChildrenRecursive(OutChildren);
		}
	}

	virtual void GenerateChildren(const FString& InFilterText, uint8 CurrentDepth = 0) override
	{
		DepthLevel = CurrentDepth;

		const int32 NumChildren = Children.Num();
		for (int32 i = NumChildren - 1; i >= 0; --i)
		{
			TSharedRef<FTraitListEntry>& Entry = Children[i];
			if (Entry->IsCategory())
			{
				Entry->GenerateChildren(InFilterText, DepthLevel + 1);
			}
			else
			{
				Children.RemoveAt(i);
			}
		}

		if (TraitList.Num() > 0)
		{
			for (const TSharedPtr<FTraitDataEditorDef>& TraitDataDef : TraitList)
			{
				if (InFilterText.IsEmpty() 
					|| TraitDataDef->TraitDisplayName.ToString().Contains(InFilterText))
				{
					Children.Add(MakeShared<FTraitEntry>(TraitDataDef));
				}
			}
		}
	}

	void AddEntry(const FName& InCategory, const FText& InCategoryText, const TSharedPtr<FTraitDataEditorDef>& InTraitDataDef)
	{
		TSharedRef<FTraitListCategoryEntry> CategoryEntry = FindOrCreateSubCategory(InCategory, InCategoryText);
		CategoryEntry->Children.Add(MakeShared<FTraitEntry>(InTraitDataDef));
	}

	TSharedRef<FTraitListCategoryEntry> FindOrCreateSubCategory(const FName& InCategory, const FText& InCategoryText)
	{
		for (TSharedRef<FTraitListEntry>& Entry : Children)
		{
			if (Entry->GetCategory() == InCategory)
			{
				return StaticCastSharedRef<FTraitListCategoryEntry>(Entry);
			}
		}

		TSharedRef<FTraitListCategoryEntry> Entry = StaticCastSharedRef<FTraitListCategoryEntry>(Children.Add_GetRef(MakeShared<FTraitListCategoryEntry>()));
		Entry->Category = InCategory;
		Entry->CategoryText = InCategoryText;

		return Entry;
	}

	FName Category;
	FText CategoryText;
	TArray<TSharedPtr<FTraitDataEditorDef>> TraitList;
	TArray<TSharedRef<FTraitListCategoryEntry>> SubCategories;

	TArray<TSharedRef<FTraitListEntry>> Children;
};


void STraitListView::Construct(const FArguments& InArgs, TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData)
{
	check(InTraitEditorSharedData.IsValid());
	TraitEditorSharedData = InTraitEditorSharedData.ToSharedRef();

	//UICommandList = MakeShared<FUICommandList>();

	OnTraitClicked = InArgs._OnTraitClicked;
	OnGetSelectedTraitData = InArgs._OnGetSelectedTraitData;

	SAssignNew(TraitListFilterBox, SSearchBox)
	.OnTextChanged(this, &STraitListView::OnFilterTextChanged);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0)
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.VAlign(VAlign_Center)
					[
						TraitListFilterBox.ToSharedRef()
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0)
				.Padding(0.f, 4.f, 0.f, 0.f)
				[
					SAssignNew(EntriesList, STreeView<TSharedRef<FTraitListEntry>>)
					.TreeItemsSource(&FilteredEntries)
					.OnGenerateRow(this, &STraitListView::HandleGenerateRow)
					.OnGetChildren(this, &STraitListView::HandleGetChildren)
					.OnItemScrolledIntoView(this, &STraitListView::HandleItemScrolledIntoView)
					.OnSelectionChanged(this, &STraitListView::HandleSelectionChanged)
					.OnExpansionChanged_Lambda([this](TSharedRef<FTraitListEntry> Entry, bool bExpanded)
						{
							if (FilterText.IsEmpty())
							{
								StoreExpansionState();
							}
						})
					.HeaderRow(
						SNew(SHeaderRow)
						+SHeaderRow::Column(Column_Trait)
						.DefaultLabel(LOCTEXT("TraitListColumnHeader", "Traits"))
						.HAlignHeader(HAlign_Center)
						.ToolTipText(LOCTEXT("TraitListColumnHeaderTooltip", "The list of available Traits"))
						.FillWidth(10.0f)
					)
				]
			]
		]
	];

	FilterText = FText();
}

void STraitListView::RefreshList()
{
	GenerateTraitList();

	StoreExpansionState();

	RefreshEntries();

	TArray<TSharedRef<FTraitListEntry>> AllEntries;
	GetAllEntries(AllEntries);
	ExpandAllCategories(AllEntries);

	FilterText = FText();
	RefreshFilter();
}

FReply STraitListView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void STraitListView::OnFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText;
	RefreshFilter();
}

bool STraitListView::HasValidEditorData() const
{
	return TraitEditorSharedData.IsValid() && TraitEditorSharedData->EdGraphNodeWeak.IsValid();
}

TSharedPtr<FTraitDataEditorDef> STraitListView::GetSelectedTraitData() const
{
	TSharedPtr<FTraitDataEditorDef> StackSelectedTrait;
	if (OnGetSelectedTraitData.IsBound())
	{
		StackSelectedTrait = OnGetSelectedTraitData.Execute().Pin();
	}

	return StackSelectedTrait;
}

void STraitListView::StoreExpansionState()
{
	if (!bIgnoreExpansion									// Do not store state while expanding all items or restoring expansion state
		&& HasValidEditorData()								// Do not store state if no data is set (or has been cleared)
		&& EntriesList->GetNumItemsBeingObserved() > 0)		// Do not store state if the entry list is empty (set after being previously cleared)
	{
		OldExpansionState.Reset();
		EntriesList->GetExpandedItems(OldExpansionState);
	}
}

template<typename ItemType, typename ComparisonType>
static void RestoreExpansionStateT(TSharedPtr< STreeView<ItemType> > InTree, const TArray<ItemType>& ItemSource, const TSet<ItemType>& OldExpansionState, ComparisonType ComparisonFunction)
{
	check(InTree.IsValid());

	// Iterate over new tree items
	for (int32 ItemIdx = 0; ItemIdx < ItemSource.Num(); ItemIdx++)
	{
		ItemType NewItem = ItemSource[ItemIdx];

		bool bFound = false;
		// Look through old expansion state
		for (typename TSet<ItemType>::TConstIterator OldExpansionIter(OldExpansionState); OldExpansionIter; ++OldExpansionIter)
		{
			const ItemType OldItem = *OldExpansionIter;
			// See if this matches this new item
			if (ComparisonFunction(OldItem, NewItem))
			{
				// It does, so expand it
				InTree->SetItemExpansion(NewItem, true);
				bFound = true;
			}
		}
		if (!bFound)
		{
			InTree->SetItemExpansion(NewItem, false);
		}
	}
}

void STraitListView::RestoreExpansionState(TArray<TSharedRef<FTraitListEntry>>& AllEntries)
{
	if (!OldExpansionState.IsEmpty() && AllEntries.Num() > 0)
	{
		TGuardValue<bool> GuardThreadId(bIgnoreExpansion, true);

		// Restore the expanded items
		RestoreExpansionStateT<TSharedRef<FTraitListEntry>>(EntriesList, AllEntries, OldExpansionState, [](TSharedRef<FTraitListEntry> A, TSharedRef<FTraitListEntry> B) -> bool
			{
				if (!A->GetCategory().IsNone() && A->GetCategory() == B->GetCategory())
				{
					return true;
				}

				return false;
			});
	}
}

void STraitListView::GetAllEntries(TArray<TSharedRef<FTraitListEntry>>& AllEntries) const
{
	for (const TSharedRef<FTraitListEntry>& Entry : Categories)
	{
		AllEntries.Add(Entry);
		Entry->GetChildrenRecursive(AllEntries);
	}
}

void STraitListView::ExpandAllCategories(const TArray<TSharedRef<FTraitListEntry>>& AllEntries)
{
	TGuardValue<bool> GuardThreadId(bIgnoreExpansion, true);

	for (const TSharedRef<FTraitListEntry>& Entry : AllEntries)
	{
		if (Entry->IsCategory())
		{
			EntriesList->SetItemExpansion(Entry, true);
		}
	}
}

void STraitListView::RefreshEntries()
{
	Categories.Reset(2);

	if (HasValidEditorData())
	{
		CreateTraitCategories(BaseCategoryName, BaseCategoryText, BaseTraitCategories);
		CreateTraitCategories(AdditiveCategoryName, AdditiveCategoryText, AdditiveTraitCategories);
	}
}

void STraitListView::CreateTraitCategories(const FName& InCategoryName, const FText& InCategoryText, TMap<FName, FTraitCategoryData>& InCategoriesMap)
{
	TSharedRef<FTraitListCategoryEntry> Category = StaticCastSharedRef<FTraitListCategoryEntry>(Categories.Add_GetRef(MakeShared<FTraitListCategoryEntry>(InCategoryName, InCategoryText)));

	const int32 NumBaseTraitCategories = InCategoriesMap.Num();
	for (const TPair<FName, FTraitCategoryData>& CategoryPair : InCategoriesMap)
	{
		const FName& CategoryName = CategoryPair.Key;
		const FTraitCategoryData& TraitCategoryData = CategoryPair.Value;
		check(CategoryName == TraitCategoryData.Category);

		if (CategoryName == NAME_DefaultCategory && NumBaseTraitCategories == 1)
		{
			Category->TraitList = TraitCategoryData.TraitList;
		}
		else
		{
			TSharedRef<FTraitListCategoryEntry> CategoryEntry = Category->FindOrCreateSubCategory(CategoryName, TraitCategoryData.CategoryText);
			CategoryEntry->TraitList = TraitCategoryData.TraitList;
		}
	}
}

void STraitListView::RefreshFilter()
{
	FilteredEntries = Categories;

	const FString FilterTextAsString = FilterText.ToString();
	for (TSharedRef<FTraitListEntry>& Category : Categories)
	{
		Category->GenerateChildren(FilterTextAsString);
	}

	TArray<TSharedRef<FTraitListEntry>> AllEntries;
	GetAllEntries(AllEntries);
	ExpandAllCategories(AllEntries);

	if (FilterText.IsEmpty())
	{
		RestoreExpansionState(AllEntries);
	}

	EntriesList->RequestListRefresh();
}

static bool IsTraitAvailable(const TWeakPtr<TArray<TSharedPtr<FTraitDataEditorDef>>> CurrentTraitsDataWeak, const FTraitUID TraitUID, const ETraitMode TraitMode, const TSharedPtr<FTraitDataEditorDef>& InStackSelectedTrait)
{
	// Disable Buttons of Traits that already exist in the list
	if (TSharedPtr<TArray<TSharedPtr<FTraitDataEditorDef>>> CurrentTraitsDataShared = CurrentTraitsDataWeak.Pin())
	{
		// Only a base Trait is allowed if the list is empty
		if (CurrentTraitsDataShared->IsEmpty())
		{
			return TraitMode == ETraitMode::Base;
		}

		// If a Trait is selected, only allow same type traits
		if (InStackSelectedTrait.IsValid())
		{
			if (InStackSelectedTrait->TraitMode != TraitMode)
			{
				return false;
			}
		}

		// Only additive Traits are allowed if the list is not empty, unless the first element is invalid
		if (TraitMode == ETraitMode::Base)
		{
			// Only one base trait allowed
			if (CurrentTraitsDataShared->Top()->TraitUID != FTraitUID() && !InStackSelectedTrait.IsValid())
			{
				for (const TSharedPtr<FTraitDataEditorDef>& Trait : *CurrentTraitsDataShared.Get())
				{
					if (Trait->TraitMode == ETraitMode::Base && Trait->TraitUID != FTraitUID())
					{
						return false;
					}
				}
			}
		}

		// No duplicated Traits allowed in the list
		for (const TSharedPtr<FTraitDataEditorDef>& Trait : *CurrentTraitsDataShared.Get())
		{
			if (Trait->TraitUID == TraitUID && !Trait->bMultipleInstanceSupport)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

TSharedRef<ITableRow> STraitListView::HandleGenerateRow(TSharedRef<FTraitListEntry> InEntry, const TSharedRef<STableViewBase>& InOwnerTable)
{
	if (InEntry->IsCategory())
	{
		return 
			SNew(SCategoryHeaderTableRow< TSharedPtr<FTraitListEntry> >, InOwnerTable)
			.Padding(InEntry->IsRoot() ? FMargin(2.0f, 2.0f, 2.0f, 2.0f) : FMargin(InEntry->GetDepthLevel() * 10.0f, 2.0f, 2.0f, 2.0f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				.Padding(FMargin(2, 2))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
						.Text(InEntry->GetCategoryText())
						.HighlightText_Lambda([this]() { return FilterText; })
					]
				]
			];
	}
	else
	{
		TWeakPtr<FTraitListEntry> EntryWeak = InEntry.ToWeakPtr();
		const TSharedPtr<FTraitEditorSharedData>& TraitEditorSharedDataLocal = TraitEditorSharedData;
		const TSharedPtr<FTraitDataEditorDef>& TraitDataShared = InEntry->GetTraitData();
		const FTraitUID TraitUID = InEntry->GetTraitUID();
		const ETraitMode TraitMode = InEntry->GetTraitMode();

		return
			SNew(STableRow<TSharedRef<FTraitListEntry>>, InOwnerTable)
			.Padding(FMargin(0.f, 2.f))
			.ShowSelection(false)
			.IsEnabled_Lambda([this, TraitUID, TraitMode]()->bool
			{
				return IsTraitAvailable(TraitEditorSharedData->CurrentTraitsDataShared, TraitUID, TraitMode, GetSelectedTraitData());
			})
			.OnDragDetected_Lambda([TraitDataShared](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
			{
				if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
				{
					if (TraitDataShared.IsValid())
					{
						TSharedRef<FTraitListDragDropOp> DragDropOp = FTraitListDragDropOp::New(TraitDataShared.ToWeakPtr());
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}
				}
				return FReply::Unhandled();
			})
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTraitListEntry> TargetItem)
			{
				const TOptional<EItemDropZone> InvalidDropZone;
				return InvalidDropZone;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f))
				.FillWidth(1.0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ContentPadding(FMargin(0.f, 0.f))
					.ButtonColorAndOpacity_Lambda([this, EntryWeak]()
					{
						if (TSharedPtr<FTraitListEntry> Entry = EntryWeak.Pin())
						{
							return FTraitEditorUtils::GetTraitBackroundDisplayColor(Entry->GetTraitMode(), /*bIsSelected*/ false, /*bIsHovered*/ false);
						}
						return FSlateColor(FColor::Red);
					})
					.ClickMethod(EButtonClickMethod::PreciseClick)
					.OnClicked_Lambda([this, TraitUID]()->FReply
						{
							if (OnTraitClicked.IsBound())
							{
								return OnTraitClicked.Execute(TraitUID);
							}

							return FReply::Unhandled();
						})
					.Content()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.White"))
						.BorderBackgroundColor_Lambda([EntryWeak, this]()
						{
							if (TSharedPtr<FTraitListEntry> Entry = EntryWeak.Pin())
							{
								return FTraitEditorUtils::GetTraitBackroundDisplayColor(Entry->GetTraitMode(), /*bIsSelected*/ false, /*bIsHovered*/ false);
							}
							return FSlateColor(FColor::Red);
						})
						[
							SNew(SVerticalBox)

							// --- Trait Required Interfaces ---
							+ SVerticalBox::Slot()
							.MinHeight(TraitEditorSharedDataLocal->bShowTraitInterfaces ? 23.0f : 0.0f)
							.AutoHeight()
							[
								FTraitEditorUtils::GetInterfaceListWidget(FTraitEditorUtils::EInterfaceDisplayType::ListRequired, TraitDataShared, TraitEditorSharedDataLocal)
							]

							// --- Trait Main Button ---
							+ SVerticalBox::Slot()
							[
								SNew(SBox)
								.MinDesiredHeight(25.0f)
								.VAlign(VAlign_Center)
								[
									SNew(SHorizontalBox)

									+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Left)
									[
										SNew(SBox)
										.VAlign(VAlign_Top)
										.HAlign(HAlign_Right)
										[
											SNew(SImage)
											.Image(FAppStyle::Get().GetBrush("Icons.Info.Small"))
											.ToolTipText_Lambda([EntryWeak, this]()->FText
											{
												if (TSharedPtr<FTraitListEntry> Entry = EntryWeak.Pin())
												{
													if (const TSharedPtr<FTraitDataEditorDef>& TraitData = Entry->GetTraitData())
													{
														// TODO zzz : Create a better looking Tooltip that is not just text based
														TStringBuilder<1024> TraitInfoString;

														TraitInfoString.Append(TraitData->TraitDisplayName.ToString())
															.Append(TEXT("\n\n"));
													
														if (TraitData->ImplementedInterfaces.Num() > 0)
														{
															TraitInfoString.Append(LOCTEXT("TraitInfoImplementedInterfaces", "Implements :").ToString())
																.AppendChar(TEXT('\n'));

															for (const auto& ImplementedInterfaceUID : TraitData->ImplementedInterfaces)
															{
																if (const ITraitInterface* ImplementedInterface = FTraitInterfaceRegistry::Get().Find(ImplementedInterfaceUID))
																{
																	TraitInfoString.Append(TEXT("- "))
																		.Append(ImplementedInterface->GetDisplayName().ToString())
																		.AppendChar(TEXT('\n'));
																}
															}
														}

														if (TraitData->RequiredInterfaces.Num() > 0)
														{
															TraitInfoString.AppendChar(TEXT('\n'));
															TraitInfoString.Append(LOCTEXT("TraitInfoRequiredInterfaces", "Requires :").ToString())
															.AppendChar(TEXT('\n'));

															for (const auto& RequiredInterfaceUID : TraitData->RequiredInterfaces)
															{
																if (const ITraitInterface* RequiredInterface = FTraitInterfaceRegistry::Get().Find(RequiredInterfaceUID))
																{
																	TraitInfoString.Append(TEXT("- "))
																		.Append(RequiredInterface->GetDisplayName().ToString())
																		.AppendChar(TEXT('\n'));
																}
															}
														}

														return FText::FromString(TraitInfoString.ToString());
													}
												}
												return FText();
											})
										]
									]

									+ SHorizontalBox::Slot()
									.HAlign(HAlign_Center)
									.FillWidth(1.f)
									[
										SNew(STextBlock)
										.Text(InEntry->GetTraitNameText())
										.HighlightText_Lambda([this]() { return FilterText; })
										.Justification(ETextJustify::Center)
										.ColorAndOpacity_Lambda([EntryWeak]()
										{
											if (TSharedPtr<FTraitListEntry> Entry = EntryWeak.Pin())
											{
												return FTraitEditorUtils::GetTraitTextDisplayColor(Entry->GetTraitMode());
											}

											return FSlateColor(FColor::Red);
										})
									]
								]
							]

							// --- Trait Implemented Interfaces ---
							+ SVerticalBox::Slot()
							.MinHeight(TraitEditorSharedDataLocal->bShowTraitInterfaces ? 23.0f : 0.0f)
							.AutoHeight()
							[
								FTraitEditorUtils::GetInterfaceListWidget(FTraitEditorUtils::EInterfaceDisplayType::ListImplemented, TraitDataShared, TraitEditorSharedDataLocal)
							]

						]
					]
				]
			];
	}
}

void STraitListView::HandleGetChildren(TSharedRef<FTraitListEntry> InEntry, TArray<TSharedRef<FTraitListEntry>>& OutChildren)
{
	if (InEntry->IsCategory())
	{
		InEntry->GetChildren(OutChildren);
	}
}

void STraitListView::HandleItemScrolledIntoView(TSharedRef<FTraitListEntry> InEntry, const TSharedPtr<ITableRow>& InWidget)
{
}

void STraitListView::HandleSelectionChanged(TSharedPtr<FTraitListEntry> InEntry, ESelectInfo::Type InSelectionType)
{
}

void STraitListView::GenerateTraitList()
{
	BaseTraitCategories.Reset();
	AdditiveTraitCategories.Reset();

	if (!HasValidEditorData())
	{
		return;
	}

	FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

	const TArray<const FTrait*> Traits = TraitRegistry.GetTraits();
	for (const FTrait* Trait : Traits)
	{
		if (Trait->IsHidden() && !TraitEditorSharedData->bAdvancedView)
		{
			continue;
		}

		FName Category = NAME_None;
		FText CategoryText;

		if (const FString* CategoryMetaData = Trait->GetTraitSharedDataStruct()->FindMetaData(NAME_Category))
		{
			Category = **CategoryMetaData;
			CategoryText = FObjectEditorUtils::GetCategoryText(Trait->GetTraitSharedDataStruct());
		}
		else
		{
			Category = NAME_DefaultCategory;
			CategoryText = LOCTEXT("DefaultTraitCategory", "Default");
		}

		static_assert((uint8)ETraitMode::Num == 2);
		TMap<FName, FTraitCategoryData>& TraitModeCategories = (Trait->GetTraitMode() == ETraitMode::Base) ? BaseTraitCategories : AdditiveTraitCategories;
		FTraitCategoryData* TraitCategoryData = TraitModeCategories.Find(Category);
		if (TraitCategoryData == nullptr)
		{
			TraitCategoryData = &TraitModeCategories.Add(Category, FTraitCategoryData(Category, CategoryText));
		}

		UScriptStruct* TraitSharedDataStruct = Trait->GetTraitSharedDataStruct();
		
		const FText TraitDisplayName = (TraitSharedDataStruct != nullptr) ? TraitSharedDataStruct->GetDisplayNameText() : FText::FromString(Trait->GetTraitName());
		
		TSharedPtr<FTraitDataEditorDef> TraitData = MakeShared<FTraitDataEditorDef>(*Trait, TraitDisplayName); // TODO zzz : Why TraitName is not an FName as in the Decorator ???
		FTraitEditorUtils::GenerateStackInterfacesUsedIndexes(TraitData, TraitEditorSharedData);

		TraitCategoryData->TraitList.Add(TraitData); 
	}
}

} // end namespace UE::Workspace

#undef LOCTEXT_NAMESPACE
