// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/ITableRow.h"
#include "Graph/TraitEditorDefs.h"

class UAnimNextEdGraphNode;

namespace UE::Workspace
{
class IWorkspaceEditor;
}


namespace UE::UAF::Editor
{

struct FTraitListEntry;
struct FTraitListCategoryEntry;

class STraitListView : public SCompoundWidget
{
public:
	/** Used to notify the Trait Editor a Trait has been clicked on the available list */
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnTraitClicked, const FTraitUID);

	/** Used to obtain the currently selected Trait in the Stack */
	DECLARE_DELEGATE_RetVal(TWeakPtr<FTraitDataEditorDef>, FOnGetSelectedTraitData);


	SLATE_BEGIN_ARGS(STraitListView) {}

	SLATE_EVENT(FOnTraitClicked, OnTraitClicked)

	SLATE_EVENT(FOnGetSelectedTraitData, OnGetSelectedTraitData)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData);

	void RefreshList();

private:
	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	//void HandleWorkspaceModified(UWorkspace* InWorkspace);

	void OnFilterTextChanged(const FText& InFilterText);

	TSharedPtr<FTraitEditorSharedData> TraitEditorSharedData;

	TMap<FName, FTraitCategoryData> BaseTraitCategories;
	TMap<FName, FTraitCategoryData> AdditiveTraitCategories;

	TSharedPtr<FUICommandList> UICommandList;

	TSharedPtr< SSearchBox > TraitListFilterBox;
	TSharedPtr<STreeView<TSharedRef<FTraitListEntry>>> EntriesList;
	TArray<TSharedRef<FTraitListEntry>> Categories;

	FText FilterText;
	TArray<TSharedRef<FTraitListEntry>> FilteredEntries;

	TSet< TSharedRef<FTraitListEntry> > OldExpansionState;

	FOnTraitClicked OnTraitClicked;
	FOnGetSelectedTraitData OnGetSelectedTraitData;

	bool bIgnoreExpansion = false;

	// --- ---
	bool HasValidEditorData() const;
	TSharedPtr<FTraitDataEditorDef> GetSelectedTraitData() const;

	void StoreExpansionState();
	void RestoreExpansionState(TArray<TSharedRef<FTraitListEntry>>& AllEntries);
	void GetAllEntries(TArray<TSharedRef<FTraitListEntry>>& AllEntries) const;
	void ExpandAllCategories(const TArray<TSharedRef<FTraitListEntry>>& AllEntries);
	void RefreshEntries();
	void CreateTraitCategories(const FName& BaseCategoryName, const FText& BaseCategoryText, TMap<FName, FTraitCategoryData>& CategoriesMap);
	void RefreshFilter();

	TSharedRef<ITableRow> HandleGenerateRow(TSharedRef<FTraitListEntry> InEntry, const TSharedRef<STableViewBase>& InOwnerTable);
	void HandleGetChildren(TSharedRef<FTraitListEntry> InEntry, TArray<TSharedRef<FTraitListEntry>>& OutChildren);
	void HandleItemScrolledIntoView(TSharedRef<FTraitListEntry> InEntry, const TSharedPtr<ITableRow>& InWidget);
	void HandleSelectionChanged(TSharedPtr<FTraitListEntry> InEntry, ESelectInfo::Type InSelectionType);

	void GenerateTraitList();
};

};
