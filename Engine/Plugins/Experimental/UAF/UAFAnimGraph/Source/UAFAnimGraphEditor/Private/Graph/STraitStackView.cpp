// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraitStackView.h"
#include "ContentBrowserModule.h"
#include "Framework/Commands/GenericCommands.h"
#include "AnimNextEdGraphNode.h"
#include "IContentBrowserSingleton.h"
#include "RigVMModel/RigVMNode.h"
#include "SAssetDropTarget.h"
#include "ScopedTransaction.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "Widgets/Views/SListView.h"
#include "WorkspaceSchema.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Layout/Margin.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Widgets/Input/SButton.h"
#include "RigVMModel/RigVMController.h"
#include "AnimNextTraitStackUnitNode.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "RigVMModel/RigVMClient.h"
#include "AnimGraphUncookedOnlyUtils.h"

#define LOCTEXT_NAMESPACE "TraitListEditor"

namespace UE::UAF::Editor
{

// --- FTraitStackViewEntry ---

struct FTraitStackViewEntry
{
	FTraitStackViewEntry() = default;

	explicit FTraitStackViewEntry(const TSharedPtr<FTraitDataEditorDef>& InTraitDataEditorDef)
		: TraitData(InTraitDataEditorDef)
	{
	}

	TSharedPtr<FTraitDataEditorDef> TraitData;
};

// --- FTraitStackDragDropOp ---
class FTraitStackDragDropOp : public FTraitListDragDropBase
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTraitStackDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FTraitStackDragDropOp> New(TWeakPtr<FTraitDataEditorDef> InDraggedTraitDataWeak)
	{
		TSharedRef<FTraitStackDragDropOp> Operation = MakeShared<FTraitStackDragDropOp>();
		Operation->DraggedTraitDataWeak = InDraggedTraitDataWeak;
		Operation->Construct();
		return Operation;
	}
};


// --- STraitStackView ---

STraitStackView::STraitStackView()
	: EntriesList(MakeShared<SListView<TSharedRef<FTraitStackViewEntry>>>())
{
}

void STraitStackView::Construct(const FArguments& InArgs, TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData)
{
	OnTraitDeleteRequest = InArgs._OnTraitDeleteRequest;
	OnStatckTraitSelectionChanged = InArgs._OnStatckTraitSelectionChanged;
	OnStackTraitDragAccepted = InArgs._OnStackTraitDragAccepted;

	TraitEditorSharedData = InTraitEditorSharedData;

	UICommandList = MakeShared<FUICommandList>();

	UICommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &STraitStackView::HandleDelete),
		FCanExecuteAction::CreateSP(this, &STraitStackView::HasValidSelection));

	const TSharedPtr<FTraitEditorSharedData>& TraitEditorSharedDataLocal = TraitEditorSharedData;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0)
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TraitStackNodeName", "Node Name :"))
					]
					+ SHorizontalBox::Slot()
					.Padding(40.f, 0.f, 0.f, 0.f)
					.FillWidth(1.f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Black"))
						.Padding(FMargin(1.0f, 1.0f))
						[
							SNew(SInlineEditableTextBlock)
							.Style(FAppStyle::Get(), "Graph.Node.NodeTitleInlineEditableText")
							.Text_Lambda([this] { return TraitEditorSharedData->EdGraphNodeWeak.IsValid() ? TraitEditorSharedData->EdGraphNodeWeak->GetNodeTitle(ENodeTitleType::EditableTitle) : FText::GetEmpty(); })
							.IsReadOnly_Lambda([TraitEditorSharedDataLocal]()->bool
							{
								bool bIsReadOnly = true;
								if (TraitEditorSharedDataLocal.IsValid())
								{
									if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(TraitEditorSharedDataLocal->EdGraphNodeWeak.Get()))
									{
										bIsReadOnly = EdGraphNode->IsDeprecated() || EdGraphNode->IsOutDated();
									}
								}
								return bIsReadOnly;
							})
							.OnTextCommitted_Lambda([TraitEditorSharedDataLocal](const FText& InNewText, ETextCommit::Type InCommitType)
							{
								if(InCommitType == ETextCommit::OnEnter)
								{
									if (TraitEditorSharedDataLocal.IsValid())
									{
										if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(TraitEditorSharedDataLocal->EdGraphNodeWeak.Get()))
										{
											if (URigVMNode* ModelNode = EdGraphNode->GetModelNode())
											{
												if (UAnimNextTraitStackUnitNode* UnitNode = Cast<UAnimNextTraitStackUnitNode>(ModelNode))
												{
													FScopedTransaction Transaction(LOCTEXT("SetNodeTitle", "Set Node title"));

													if (UScriptStruct* Struct = UnitNode->GetScriptStruct())
													{
														if (URigVMController* Controller = EdGraphNode->GetController())
														{
															Controller->SetNodeTitle(ModelNode, InNewText.ToString(), true, false, true);
														}
													}
												}
											}
										}
									}
								}
							})
						]
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0)
				[
					SNew(SBorder)
					.Visibility(EVisibility::Visible)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.0)
						[
							SAssignNew(EntriesList, SListView<TSharedRef<FTraitStackViewEntry>>)
							.ListItemsSource(&Entries)
							.ItemHeight_Lambda([TraitEditorSharedDataLocal]()
								{
									constexpr const float ButtonSize = 20.f;
									constexpr const float ButtonSizeWithInterfaces = 40.f;

									if (TraitEditorSharedDataLocal.IsValid() && TraitEditorSharedDataLocal->bShowTraitInterfaces)
									{
										return ButtonSizeWithInterfaces; 
									}
									return ButtonSize;
								})
							.OnGenerateRow(this, &STraitStackView::HandleGenerateRow)
							.SelectionMode(ESelectionMode::SingleToggle)
							.OnSelectionChanged_Lambda([this](TSharedPtr<FTraitStackViewEntry> InEntry, ESelectInfo::Type InSelectionType)
							{
								if (InEntry.IsValid())
								{
									SelectedTraitData = InEntry->TraitData;
								}
								else
								{
									SelectedTraitData.Reset();
								}

								if (OnStatckTraitSelectionChanged.IsBound())
								{
									OnStatckTraitSelectionChanged.Execute(GetSelectedTraitUID());
								}
							})
						]
					]
				]
			]
		]
	];
}

void STraitStackView::RefreshList()
{
	if (TraitEditorSharedData.IsValid() && TraitEditorSharedData->CurrentTraitsDataShared.IsValid())
	{
		const TArray<TSharedPtr<FTraitDataEditorDef>>& CurrentTraitsData = *(TraitEditorSharedData->CurrentTraitsDataShared.Get());
		
		Entries.Reset(CurrentTraitsData.Num());

		for (const TSharedPtr<FTraitDataEditorDef>& TraitData : CurrentTraitsData)
		{
			Entries.Add(MakeShared<FTraitStackViewEntry>(TraitData));
		}
	}
	else
	{
		Entries.Reset();
	}

	EntriesList->RebuildList();
}

FReply STraitStackView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void STraitStackView::HandleDelete()
{
	const FTraitUID SelectedTraitUID = GetSelectedTraitUID();
	if (SelectedTraitUID != FTraitUID())
	{
		ExecuteDelete(SelectedTraitUID);
	}
}

bool STraitStackView::HasValidSelection() const 
{
	const FTraitUID SelectedTraitUID = GetSelectedTraitUID();

	if (SelectedTraitUID != FTraitUID())
	{
		return true;
	}

	return false;
}

// Helper to get direct access to the Paint delegate
class STraitStackTableRow : public STableRow<TSharedRef<FTraitStackViewEntry>>
{
public:
	/** Optional delegate for painting drop indicators */
	FOnPaintDropIndicator GetOnPaintDropIndicatorDelegate() { return PaintDropIndicatorEvent; }
};

TSharedRef<ITableRow> STraitStackView::HandleGenerateRow(TSharedRef<FTraitStackViewEntry> InEntry, const TSharedRef<STableViewBase>& InOwnerTable)
{
	const TWeakPtr<FTraitStackViewEntry> InEntryWeak = InEntry.ToWeakPtr();
	const TSharedPtr<FTraitEditorSharedData> TraitEditorSharedDataLocal = TraitEditorSharedData;
	const TSharedPtr<FTraitDataEditorDef>& TraitDataShared = InEntry->TraitData;

	TSharedPtr<STraitStackTableRow> Row;

	SAssignNew(Row, STraitStackTableRow, InOwnerTable)
		.Padding(FMargin(4.f, 2.f))
		.OnDragDetected_Lambda([InEntryWeak](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
			{
				if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
				{
					if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
					{
						if (Entry->TraitData.IsValid())
						{
							//TSharedRef<FTraitStackDragDropOp> DragDropOp = FTraitStackDragDropOp::New(Entry);
							TSharedRef<FTraitStackDragDropOp> DragDropOp = FTraitStackDragDropOp::New(Entry->TraitData.ToWeakPtr());
							return FReply::Handled().BeginDragDrop(DragDropOp);
						}
					}
				}
				return FReply::Unhandled();
			})
		.OnCanAcceptDrop_Lambda([TraitEditorSharedDataLocal](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTraitStackViewEntry> TargetItem)
			{
				const TOptional<EItemDropZone> InvalidDropZone;
				TOptional<EItemDropZone> ReturnDropZone = InvalidDropZone;

				if (!TargetItem.IsValid())
				{
					return ReturnDropZone;
				}

				// --- Drops from the Trait List ---
				if (TSharedPtr<FTraitListDragDropOp> TraitListDragDropOp = DragDropEvent.GetOperationAs<FTraitListDragDropOp>())
				{
					if (TSharedPtr<FTraitDataEditorDef> DraggedEntryTraitData = TraitListDragDropOp->GetDraggedTraitData().Pin(); DraggedEntryTraitData.IsValid())
					{
						const ETraitMode DraggedTraitMode = DraggedEntryTraitData->TraitMode;
						const ETraitMode TargetTraitMode = TargetItem->TraitData->TraitMode;

						if (TargetTraitMode == ETraitMode::Base)
						{
							if (DraggedTraitMode == ETraitMode::Additive)
							{
								ReturnDropZone = EItemDropZone::BelowItem;	// Force BelowItem for additive traits
							}
							else
							{
								ReturnDropZone = EItemDropZone::OntoItem; // Force OntoItem independently of the zone for base traits
							}
						}
						else
						{
							if (DraggedTraitMode == ETraitMode::Base)
							{
								ReturnDropZone = InvalidDropZone;
							}
							else
							{
								ReturnDropZone = DropZone == EItemDropZone::AboveItem ? EItemDropZone::OntoItem : DropZone; // For additives disalow AboveItem
							}
						}
					}
				}
				// --- Drops from the Trait Stack itself (rearrange items in the stack) ---
				else if (TSharedPtr<FTraitStackDragDropOp> TraitStackDragDropOp = DragDropEvent.GetOperationAs<FTraitStackDragDropOp>())
				{
					if (DropZone == EItemDropZone::OntoItem)
					{
						return InvalidDropZone;
					}
					
					if (TSharedPtr<FTraitDataEditorDef> DraggedEntryTraitData = TraitStackDragDropOp->GetDraggedTraitData().Pin(); DraggedEntryTraitData.IsValid())
					{
						// Base can not be dropped anywhere
						if (DraggedEntryTraitData->TraitMode == ETraitMode::Base)
						{
							return InvalidDropZone;
						}

						// An additive can only be dropped below a base
						if (TargetItem->TraitData->TraitMode == ETraitMode::Base && DropZone != EItemDropZone::BelowItem)
						{
							return InvalidDropZone;
						}

						int32 TargetTraitIndex = INDEX_NONE;
						FTraitEditorUtils::FindTraitInCurrentStackData(TargetItem->TraitData->TraitUID, TraitEditorSharedDataLocal->CurrentTraitsDataShared, &TargetTraitIndex);
						int32 DraggedTraitIndex = INDEX_NONE;
						FTraitEditorUtils::FindTraitInCurrentStackData(DraggedEntryTraitData->TraitUID, TraitEditorSharedDataLocal->CurrentTraitsDataShared, &DraggedTraitIndex);

						if (TargetTraitIndex == INDEX_NONE || DraggedTraitIndex == INDEX_NONE)
						{
							return InvalidDropZone;
						}

						const int32 IndexDiff = TargetTraitIndex - DraggedTraitIndex;
						const int32 AbsIndexDiff = FMath::Abs(IndexDiff);
						if (AbsIndexDiff < 1 // Can not drop on self
							|| DropZone == EItemDropZone::AboveItem // Only BelowItem is allowed
							|| (AbsIndexDiff == 1 && IndexDiff < 0))// And on neighbors only if it is the down neighbor
						{
							return InvalidDropZone;
						}

						ReturnDropZone = DropZone;
					}
				}

				return ReturnDropZone;
			})
		.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTraitStackViewEntry> TargetItem)->FReply
			{
				if(TargetItem.IsValid() && TargetItem->TraitData.IsValid())
				{
					TSharedPtr<FTraitDataEditorDef> DraggedEntryTraitData;

					if (TSharedPtr<FTraitStackDragDropOp> TraitStackDragDropOp = DragDropEvent.GetOperationAs<FTraitStackDragDropOp>())
					{
						DraggedEntryTraitData = TraitStackDragDropOp->GetDraggedTraitData().Pin();
					}
					else if (TSharedPtr<FTraitListDragDropOp> TraitListDragDropOp = DragDropEvent.GetOperationAs<FTraitListDragDropOp>())
					{
						DraggedEntryTraitData = TraitListDragDropOp->GetDraggedTraitData().Pin();
					}

					if (DraggedEntryTraitData.IsValid())
					{
						if (OnStackTraitDragAccepted.IsBound())
						{
							return OnStackTraitDragAccepted.Execute(DraggedEntryTraitData->TraitUID, TargetItem->TraitData->TraitUID, DropZone);
						}
					}
				}

				return FReply::Unhandled();
			})
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.White"))
				.BorderBackgroundColor_Lambda([InEntryWeak, this]()
				{
					if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
					{
						const FTraitUID SelectedTraitUID = GetSelectedTraitUID();
						const bool bIsSelected = SelectedTraitUID != FTraitUID() && Entry->TraitData.IsValid() && Entry->TraitData->TraitUID == SelectedTraitUID;

						return FTraitEditorUtils::GetTraitBackroundDisplayColor(Entry->TraitData->TraitMode, bIsSelected);
					}
					return FSlateColor(FColor::Red);
				})
				.Padding(FMargin(1.0f, 1.0f))
				[
					SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							FTraitEditorUtils::GetInterfaceListWidget(FTraitEditorUtils::EInterfaceDisplayType::StackRequired, TraitDataShared, TraitEditorSharedDataLocal)
						]

						+ SVerticalBox::Slot()
						[
							SNew(SBox)
							//.Padding(30.f, 10.f)
							.MinDesiredHeight(30.0f)
							.VAlign(VAlign_Center)
							[
								GetStackListItemWidget(InEntryWeak, TraitEditorSharedDataLocal)
							]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							FTraitEditorUtils::GetInterfaceListWidget(FTraitEditorUtils::EInterfaceDisplayType::StackImplemented, TraitDataShared, TraitEditorSharedDataLocal)
						]
				]
			]
		];

		const TWeakPtr<STraitStackTableRow> RowWeak = Row.ToWeakPtr();
		
		// Use direct access to the paint delegate, in order to be able to pass the Row to the lambda
		Row->GetOnPaintDropIndicatorDelegate().BindLambda([RowWeak](EItemDropZone InItemDropZone, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled)
			{
				if (const TSharedPtr<STraitStackTableRow> RowShared = RowWeak.Pin())
				{
					const FSlateBrush* DropIndicatorBrush = RowShared->GetDropIndicatorBrush(InItemDropZone);
					static float OffsetX = 10.0f;
					FVector2D Offset(OffsetX * RowShared->GetIndentLevel(), 0.0f);

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId++,
						AllottedGeometry.ToPaintGeometry(FVector2D(AllottedGeometry.GetLocalSize() - Offset), FSlateLayoutTransform(Offset)),
						DropIndicatorBrush,
						ESlateDrawEffect::None,
						DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
					);
				}

				return LayerId;
			});

	return Row.ToSharedRef();
}

TSharedRef<SWidget> STraitStackView::GetStackListItemWidget(const TWeakPtr<FTraitStackViewEntry>& InEntryWeak, const TSharedPtr<FTraitEditorSharedData>& TraitEditorSharedDataLocal)
{
	return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.MaxDesiredHeight(20.0f)
					.MaxDesiredWidth(20.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Error"))
						.ColorAndOpacity_Lambda([InEntryWeak]()
						{
							if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
							{
								return FTraitEditorUtils::GetTraitIconErrorDisplayColor(Entry->TraitData->StackStatus);
							}

							return FSlateColor(FColor::Red);
						})
						.ToolTipText_Lambda([InEntryWeak]()->FText
						{
							if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
							{
								if (Entry->TraitData->StackStatus.TraitStatus != FTraitStackTraitStatus::EStackStatus::Ok)
								{
									TStringBuilder<1024> ErrorMessage;
									for (const auto& Status : Entry->TraitData->StackStatus.StatusMessages)
									{
										ErrorMessage.Append(Status.MessageText.ToString()).AppendChar(TEXT('\n'));
									}
									return FText::FromString(ErrorMessage.ToString());
								}
							}
							return FText::GetEmpty();
						})
						.Visibility_Lambda([InEntryWeak]()->EVisibility
						{
							if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
							{
								if (Entry->TraitData->StackStatus.TraitStatus != FTraitStackTraitStatus::EStackStatus::Ok)
								{
									return EVisibility::Visible;
								}
							}
							return EVisibility::Hidden;
						})
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(InEntryWeak.Pin()->TraitData->TraitDisplayName)
					.ColorAndOpacity_Lambda([InEntryWeak]()
					{
						if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
						{
							return FTraitEditorUtils::GetTraitTextDisplayColor(Entry->TraitData->TraitMode);
						}

						return FSlateColor(FColor::Red);
					})
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.MaxDesiredHeight(20.0f)
					.MaxDesiredWidth(20.0f)
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Right)
					[
						SNew(SButton)
						.ClickMethod(EButtonClickMethod::MouseUp)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.Visibility_Lambda([InEntryWeak, TraitEditorSharedDataLocal]()
						{
							if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
							{
								if (TraitEditorSharedDataLocal.IsValid() && TraitEditorSharedDataLocal->CurrentTraitsDataShared.IsValid())
								{
									// If the user has deleted the base but there are still additive Traits in the stack
									if (Entry->TraitData->TraitMode == ETraitMode::Base 
										&& TraitEditorSharedDataLocal->CurrentTraitsDataShared->Num() > 1
										&& (Entry->TraitData->StackStatus.TraitStatus == FTraitStackTraitStatus::EStackStatus::Invalid)
										&& Entry->TraitData->TraitUID == FTraitUID())
									{
										return EVisibility::Hidden;
									}
								}
							}
							return EVisibility::Visible;
						})
						.OnClicked_Lambda([InEntryWeak, this]()->FReply
						{
							FReply Reply = FReply::Unhandled();

							if (TSharedPtr<FTraitStackViewEntry> Entry = InEntryWeak.Pin())
							{
								Reply = ExecuteDelete(Entry->TraitData->TraitUID);
							}

							return Reply;
						})
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						]
					]
				]
			];

}

const TSharedPtr<FTraitDataEditorDef>& STraitStackView::GetSelectedTraitData() const
{
	return SelectedTraitData;
}

const FTraitUID STraitStackView::GetSelectedTraitUID() const
{
	FTraitUID TraitUID;

	if (const TSharedPtr<FTraitDataEditorDef> TraitDataEditorData = GetSelectedTraitData())
	{
		if (TraitDataEditorData.IsValid())
		{
			TraitUID = TraitDataEditorData->TraitUID;
		}
	}

	return TraitUID;
}

FReply STraitStackView::ExecuteDelete(FTraitUID TraitUID)
{
	FReply Reply = FReply::Unhandled();

	if (OnTraitDeleteRequest.IsBound())
	{
		Reply = OnTraitDeleteRequest.Execute(TraitUID);
	}

	SelectedTraitData.Reset();
	if (OnStatckTraitSelectionChanged.IsBound())
	{
		OnStatckTraitSelectionChanged.Execute(FTraitUID());
	}

	return Reply;
}

} // end namespace UE::Workspace

#undef LOCTEXT_NAMESPACE
