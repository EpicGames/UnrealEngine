// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "TraitCore/TraitUID.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "TraitCore/TraitMode.h"
#include "Graph/TraitEditorDefs.h"

class UAnimNextEdGraphNode;

namespace UE::Workspace
{

class IWorkspaceEditor;

}

namespace UE::UAF::Editor
{

struct FTraitStackViewEntry;


class STraitStackView : public SCompoundWidget
{
public:
	/** Used to notify the Trait Editor a Trait has been selected on the Stack */
	DECLARE_DELEGATE_OneParam(FOnStatckTraitSelectionChanged, const FTraitUID /*SelectedUID*/);

	/** Used to notify the Trait Editor a request to delete a Trait */
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnStackTraitDeleteRequest, const FTraitUID /*DeleteUID*/);

	/** Used to notify the Trait Editor a Trait Drag has been accepted on the Stack */
	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FOnStackTraitDragAccepted, const FTraitUID /*DraggedUID*/, const FTraitUID /*TargetUID*/, EItemDropZone /*DropZone*/);


	STraitStackView();

	SLATE_BEGIN_ARGS(STraitStackView) {}

	/** Called to notify a click on the delete button */
	SLATE_EVENT(FOnStackTraitDeleteRequest, OnTraitDeleteRequest)

	/** Called to notify a trait has been selected */
	SLATE_EVENT(FOnStatckTraitSelectionChanged, OnStatckTraitSelectionChanged)

	/** Called to notify a trait has been dragged onto the stack */
	SLATE_EVENT(FOnStackTraitDragAccepted, OnStackTraitDragAccepted)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData);

	void RefreshList();

private:

	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void HandleDelete();
	bool HasValidSelection() const;

	TSharedRef<ITableRow> HandleGenerateRow(TSharedRef<FTraitStackViewEntry> InEntry, const TSharedRef<STableViewBase>& InOwnerTable);
	TSharedRef<SWidget> GetStackListItemWidget(const TWeakPtr<FTraitStackViewEntry>& InEntryWeak, const TSharedPtr<FTraitEditorSharedData>& TraitEditorSharedDataLocal);

	const TSharedPtr<FTraitDataEditorDef>& GetSelectedTraitData() const;
	const FTraitUID GetSelectedTraitUID() const;
	
	FReply ExecuteDelete(FTraitUID TraitUID);

	TSharedPtr<FUICommandList> UICommandList;

	TSharedPtr<FTraitDataEditorDef> SelectedTraitData;
	TSharedPtr<FTraitEditorSharedData> TraitEditorSharedData;

	FOnStackTraitDeleteRequest OnTraitDeleteRequest;
	FOnStatckTraitSelectionChanged OnStatckTraitSelectionChanged;
	FOnStackTraitDragAccepted OnStackTraitDragAccepted;

	TSharedRef<SListView<TSharedRef<FTraitStackViewEntry>>> EntriesList;
	TArray<TSharedRef<FTraitStackViewEntry>> Entries;
};

};
