// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/TraitEditorDefs.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "Widgets/Layout/SBox.h"
#include "TraitCore/TraitInterfaceRegistry.h"

#define LOCTEXT_NAMESPACE "TraitListEditor"

namespace UE::UAF::Editor
{

/*static*/ FSlateColor FTraitEditorUtils::GetTraitIconErrorDisplayColor(const FTraitStackTraitStatus& InTraitStatus)
{
	switch (InTraitStatus.TraitStatus)
	{
		case FTraitStackTraitStatus::EStackStatus::Error:
		case FTraitStackTraitStatus::EStackStatus::Invalid:
		{
			return FSlateColor(FColor::Red);
		}
		case FTraitStackTraitStatus::EStackStatus::Warning:
		{
			return FSlateColor(FColor::Yellow);
		}
		default:
		{
			break;
		}
	}

	return FSlateColor::UseForeground();
}

/*static*/ FSlateColor FTraitEditorUtils::GetTraitTextDisplayColor(const ETraitMode TraitMode)
{
	switch (TraitMode)
	{
		case ETraitMode::Base: return FSlateColor(FColor::White);
		case ETraitMode::Additive: return FSlateColor(FColor::White);
		default:
		{
			break;
		}
	}

	return FSlateColor(FSlateColor::UseSubduedForeground());
}

/*static*/ FSlateColor FTraitEditorUtils::GetTraitBackroundDisplayColor(const ETraitMode TraitMode, bool bIsSelected, bool bIsHovered)
{
	static const FLinearColor SelectedColor(FColor::FromHex(TEXT("#5555ff")));
	static const FLinearColor BaseColor(FColor::FromHex(TEXT("#505050")));
	static const FLinearColor BaseColorHovered(FColor::FromHex(TEXT("#555555")));
	static const FLinearColor AdditiveColor(FColor::FromHex(TEXT("#707070")));
	static const FLinearColor AdditiveColorHovered(FColor::FromHex(TEXT("#757575")));

	if (bIsSelected)
	{
		return SelectedColor;
	}

	switch (TraitMode)
	{
		case ETraitMode::Base: return bIsHovered ? BaseColorHovered : BaseColor;
		case ETraitMode::Additive: return bIsHovered ? AdditiveColorHovered : AdditiveColor;
		default:
		{
			break;
		}
	}

	return FSlateColor(FSlateColor::UseSubduedForeground());
}

/*static*/ TSharedRef<SWidget> FTraitEditorUtils::GetInterfaceListWidget(EInterfaceDisplayType InterfaceDisplayType, const TSharedPtr<FTraitDataEditorDef>& InTraitDataShared, const TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedDataShared)
{
	if (!InTraitEditorSharedDataShared.IsValid()
		|| (!InTraitEditorSharedDataShared->bShowTraitInterfaces
			&& (!InTraitEditorSharedDataShared->bStackContainsErrors || !InTraitEditorSharedDataShared->bShowTraitInterfacesIfWarningsOrErrors)))
	{
		return SNullWidget::NullWidget;
	}

	TSharedRef<SHorizontalBox> InterfaceWidgetsHorizontalBox = SNew(SHorizontalBox);

	switch (InterfaceDisplayType)
	{
		case EInterfaceDisplayType::ListImplemented:
		{
			for(const FTraitInterfaceUID& InterfaceUID : InTraitDataShared->ImplementedInterfaces)
			{
				// For the trait list just display the interfaces with no spacing, one after another
				if( !IsInternal(InterfaceUID))
				{
					InterfaceWidgetsHorizontalBox->AddSlot()
						.AutoWidth()
						.Padding(FMargin(4.f, 1.f))
						[
							GetInterfaceWidget(InterfaceDisplayType, InterfaceUID, InTraitDataShared, InTraitEditorSharedDataShared)
						];
				}
			}
			break;
		}

		case EInterfaceDisplayType::ListRequired:
		{
			for (const FTraitInterfaceUID& InterfaceUID : InTraitDataShared->RequiredInterfaces)
			{
				if (!IsInternal(InterfaceUID))
				{
					InterfaceWidgetsHorizontalBox->AddSlot()
						.AutoWidth()
						.Padding(FMargin(4.f, 1.f))
						[
							GetInterfaceWidget(InterfaceDisplayType, InterfaceUID, InTraitDataShared, InTraitEditorSharedDataShared)
						];
				}
			}
			break;
		}

		case EInterfaceDisplayType::StackRequired:
		case EInterfaceDisplayType::StackImplemented:
		{
			// For the trait stack display the interfaces aligning them by type, so the same interface is always in the same position
			const int32 NumSlots = InTraitEditorSharedDataShared->StackUsedInterfaces.Num();
			for (int32 SlotIndex = 0; SlotIndex < NumSlots; SlotIndex++)
			{
				if (SlotIndex < InTraitEditorSharedDataShared->StackUsedInterfaces.Num())
				{
					// If the trait contains the slot, display the interface
					const FTraitInterfaceUID& InterfaceUID = InTraitEditorSharedDataShared->StackUsedInterfaces[SlotIndex];

					InterfaceWidgetsHorizontalBox->AddSlot()
					.AutoWidth()
					.Padding(FMargin(4.f, 1.f))
					[
						GetInterfaceWidget(InterfaceDisplayType, InterfaceUID, InTraitDataShared, InTraitEditorSharedDataShared)
					];
				}
				else
				{
					// else, display an empty space (so we keep the alignment)
					InterfaceWidgetsHorizontalBox->AddSlot()
						.AutoWidth()
						.Padding(FMargin(4.f, 1.f))
						[
							SNullWidget::NullWidget
						];
				}
			}
			break;
		}
	}

	return SNew(SBorder)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				InterfaceWidgetsHorizontalBox
			]
		];
}

/*static*/ TSharedRef<SWidget> FTraitEditorUtils::GetInterfaceWidget(EInterfaceDisplayType InterfaceDisplayType, FTraitInterfaceUID InterfaceUID, const TSharedPtr<FTraitDataEditorDef>& InTraitDataShared, const TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedDataShared)
{
	return SNew(SBorder)
		.BorderImage_Lambda([InterfaceDisplayType, InTraitDataShared, InTraitEditorSharedDataShared, InterfaceUID]()
			{
				switch (InterfaceDisplayType)
				{
					case EInterfaceDisplayType::StackRequired:
					{
						if (InTraitEditorSharedDataShared.IsValid() && InterfaceUID.IsValid())
						{
							if (InTraitDataShared.IsValid())
							{
								if (InTraitDataShared->StackStatus.MissingInterfaces.Find(InterfaceUID) != INDEX_NONE)
								{
									return FAppStyle::Get().GetBrush("Brushes.Error");
								}
							}
						}
						break;
					}
					case EInterfaceDisplayType::ListImplemented:
					{
						if (InTraitEditorSharedDataShared.IsValid() && InterfaceUID.IsValid())
						{
							if (InTraitDataShared.IsValid())
							{
								if (InTraitEditorSharedDataShared->StackMissingInterfaces.Find(InterfaceUID) != INDEX_NONE)
								{
									return FAppStyle::Get().GetBrush("Brushes.Select");
								}
							}
						}
						break;
					}
					default:
					{
						break;
					}
				}

				return FAppStyle::Get().GetBrush("Brushes.Background");
			})
		.Visibility_Lambda([InterfaceDisplayType, InTraitDataShared, InTraitEditorSharedDataShared, InterfaceUID]()
			{
				if (InTraitDataShared.IsValid() && InterfaceUID.IsValid())
				{
					switch (InterfaceDisplayType)
					{
						case EInterfaceDisplayType::ListImplemented:
						case EInterfaceDisplayType::StackImplemented:
						{
							if (InTraitDataShared->ImplementedInterfaces.Find(InterfaceUID) != INDEX_NONE)
							{
								return EVisibility::Visible;
							}
							break;
						}
						case EInterfaceDisplayType::ListRequired:
						case EInterfaceDisplayType::StackRequired:
						{
							if (InTraitDataShared->RequiredInterfaces.Find(InterfaceUID) != INDEX_NONE)
							{
								return EVisibility::Visible;
							}
							break;
						}
						default:
						{
							break;
						}
					}
				}
				return EVisibility::Hidden;
			})
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.BoldFont"))
			.ColorAndOpacity(FSlateColor(FColor::White))
			.Text_Lambda([InTraitEditorSharedDataShared, InterfaceUID]()->FText
			{
				if (InTraitEditorSharedDataShared.IsValid() && InterfaceUID.IsValid())
				{
					if (const ITraitInterface* TraitInterface = FTraitInterfaceRegistry::Get().Find(InterfaceUID))
					{
						return TraitInterface->GetDisplayShortName();
					}
				}
				return FText(LOCTEXT("MissingInterfaceShortName", "???"));
			})
			.ToolTipText_Lambda([InTraitEditorSharedDataShared, InterfaceUID]()->FText
			{
				if (InTraitEditorSharedDataShared.IsValid() && InterfaceUID.IsValid())
				{
					if (const ITraitInterface* TraitInterface = FTraitInterfaceRegistry::Get().Find(InterfaceUID))
					{
						return TraitInterface->GetDisplayName();
					}
				}
				return FText(LOCTEXT("MissingInterfaceName", "Invalid or Missing Interface"));
			})
		];
}

/*static*/ void FTraitEditorUtils::GenerateStackInterfacesUsedIndexes(TSharedPtr<FTraitDataEditorDef>& InTraitData, const TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData)
{
	if (InTraitData.IsValid() && InTraitEditorSharedData.IsValid())
	{
		for (const FTraitInterfaceUID& ImplementedInterface : InTraitData->ImplementedInterfaces)
		{
			const int32 IndexInStackList = InTraitEditorSharedData->StackUsedInterfaces.Find(ImplementedInterface);
			InTraitData->ImplementedInterfacesStackListIndexes.Add(IndexInStackList);
		}
		for (const FTraitInterfaceUID& RequiredInterface : InTraitData->RequiredInterfaces)
		{
			const int32 IndexInStackList = InTraitEditorSharedData->StackUsedInterfaces.Find(RequiredInterface);
			InTraitData->RequiredInterfacesStackListIndexes.Add(IndexInStackList);
		}
	}
}

/*static*/ TSharedPtr<FTraitDataEditorDef> FTraitEditorUtils::FindTraitInCurrentStackData(const FTraitUID InTraitUID, TSharedPtr<TArray<TSharedPtr<FTraitDataEditorDef>>> InTraitsDataShared, int32* OutIndex)
{
	if (OutIndex != nullptr)
	{
		*OutIndex = INDEX_NONE;
	}

	if (InTraitsDataShared.IsValid())
	{
		TArray<TSharedPtr<FTraitDataEditorDef>>* CurrentTraitsData = InTraitsDataShared.Get();

		const int32 NumTraits = CurrentTraitsData->Num();
		for (int32 i = 0; i < NumTraits; ++i)
		{
			TSharedPtr<FTraitDataEditorDef>& TraitData = (*CurrentTraitsData)[i];
			if (TraitData->TraitUID == InTraitUID)
			{
				if (OutIndex != nullptr)
				{
					*OutIndex = i;
				}
				return TraitData;
			}
		}
	}

	return nullptr;
}

/*static*/ bool FTraitEditorUtils::IsInternal(const FTraitInterfaceUID& InTraitInterfaceUID) 
{
	if (const ITraitInterface* ImplementedInterface = FTraitInterfaceRegistry::Get().Find(InTraitInterfaceUID))
	{
		return ImplementedInterface->IsInternal();
	}

	return false;
}


// --- FTraitListDragDropBase ---
TSharedPtr<SWidget> FTraitListDragDropBase::GetDefaultDecorator() const
{
	TWeakPtr<FTraitDataEditorDef> DraggedTraitDataWeakLocal = DraggedTraitDataWeak;

	return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.White"))
				.BorderBackgroundColor_Lambda([DraggedTraitDataWeakLocal]()
				{
					if (TSharedPtr<FTraitDataEditorDef> TraitData = DraggedTraitDataWeakLocal.Pin())
					{
						return FTraitEditorUtils::GetTraitBackroundDisplayColor(TraitData->TraitMode, false);
					}
					return FSlateColor(FColor::Red);
				})
				.Padding(FMargin(1.0f, 1.0f))
				[
					SNew(SVerticalBox)
					
					+ SVerticalBox::Slot()
					[
						SNew(SBox)
						.Padding(10.f, 10.f)
						.MinDesiredHeight(30.0f)
						.MinDesiredWidth(200.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.f)
							.HAlign(HAlign_Center)
							[
								SNew(STextBlock)
								.Justification(ETextJustify::Center)
								.Text_Lambda([DraggedTraitDataWeakLocal]()
								{
									if (TSharedPtr<FTraitDataEditorDef> TraitData = DraggedTraitDataWeakLocal.Pin())
									{
										return TraitData->TraitDisplayName;
									}
									return FText();
								})
								.ColorAndOpacity_Lambda([DraggedTraitDataWeakLocal]()
								{
									if (TSharedPtr<FTraitDataEditorDef> TraitData = DraggedTraitDataWeakLocal.Pin())
									{
										return FTraitEditorUtils::GetTraitTextDisplayColor(TraitData->TraitMode);
									}

									return FSlateColor(FColor::Red);
								})
							]
						]
					]
				]
			];
}

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE