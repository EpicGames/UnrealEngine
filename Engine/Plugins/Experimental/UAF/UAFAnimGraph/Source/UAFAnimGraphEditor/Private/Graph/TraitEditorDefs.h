// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/Trait.h"
#include "TraitCore/TraitMode.h"
#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitInterfaceUID.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"

class UAnimNextEdGraphNode;
struct FSlateColor;

namespace UE::Workspace
{

class IWorkspaceEditor;

}

namespace UE::UAF::Editor
{

struct FTraitStackTraitStatus
{
	enum class EStackStatus : uint8
	{
		Invalid,
		Ok,
		Warning,
		Error,
	};

	struct FStatusMessage
	{
		FStatusMessage() = default;
		FStatusMessage(EStackStatus InStatus, const FText& InMessageText)
			: Status(InStatus)
			, MessageText(InMessageText)
		{}

		EStackStatus Status = EStackStatus::Invalid;
		FText MessageText;
	};

	bool HasWarnings() const
	{
		return HasStatusType(EStackStatus::Warning);
	}
	bool HasErrors() const
	{
		return HasStatusType(EStackStatus::Error);
	}

	EStackStatus TraitStatus = EStackStatus::Invalid;
	TArray<FStatusMessage> StatusMessages;
	TArray<FTraitInterfaceUID> MissingInterfaces;

	bool HasStatusType(EStackStatus InStatus) const
	{
		for (const FStatusMessage& Status : StatusMessages)
		{
			if (Status.Status == InStatus)
			{
				return true;
			}
		}
		return false;
	}
};

struct FTraitDataEditorDef
{
	FTraitDataEditorDef() = default;

	FTraitDataEditorDef(const FTrait& Trait, const FText& InTraitDisplayName)
		: TraitName(Trait.GetTraitName())
		, TraitDisplayName(InTraitDisplayName)
		, TraitUID(Trait.GetTraitUID())
		, TraitMode(Trait.GetTraitMode())
		, ImplementedInterfaces(Trait.GetTraitInterfaces())
		, RequiredInterfaces(Trait.GetTraitRequiredInterfaces())
	{
	}

	FTraitDataEditorDef(const FName& InTraitName, const FText& InTraitDisplayName, FTraitUID InTraitUUID, ETraitMode InTraitMode, const TArray<FTraitInterfaceUID>& InImplementedInterfaces, const TArray<FTraitInterfaceUID>& InRequiredInterfaces, bool bInMultipleInstanceSupport)
		: TraitName(InTraitName)
		, TraitDisplayName(InTraitDisplayName)
		, TraitUID(InTraitUUID)
		, TraitMode(InTraitMode)
		, ImplementedInterfaces(InImplementedInterfaces)
		, RequiredInterfaces(InRequiredInterfaces)
		, bMultipleInstanceSupport(bInMultipleInstanceSupport)
	{}

	FName TraitName;
	FText TraitDisplayName;
	FTraitUID TraitUID;
	ETraitMode TraitMode = ETraitMode::Invalid;

	TArray<FTraitInterfaceUID> ImplementedInterfaces;
	TArray<int32> ImplementedInterfacesStackListIndexes;
	TArray<FTraitInterfaceUID> RequiredInterfaces;
	TArray<int32> RequiredInterfacesStackListIndexes;

	bool bMultipleInstanceSupport = false;

	FTraitStackTraitStatus StackStatus;
};

struct FTraitCategoryData
{
	FTraitCategoryData() = default;

	FTraitCategoryData( const FName& InCategory, const FText& InCategoryText)
		: Category(InCategory)
		, CategoryText(InCategoryText)
	{}

	FName Category;
	FText CategoryText;

	TArray<TSharedPtr<FTraitDataEditorDef>> TraitList;
};

struct FTraitEditorSharedData
{
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditorWeak;
	TWeakObjectPtr<UAnimNextEdGraphNode> EdGraphNodeWeak = nullptr;
	TSharedPtr<TArray<TSharedPtr<FTraitDataEditorDef>>> CurrentTraitsDataShared;

	TArray<FTraitInterfaceUID> StackUsedInterfaces;
	TArray<FTraitInterfaceUID> StackMissingInterfaces;
	TArray<int32> StackUsedInteraceMissingIndexes;

	bool bStackContainsErrors = false;
	bool bShowTraitInterfaces = false;
	bool bShowTraitInterfacesIfWarningsOrErrors = false;
	bool bAdvancedView = false;
};

struct FTraitEditorUtils
{
	static FSlateColor GetTraitIconErrorDisplayColor(const FTraitStackTraitStatus& InTraitStatus);
	static FSlateColor GetTraitTextDisplayColor(const ETraitMode TraitMode);
	static FSlateColor GetTraitBackroundDisplayColor(const ETraitMode TraitMode, bool bIsSelected = false, bool bIsHovered = false);

	enum class EInterfaceDisplayType : uint8
	{
		ListImplemented,
		ListRequired,
		StackImplemented,
		StackRequired
	};
	static TSharedRef<SWidget> GetInterfaceListWidget(EInterfaceDisplayType InterfaceDisplayType, const TSharedPtr<FTraitDataEditorDef>& InTraitDataShared, const TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedDataShared);
	static TSharedRef<SWidget> GetInterfaceWidget(EInterfaceDisplayType InterfaceDisplayType, FTraitInterfaceUID InterfaceUID, const TSharedPtr<FTraitDataEditorDef>& InTraitDataShared, const TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedDataShared);
	static void GenerateStackInterfacesUsedIndexes(TSharedPtr<FTraitDataEditorDef>& TraitData, const TSharedPtr<FTraitEditorSharedData>& InTraitEditorSharedData);

	static TSharedPtr<FTraitDataEditorDef> FindTraitInCurrentStackData(const FTraitUID InTraitUID, TSharedPtr<TArray<TSharedPtr<FTraitDataEditorDef>>> CurrentTraitsDataShared, int32* OutIndex = nullptr);

	// Internal Interfaces are not shown in the Traits Editor
	static bool IsInternal(const FTraitInterfaceUID& InterfaceUID);
};

// --- FTraitListDragDropBase ---
class FTraitListDragDropBase : public FDecoratedDragDropOp
{
public:
	/** @return The identifier being dragged */
	const TWeakPtr<FTraitDataEditorDef>& GetDraggedTraitData() const
	{
		return DraggedTraitDataWeak;
	}

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

protected:
	TWeakPtr<FTraitDataEditorDef> DraggedTraitDataWeak;
};

// --- FTraitListDragDropOp ---
class FTraitListDragDropOp : public FTraitListDragDropBase
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTraitListDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FTraitListDragDropOp> New(TWeakPtr<FTraitDataEditorDef> InDraggedTraitDataWeak)
	{
		TSharedRef<FTraitListDragDropOp> Operation = MakeShared<FTraitListDragDropOp>();
		Operation->DraggedTraitDataWeak = InDraggedTraitDataWeak;
		Operation->Construct();
		return Operation;
	}
};

};
