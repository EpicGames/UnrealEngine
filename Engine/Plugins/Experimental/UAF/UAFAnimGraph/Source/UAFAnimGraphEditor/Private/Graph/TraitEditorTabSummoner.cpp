// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/TraitEditorTabSummoner.h"
#include "IWorkspaceEditor.h"

#define LOCTEXT_NAMESPACE "WorkspaceTabSummoner"

namespace UE::UAF::Editor
{

const FLazyName TraitEditorTabName("TraitEditorTab");

FTraitEditorTabSummoner::FTraitEditorTabSummoner(const TSharedPtr<UE::Workspace::IWorkspaceEditor>& InHostingApp)
	: FWorkflowTabFactory(TraitEditorTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
{
	TabLabel = LOCTEXT("TraitEditorTabLabel", "Trait Editor");
	TabIcon = FSlateIcon("EditorStyle", "LevelEditor.Tabs.Outliner");
	ViewMenuDescription = LOCTEXT("TraitEditorTabMenuDescription", "Trait Editor");
	ViewMenuTooltip = LOCTEXT("TraitEditorTabToolTip", "Shows the Trait Editor tab.");
	bIsSingleton = true;

	TraitEditorView = SNew(STraitEditorView, InHostingApp.ToWeakPtr());
}

TSharedRef<SWidget> FTraitEditorTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return TraitEditorView.ToSharedRef();
}

FText FTraitEditorTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewMenuTooltip;
}

} // end namespace UE::UAF::Graph

#undef LOCTEXT_NAMESPACE