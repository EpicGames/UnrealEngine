// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "STraitEditorView.h"

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{

extern const FLazyName TraitEditorTabName;

DECLARE_DELEGATE_OneParam(FOnTraitEditorCreated, TSharedRef<STraitEditorView>);

struct FTraitEditorTabSummoner : public FWorkflowTabFactory
{
public:
	FTraitEditorTabSummoner(const TSharedPtr<UE::Workspace::IWorkspaceEditor>& InHostingApp);

private:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

	// The widget this tab spawner wraps
	TSharedPtr<STraitEditorView> TraitEditorView;
};

} // end namespace UE::UAF::Editor
