// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"
#include "AnimNextAnimGraphProvider.h"
#include "TraceServices/Utils.h"
#include "StructUtils/PropertyBag.h"
#include "Serialization/ObjectReader.h"

FAnimNextAnimGraphAnalyzer::FAnimNextAnimGraphAnalyzer(TraceServices::IAnalysisSession& InSession, FAnimNextAnimGraphProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FAnimNextAnimGraphAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_EvaluationProgram, "UAFAnimGraph", "EvaluationProgram");
}

bool FAnimNextAnimGraphAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FAnimNextAnimGraphAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_EvaluationProgram:
		{
			uint64 OuterObjectId = EventData.GetValue<uint64>("OuterObjectId");
			uint64 GraphInstanceId = EventData.GetValue<uint64>("InstanceId");
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");

			TArrayView<const uint8> ProgramData = EventData.GetArrayView<uint8>("ProgramData");

			Provider.AppendEvaluationProgram(Context.EventTime.AsSeconds(Cycle), RecordingTime, OuterObjectId, GraphInstanceId, ProgramData);
			break;
		}
	}

	return true;
}
