// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

class FAnimNextAnimGraphProvider;
namespace TraceServices { class IAnalysisSession; }

class FAnimNextAnimGraphAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FAnimNextAnimGraphAnalyzer(TraceServices::IAnalysisSession& InSession, FAnimNextAnimGraphProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_EvaluationProgram,
	};

	TraceServices::IAnalysisSession& Session;
	FAnimNextAnimGraphProvider& Provider;
};
