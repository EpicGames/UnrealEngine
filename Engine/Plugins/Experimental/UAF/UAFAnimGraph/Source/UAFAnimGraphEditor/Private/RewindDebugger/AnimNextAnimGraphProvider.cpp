// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphProvider.h"
#include "ObjectTrace.h"

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GAnimNextAnimGraphProviderLockState;
}

FName FAnimNextAnimGraphProvider::ProviderName("AnimNextAnimGraphProvider");

#define LOCTEXT_NAMESPACE "AnimNextAnimGraphProvider"

FAnimNextAnimGraphProvider::FAnimNextAnimGraphProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FAnimNextAnimGraphProvider::AppendEvaluationProgram(double ProfileTime, double RecordingTime, uint64 OuterObjectId, uint64 GraphInstanceId, const TArrayView<const uint8>& ProgramData)
{
	EditAccessCheck();

	EvaluationGraphs.Add(OuterObjectId, GraphInstanceId);
	
	const TSharedRef<FEvaluationProgramData>* Data = EvaluationProgramData.Find(GraphInstanceId);
	if (Data == nullptr)
	{
		TSharedRef<FEvaluationProgramData> NewData = MakeShared<FEvaluationProgramData>(Session);
       	NewData->GraphInstanceId = GraphInstanceId;
		EvaluationProgramData.Add(GraphInstanceId, NewData);
		Data = EvaluationProgramData.Find(GraphInstanceId);
	}
	
	if (!ProgramData.IsEmpty())
	{
		(*Data)->EvaluationProgramTimeline.AppendEvent(ProfileTime, TArray<uint8>(ProgramData));
	}
}

const FEvaluationProgramData* FAnimNextAnimGraphProvider::GetEvaluationProgramData(uint64 GraphInstanceId) const
{
	ReadAccessCheck();
	
	const TSharedRef<FEvaluationProgramData>* Data = EvaluationProgramData.Find(GraphInstanceId);
	if (Data)
	{
		return &**Data;
	}
	return nullptr;
}


void FAnimNextAnimGraphProvider::EnumerateEvaluationGraphs(uint64 OuterObjectId, TFunctionRef<void(uint64 GraphInstanceId)> Callback) const
{
	ReadAccessCheck();

	TArray<uint64> GraphIds;
	EvaluationGraphs.MultiFind(OuterObjectId, GraphIds);
	
	for (uint64 GraphId : GraphIds)
	{
		Callback(GraphId);
	}
}

#undef LOCTEXT_NAMESPACE
