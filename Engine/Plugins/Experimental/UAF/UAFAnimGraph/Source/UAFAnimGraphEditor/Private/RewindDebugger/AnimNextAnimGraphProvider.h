// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Model/PointTimeline.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Model/IntervalTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"

namespace TraceServices
{
	class IAnalysisSession;
	extern thread_local FProviderLock::FThreadLocalState GAnimNextAnimGraphProviderLockState;
}

struct FEvaluationProgramData
{
	FEvaluationProgramData(TraceServices::IAnalysisSession& Session)
		: GraphInstanceId(0),
		EvaluationProgramTimeline(Session.GetLinearAllocator( ))
			
	{
	}

	uint64 GraphInstanceId;
	TraceServices::TPointTimeline<TArray<uint8>> EvaluationProgramTimeline;
};

class FAnimNextAnimGraphProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	static FName ProviderName;

	FAnimNextAnimGraphProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GAnimNextAnimGraphProviderLockState);
	}

	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GAnimNextAnimGraphProviderLockState);
	}

	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GAnimNextAnimGraphProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GAnimNextAnimGraphProviderLockState);
	}

	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GAnimNextAnimGraphProviderLockState);
	}

	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GAnimNextAnimGraphProviderLockState);
	}

	const FEvaluationProgramData* GetEvaluationProgramData(uint64 GraphInstanceId) const;
	void AppendEvaluationProgram(double ProfileTime, double RecordingTime, uint64 ObjectInstanceId, uint64 GraphInstanceId, const TArrayView<const uint8>& ProgramData);
	
	void EnumerateEvaluationGraphs(uint64 OuterObjectId, TFunctionRef<void(uint64 GraphInstanceId)> Callback) const;
	
private:
	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;

	TMap<uint64, TSharedRef<FEvaluationProgramData>> EvaluationProgramData;

	// map from object id to a list of graph instance ids that have evaluation data
	TMultiMap<uint64, uint64> EvaluationGraphs;
};
