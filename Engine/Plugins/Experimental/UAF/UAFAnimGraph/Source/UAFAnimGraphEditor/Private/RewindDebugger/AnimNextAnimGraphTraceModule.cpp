// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphTraceModule.h"
#include "AnimNextAnimGraphProvider.h"
#include "AnimNextAnimGraphAnalyzer.h"

FName FAnimNextAnimGraphTraceModule::ModuleName("UAFAnimGraph");

void FAnimNextAnimGraphTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("UAFAnimGraph");
}

void FAnimNextAnimGraphTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FAnimNextAnimGraphProvider> AnimNextAnimGraphProvider = MakeShared<FAnimNextAnimGraphProvider>(InSession);
	InSession.AddProvider(FAnimNextAnimGraphProvider::ProviderName, AnimNextAnimGraphProvider);

	InSession.AddAnalyzer(new FAnimNextAnimGraphAnalyzer(InSession, *AnimNextAnimGraphProvider));
}

void FAnimNextAnimGraphTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("UAFAnimGraph"));
}

void FAnimNextAnimGraphTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

