// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationProgramTrack.h"

#include "AnimNextAnimGraphProvider.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "Component/AnimNextComponent.h"
#include "Serialization/ObjectReader.h"
#include "TraceServices/Model/Frames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EvaluationProgramTrack)


#define LOCTEXT_NAMESPACE "EvaluationProgramTrack"

namespace UE::UAF::Editor
{

static const FName AnimNextModulesName("AnimNextModules");

FName FEvaluationProgramTrackCreator::GetTargetTypeNameInternal() const
{
 	return FUAFAssetInstance::StaticStruct()->GetFName();
}

	
FText FEvaluationProgramTrack::GetDisplayNameInternal() const
{
 	return NSLOCTEXT("RewindDebugger", "EvaluationProgramTrackName", "EvaluationProgram");
}

void FEvaluationProgramTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({AnimNextModulesName, LOCTEXT("UAFSystems", "UAF Systems")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FEvaluationProgramTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FEvaluationProgramTrack>(InObjectId.GetMainId());
}

		
FEvaluationProgramTrack::FEvaluationProgramTrack(uint64 InObjectId) :
	ObjectId(InObjectId)
{
	Initialize();
}
	
FEvaluationProgramTrack::FEvaluationProgramTrack(uint64 InObjectId, uint64 InInstanceId) :
   	ObjectId(InObjectId),
	InstanceId(InInstanceId)
{
   	Initialize();
}
	
	
FEvaluationProgramTrack::~FEvaluationProgramTrack()
{
 	if (UEvaluationProgramDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get())
   	{
   		DetailsObject->ClearFlags(RF_Standalone);
   	}
}
	
void FEvaluationProgramTrack::Initialize()
{
	ExistenceRange = MakeShared<SEventTimelineView::FTimelineEventData>();
	ExistenceRange->EventWindows.Add({0,0, GetDisplayNameInternal(), GetDisplayNameInternal(), FLinearColor(0.1f,0.15f,0.11f)});
	
   	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
   	FDetailsViewArgs DetailsViewArgs;
   	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
   	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InitializeDetailsObject();
}
	
UEvaluationProgramDetailsObject* FEvaluationProgramTrack::InitializeDetailsObject()
{
	UEvaluationProgramDetailsObject* DetailsObject = NewObject<UEvaluationProgramDetailsObject>();
	DetailsObject->SetFlags(RF_Standalone);
	DetailsObjectWeakPtr = MakeWeakObjectPtr(DetailsObject);
	DetailsView->SetObject(DetailsObject);
 	return DetailsObject;
}

bool FEvaluationProgramTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEvaluationProgramTrack::UpdateInternal);
 	
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
 	const TRange<double> ViewRange = RewindDebugger->GetCurrentViewRange();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

 	bool bChanged = false;
	
	if (const FAnimNextAnimGraphProvider* AnimNextAnimGraphProvider = AnalysisSession->ReadProvider<FAnimNextAnimGraphProvider>("AnimNextAnimGraphProvider"))
	{
		double CurrentScrubTime = RewindDebugger->CurrentTraceTime();

		UEvaluationProgramDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get();
		if (DetailsObject == nullptr)
		{
			// this should not happen unless the object was garbage collected (which should not happen since it's marked as Standalone)
			Initialize();
		}

		TraceServices::FProviderReadScopeLock AnimNextProviderReadScope(*AnimNextAnimGraphProvider);

		if (InstanceId == 0)
		{
		 	AnimNextAnimGraphProvider->EnumerateEvaluationGraphs(ObjectId, [this](uint64 GraphId)
		 	{
		 		InstanceId = GraphId;
		 	});
		}
		
		if(InstanceId != 0)
		{
			if (const FEvaluationProgramData* Data = AnimNextAnimGraphProvider->GetEvaluationProgramData(InstanceId))
			{
				if (PreviousScrubTime != CurrentScrubTime)
				{
					PreviousScrubTime = CurrentScrubTime;

					bool bFrameFound;
					TraceServices::FFrame MarkerFrame;
					{
						TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
						const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
						bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame);
					}

					if (bFrameFound)
					{
						const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
						TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
						Data->EvaluationProgramTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime, [DetailsObject, GameplayProvider](double InStartTime, double InEndTime, uint32 InDepth, const TArray<uint8>& VariableData)
						{
							FMemoryReader Reader(VariableData);
							FObjectAsTraceIdProxyArchiveReader Archive(Reader, GameplayProvider);

							static const FSerializableEvaluationProgram Defaults;
							FSerializableEvaluationProgram::StaticStruct()->SerializeItem(Archive, &DetailsObject->Program, &Defaults);
								
							return TraceServices::EEventEnumerate::Stop;
						});
					}
				}
			}

			// TArray<uint64, TInlineAllocator<32>> CurrentChildren;
		
			// // update/create child tracks
			// AnimNextProvider->EnumerateChildInstances(InstanceId, [this, &bChanged, &CurrentChildren, &ViewRange](const FDataInterfaceData& ChildData)
			// {
			// 	if (ChildData.StartTime < ViewRange.GetUpperBoundValue() && ChildData.EndTime > ViewRange.GetLowerBoundValue())
			// 	{
			// 		CurrentChildren.Add(ChildData.InstanceId);
			// 		if (!Children.ContainsByPredicate([&ChildData](TSharedPtr<FEvaluationProgramTrack>& Child) { return Child->InstanceId == ChildData.InstanceId; } ))
			// 		{
			// 			Children.Add(MakeShared<FEvaluationProgramTrack>(ObjectId, ChildData.InstanceId));
			// 			bChanged = true;
			// 		}
			// 	}
			// });
			//
			// int32 NumRemoved = Children.RemoveAll([&CurrentChildren](TSharedPtr<FEvaluationProgramTrack>& Child)
			// {
			// 	return !CurrentChildren.Contains(Child->InstanceId);
			// });
			// bChanged |= (NumRemoved > 0);
		}

		// for (auto& Child : Children)
		// {
		// 	bChanged |= Child->Update();
		// }
	}
	
 	return bChanged;
}

bool FEvaluationProgramTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	// TRACE_CPUPROFILER_EVENT_SCOPE(FEvaluationProgramTrack::HasDebugInfoInternal);

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FAnimNextAnimGraphProvider* AnimNextAnimGraphProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<FAnimNextAnimGraphProvider>("AnimNextAnimGraphProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimNextAnimGraphProvider);
			uint64 InstanceId = 0;
			AnimNextAnimGraphProvider->EnumerateEvaluationGraphs(InObjectId.GetMainId(), [&InstanceId](uint64 GraphId)
				{
					InstanceId = GraphId;
				});

			if (const FEvaluationProgramData* Data = AnimNextAnimGraphProvider->GetEvaluationProgramData(InstanceId))
			{
				return Data->EvaluationProgramTimeline.GetEventCount() != 0;
			}
		}
	}

	// We don't have any evaluation tasks
	return false;
}


}

#undef LOCTEXT_NAMESPACE
