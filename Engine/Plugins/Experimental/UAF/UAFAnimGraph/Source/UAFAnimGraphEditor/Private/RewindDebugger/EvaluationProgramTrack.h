// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "EvaluationVM/SerializableEvaluationProgram.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"
#include "IDetailsView.h"
#include "SEventTimelineView.h"

#include "EvaluationProgramTrack.generated.h"

UCLASS()
class UEvaluationProgramDetailsObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Details, meta=(ShowOnlyInnerProperties))
	FSerializableEvaluationProgram Program;
};

namespace UE::UAF::Editor
{
	
class FEvaluationProgramTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FEvaluationProgramTrack(uint64 InObjectId);
	FEvaluationProgramTrack(uint64 InObjectId, uint64 InstanceId);
	virtual ~FEvaluationProgramTrack();
	
	TSharedPtr<SEventTimelineView::FTimelineEventData> GetExistenceRange() const { return ExistenceRange; }
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override { return DetailsView; }

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "AnimNextModule"; }
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	void Initialize();
	UEvaluationProgramDetailsObject* InitializeDetailsObject();

	TSharedPtr<IDetailsView> DetailsView;
	FSlateIcon Icon;
	uint64 ObjectId;
	uint64 InstanceId = 0; 
	double PreviousScrubTime = -1.0;
	TWeakObjectPtr<UEvaluationProgramDetailsObject> DetailsObjectWeakPtr;
	TSharedPtr<SEventTimelineView::FTimelineEventData> ExistenceRange;

	// TArray<TSharedPtr<FEvaluationProgramTrack>> Children;
};

#if WITH_ENGINE
class FEvaluationProgramTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override { return "AnimNextModule"; }
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}