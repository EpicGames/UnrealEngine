// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceInfoTrack.h"

#include "AnimNextAnimGraphProvider.h"
#include "Common/ProviderLock.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Animation/AnimSequence.h"
#include "EvaluationVM/Tasks/PushAnimSequenceKeyframe.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "Component/AnimNextComponent.h"
#include "TraceServices/Model/Frames.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Colors/SColorBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceInfoTrack)

#define LOCTEXT_NAMESPACE "EvaluationProgramTrack"

namespace UE::UAF::Editor
{
FName FSequenceInfoTrackCreator::GetTargetTypeNameInternal() const
{
	return UUAFComponent::StaticClass()->GetFName();
}


FText FSequenceInfoTrack::GetDisplayNameInternal() const
{
	return NSLOCTEXT("RewindDebugger", "SequenceInfoTrackName", "SequenceInfo");
}

void FSequenceInfoTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({FSequenceInfoTrack::TrackName, LOCTEXT("AnimNextSequenceInfo", "AnimNextSequenceInfo")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FSequenceInfoTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FSequenceInfoTrack>(InObjectId.GetMainId());
}


void FAnimNextSequenceTraceInfoCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow,
                                                              IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

// Copied from SAnimNotifyPanel
// Todo: find common place for this to live
static FLinearColor GenerateColorFromName(FName Name)
{
	constexpr uint8 Saturation = 255;
	constexpr uint8 Luminosity = 255;
	const uint8 Hue = static_cast<uint8>(GetTypeHash(Name.ToString())) * 157;
	return FLinearColor::MakeFromHSV8(Hue, Saturation, Luminosity);
}

// Helper for lambdas
const FAnimNextSequenceTraceInfo* ExtractSequenceTraceInfo(TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	void* StructData = nullptr;
	FPropertyAccess::Result Result = StructPropertyHandle->GetValueData(StructData);
	if (Result != FPropertyAccess::Success)
		return nullptr;

	const FAnimNextSequenceTraceInfo* SequenceInfo = static_cast<FAnimNextSequenceTraceInfo*>(StructData);
	return SequenceInfo;
}

void FAnimNextSequenceTraceInfoCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
                                                                class IDetailChildrenBuilder& StructBuilder,
                                                                IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(StructPropertyHandle->GetProperty()))
	{
		if (StructProperty->Struct == FAnimNextSequenceTraceInfo::StaticStruct())
		{
			static const FColor TimelineBackground(0xFF575761);
			static const FColor TimelineForeground(0xFFF59F00);
			constexpr float TimelineWidth = 300.f;
			constexpr float TimelineHeight = 20.f;

			const FAnimNextSequenceTraceInfo* SequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
			if (SequenceInfo == nullptr)
				return;

			FString SequenceName = SequenceInfo->AnimSequence.IsValid() ? SequenceInfo->AnimSequence->GetName() : TEXT("NULL");

			TSharedRef<SCanvas> TimelineCanvas =
				SNew(SCanvas)

				// Before playhead block
				+ SCanvas::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Position(FVector2D::ZeroVector)
				.Size_Lambda( [=]()
				{
					const FAnimNextSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					if (UpdatedSequenceInfo == nullptr)
						return FVector2D::ZeroVector;
					return FVector2D(TimelineWidth * UpdatedSequenceInfo->CalcAnimTimeRatio(), TimelineHeight);
				} )
				[
					SNew(SColorBlock)
					.Color(TimelineForeground)
				]

				// After playhead block
				+ SCanvas::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Position_Lambda([=]()
				{
					const FAnimNextSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					if (UpdatedSequenceInfo == nullptr)
						return FVector2D::ZeroVector;
					
					return FVector2D(UpdatedSequenceInfo->CalcAnimTimeRatio() * TimelineWidth, 0.f);
				})
				.Size_Lambda([=]()
				{
					const FAnimNextSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					if (UpdatedSequenceInfo == nullptr)
						return FVector2D::ZeroVector;
					
					return FVector2D(TimelineWidth * (1.0f - UpdatedSequenceInfo->CalcAnimTimeRatio()), TimelineHeight);
				})
				[
					SNew(SColorBlock)
					.Color(TimelineBackground)
				]

				// Time info
				+ SCanvas::Slot()
				.Position(FVector2D(TimelineWidth, 0.f))
				.Size(FVector2D(TimelineWidth, TimelineHeight))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					                .Text_Lambda([=]()
					                {
					                	const FAnimNextSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					                	if (UpdatedSequenceInfo == nullptr)
					                		return FText::FromString("NULL");
						                return FText::FromString(
					   FString::Printf(TEXT("%.2fs (%.0f%%)"), UpdatedSequenceInfo->CurrentTimeSeconds, UpdatedSequenceInfo->CalcAnimTimeRatio() * 100.0f));
					                })
					                .Margin(FMargin(2.0f))
					                .Justification(ETextJustify::Left)
				];

			// Todo: how to do this with lambda?
			// Add all sync markers
			if (SequenceInfo->DurationSeconds > 0.0f)
			{
				for (auto Marker : SequenceInfo->SyncMarkers)
				{
					constexpr float MarkerHeight = 8.f;
					constexpr float MarkerWidth = 4.0f;
					float MarkerTimeRatio = Marker.Time / SequenceInfo->DurationSeconds;

					TimelineCanvas->AddSlot()
					              .HAlign(HAlign_Left)
					              .VAlign(VAlign_Center)
					              .Position(FVector2D((MarkerTimeRatio * TimelineWidth) - 0.5f * MarkerWidth,
					                                  (0.5f * (TimelineHeight) + MarkerHeight)))
					              .Size(FVector2D(MarkerWidth, MarkerHeight))
					[
						SNew(SColorBlock)
						.Color(GenerateColorFromName(Marker.Name))
						.ToolTipText(FText::FromString(Marker.Name.ToString()))
					];
				}
			}

			TSharedPtr<IPropertyHandle> SequenceProperty =
				StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextSequenceTraceInfo, AnimSequence));

			StructBuilder.AddCustomRow(LOCTEXT("FAnimNextSequenceTraceInfoRow", "FAnimNextSequenceTraceInfo"))
			             .NameContent()
				[
					SequenceProperty->CreatePropertyValueWidget()
				]
				.ValueContent()
				[
					TimelineCanvas
				];
		}
	}

	StructPropertyHandle->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateLambda([]
		{
			UE_LOGF(LogTemp, Display, "PropertyChange");
		}));
}

FSequenceInfoTrack::FSequenceInfoTrack(uint64 InObjectId)
	: ObjectId(InObjectId)
{
	Initialize();
}

FSequenceInfoTrack::FSequenceInfoTrack(uint64 InObjectId, uint64 InInstanceId)
	: ObjectId(InObjectId)
	, InstanceId(InInstanceId)
{
	Initialize();
}


FSequenceInfoTrack::~FSequenceInfoTrack()
{
	if (USequenceInfoDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get())
	{
		DetailsObject->ClearFlags(RF_Standalone);
	}
}

void FSequenceInfoTrack::Initialize()
{
	ExistenceRange = MakeShared<SEventTimelineView::FTimelineEventData>();
	ExistenceRange->EventWindows.Add({0, 0, GetDisplayNameInternal(), GetDisplayNameInternal(), FLinearColor(0.1f, 0.15f, 0.11f)});

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InitializeDetailsObject();
}

USequenceInfoDetailsObject* FSequenceInfoTrack::InitializeDetailsObject()
{
	USequenceInfoDetailsObject* DetailsObject = NewObject<USequenceInfoDetailsObject>();
	DetailsObject->SetFlags(RF_Standalone);
	DetailsObjectWeakPtr = MakeWeakObjectPtr(DetailsObject);
	DetailsView->SetObject(DetailsObject);
	return DetailsObject;
}

void FSequenceInfoTrack::RefreshSequenceInfoFromEvaluationProgram(TArray<FAnimNextSequenceTraceInfo>& OutSequenceInfo,
                                                                  const FSerializableEvaluationProgram& Program)
{
	OutSequenceInfo.Reset();
	for (const FInstancedStruct& Task : Program.Tasks)
	{
		if (Task.GetScriptStruct() == FAnimNextAnimSequenceKeyframeTask::StaticStruct())
		{
			const FAnimNextAnimSequenceKeyframeTask* SequenceTask = Task.GetPtr<FAnimNextAnimSequenceKeyframeTask>();
			FAnimNextSequenceTraceInfo& TraceInfo = OutSequenceInfo.Emplace_GetRef();
			TraceInfo.AnimSequence = SequenceTask->AnimSequence;
			TraceInfo.CurrentTimeSeconds = SequenceTask->SampleTime;

			if (const UAnimSequence* AnimSequence = SequenceTask->AnimSequence.Get())
			{
				TraceInfo.DurationSeconds = AnimSequence->GetPlayLength();

				// Fill out sync markers
				TraceInfo.SyncMarkers.Reserve(AnimSequence->AuthoredSyncMarkers.Num());
				for (auto Marker : AnimSequence->AuthoredSyncMarkers)
				{
					TraceInfo.SyncMarkers.Emplace(Marker.Time, Marker.MarkerName);
				}
			}
		}
	}
}

bool FSequenceInfoTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSequenceInfoTrack::UpdateInternal);

	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bChanged = false;

	if (const FAnimNextAnimGraphProvider* AnimNextAnimGraphProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<
		FAnimNextAnimGraphProvider>("AnimNextAnimGraphProvider"))
	{
		double CurrentScrubTime = IRewindDebugger::Instance()->CurrentTraceTime();

		USequenceInfoDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get();
		if (DetailsObject == nullptr)
		{
			// this should not happen unless the object was garbage collected (which should not happen since it's marked as Standalone)
			Initialize();
		}

		if (InstanceId == 0)
		{
			AnimNextAnimGraphProvider->EnumerateEvaluationGraphs(ObjectId, [this](uint64 GraphId)
			{
				InstanceId = GraphId;
			});
		}

		if (InstanceId != 0)
		{
			TraceServices::FProviderReadScopeLock AnimNextProviderReadScope(*AnimNextAnimGraphProvider);
			if (const FEvaluationProgramData* Data = AnimNextAnimGraphProvider->GetEvaluationProgramData(InstanceId))
			{
				if (PreviousScrubTime != CurrentScrubTime)
				{
					PreviousScrubTime = CurrentScrubTime;

					bool bFrameFound;
					TraceServices::FFrame MarkerFrame;
					{
						TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
						const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
						bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame);
					}
		
					if (bFrameFound)
					{
						const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
						TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

						Data->EvaluationProgramTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
						[DetailsObject, GameplayProvider, this] ( double InStartTime, double InEndTime, uint32 InDepth, const TArray<uint8>& VariableData)
						{
							// Why do this if panel is not selected?
							FMemoryReader Reader(VariableData);
							FObjectAsTraceIdProxyArchiveReader Archive(Reader, GameplayProvider);

							static const FSerializableEvaluationProgram Defaults;
							FSerializableEvaluationProgram Program;
							FSerializableEvaluationProgram::StaticStruct()->SerializeItem(
							Archive, &Program, &Defaults);
							RefreshSequenceInfoFromEvaluationProgram(
							DetailsObject->SequenceTraceInfo, Program);
							DetailsView->ForceRefresh(); // We need to force refresh to get the sync markers to refresh. todo: how to handle without recreating UI?
							return TraceServices::EEventEnumerate::Stop;
							});
						}
				}
			}
		}
	}

	return bChanged;
}

bool FSequenceInfoTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FAnimNextAnimGraphProvider* AnimNextAnimGraphProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<FAnimNextAnimGraphProvider>("AnimNextAnimGraphProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimNextAnimGraphProvider);
			uint64 InstanceId = 0;
			AnimNextAnimGraphProvider->EnumerateEvaluationGraphs(InObjectId.GetMainId(), [&InstanceId](uint64 GraphId)
				{
					InstanceId = GraphId;
				});

			if (const FEvaluationProgramData* Data = AnimNextAnimGraphProvider->GetEvaluationProgramData(InstanceId))
			{
				return Data->EvaluationProgramTimeline.GetEventCount() != 0;
			}
		}
	}

	// We don't have any evaluation tasks
	return false;
}
}

#undef LOCTEXT_NAMESPACE
