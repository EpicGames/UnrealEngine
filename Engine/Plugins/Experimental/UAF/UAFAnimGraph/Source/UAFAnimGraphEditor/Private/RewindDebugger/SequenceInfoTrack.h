// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "EvaluationVM/SerializableEvaluationProgram.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"
#include "IDetailsView.h"
#include "IPropertyTypeCustomization.h"

#include "SequenceInfoTrack.generated.h"

class UAnimSequence;

USTRUCT()
struct FAnimNextSyncMarkerTraceInfo
{
	GENERATED_BODY()
	
	UPROPERTY()
	float Time = 0.0f;

	UPROPERTY()
	FName Name;
};

USTRUCT()
struct FAnimNextSequenceTraceInfo
{
	GENERATED_BODY()

	FORCEINLINE float CalcAnimTimeRatio() const
	{
		return DurationSeconds > 0.0f ? CurrentTimeSeconds / DurationSeconds : 0.0f;
	}

	UPROPERTY(VisibleAnywhere, Category = Properties)
	TWeakObjectPtr<const UAnimSequence> AnimSequence;

	UPROPERTY(VisibleAnywhere, Category = Properties)
	float DurationSeconds = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = Properties)
	float CurrentTimeSeconds = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = Properties)
	TArray<FAnimNextSyncMarkerTraceInfo> SyncMarkers;
};

UCLASS()
class USequenceInfoDetailsObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Details, meta=(ShowOnlyInnerProperties, FullyExpand=true))
	TArray<FAnimNextSequenceTraceInfo> SequenceTraceInfo;
};

namespace UE::UAF::Editor
{
class FAnimNextSequenceTraceInfoCustomization : public IPropertyTypeCustomization
{
public:
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,
		class FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
		class IDetailChildrenBuilder& StructBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};

class FSequenceInfoTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FSequenceInfoTrack(uint64 InObjectId);
	FSequenceInfoTrack(uint64 InObjectId, uint64 InstanceId);
	virtual ~FSequenceInfoTrack();
	
	TSharedPtr<SEventTimelineView::FTimelineEventData> GetExistenceRange() const { return ExistenceRange; }

	static constexpr const TCHAR* TrackName = TEXT("SequenceInfoTrack");
	
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override { return DetailsView; }

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return TrackName; }
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	void Initialize();
	USequenceInfoDetailsObject* InitializeDetailsObject();

	static void RefreshSequenceInfoFromEvaluationProgram(TArray<FAnimNextSequenceTraceInfo>& OutSequenceInfo, const FSerializableEvaluationProgram& Program);

	TSharedPtr<IDetailsView> DetailsView;
	FSlateIcon Icon;
	uint64 ObjectId;
	uint64 InstanceId = 0; 
	double PreviousScrubTime = -1.0;
	TWeakObjectPtr<USequenceInfoDetailsObject> DetailsObjectWeakPtr;
	TSharedPtr<SEventTimelineView::FTimelineEventData> ExistenceRange;

	// TArray<TSharedPtr<FEvaluationProgramTrack>> Children;
};

#if WITH_ENGINE
class FSequenceInfoTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override { return FSequenceInfoTrack::TrackName; }
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}