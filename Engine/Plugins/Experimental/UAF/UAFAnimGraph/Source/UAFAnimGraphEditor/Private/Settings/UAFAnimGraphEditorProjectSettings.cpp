// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimGraphEditorProjectSettings.h"

#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAnimGraphEditorProjectSettings)

void UUAFAnimGraphEditorProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		// Sets the CVar for MaxGraphUpdateLoopCount to the value of the project setting 
		if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("UAF.MaxGraphUpdateLoopCount")))
		{
			CVar->Set(MaxGraphUpdateLoopCount, ECVF_SetByProjectSetting);
		}
		else
		{
			UE_LOGF(LogAnimation, Error, "Could not find console variable UAF.MaxGraphUpdateLoopCount");
		}
	}
#endif
}

void UUAFAnimGraphEditorProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
