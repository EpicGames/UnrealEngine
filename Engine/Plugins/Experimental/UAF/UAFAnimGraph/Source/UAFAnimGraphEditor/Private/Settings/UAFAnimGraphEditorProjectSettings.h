// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"

#include "UAFAnimGraphEditorProjectSettings.generated.h"

/**
 * UAF Anim Graph Editor Settings that are shared across a project.
 */
UCLASS(MinimalAPI, Config = Editor, defaultconfig, DisplayName = "UAF Anim Graph Settings")
class UUAFAnimGraphEditorProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	UAFANIMGRAPHEDITOR_API virtual void PostInitProperties() override;
	UAFANIMGRAPHEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
public:
	/** The maximum amount of update loops per frame before a graph is considered to be stuck in an endless loop and breaks out of the update 
	 *  This setting is linked with the CVar UAF.MaxGraphUpdateLoopCount  */
	UPROPERTY(Config, EditAnywhere, Category="UAF|Graph", meta=(ConsoleVariable="UAF.MaxGraphUpdateLoopCount"))
	int32 MaxGraphUpdateLoopCount = 1000;
};
