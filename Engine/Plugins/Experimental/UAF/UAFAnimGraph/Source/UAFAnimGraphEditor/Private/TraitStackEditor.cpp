// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitStackEditor.h"

#include "AnimNextController.h"
#include "AnimNextEdGraphNode.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "IWorkspaceEditor.h"
#include "PropertyHandle.h"
#include "SourceCodeNavigation.h"
#include "SPositiveActionButton.h"
#include "SSimpleComboButton.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Graph/TraitEditorTabSummoner.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "TraitCore/TraitInterfaceRegistry.h"
#include "TraitCore/TraitRegistry.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "TraitStackEditor"

namespace UE::UAF::Editor
{

void FTraitStackEditor::SetTraitData(const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor, const FTraitStackData& InTraitStackData)
{
	if (TSharedPtr<SDockTab> DockTab = InWorkspaceEditor->GetTabManager()->FindExistingLiveTab(FTabId(TraitEditorTabName)))
	{
		if (TSharedPtr<STraitEditorView> TraitEditorView = StaticCastSharedPtr<STraitEditorView>(DockTab->GetContent().ToSharedPtr()))
		{
			TraitEditorView->SetTraitData(InTraitStackData);
		}
	}
}

namespace Private
{
	class FTraitStackDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FTraitStackDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FTraitStackDragDropOp> New(const TSharedRef<IPropertyHandle>& InPropertyHandle, FText InDisplayName, FVector2f InSize)
		{
			TSharedRef<FTraitStackDragDropOp> Operation = MakeShared<FTraitStackDragDropOp>();
			Operation->WeakPropertyHandle = InPropertyHandle;
			Operation->DisplayName = InDisplayName;
			Operation->Size = FVector2f(FMath::Min(InSize.X, 300.0f), InSize.Y);
			Operation->Construct();
			return Operation;
		}

		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return
				SNew(SBox)
				.WidthOverride(Size.X)
				.HeightOverride(Size.Y)
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.White"))
						.BorderBackgroundColor(FStyleColors::Select.GetSpecifiedColor())
					]
					+SOverlay::Slot()
					.Padding(1.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
						.Content()
						[
							SNew(SBox)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							.Padding(3.0f)
							[
								SNew(STextBlock)
								.Text(DisplayName)
								.Font(IDetailLayoutBuilder::GetDetailFontBold())
							]
						]
					]
				];
		}

		TWeakPtr<IPropertyHandle> WeakPropertyHandle;
		FText DisplayName;
		FVector2f Size;
	};

	class STraitHeader : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(STraitHeader) {}

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, IDetailCategoryBuilder& InCategory, IDetailPropertyRow& InPropertyRow, URigVMController* InController);

		/** Retrieves a brush for rendering a drop indicator for the specified drop zone */
		static const FSlateBrush* GetDropIndicatorBrush(EItemDropZone InItemDropZone)
		{
			const FTableRowStyle& TreeViewStyle = FAppStyle::GetWidgetStyle<FTableRowStyle>("DetailsView.TreeView.TableRow");
			switch (InItemDropZone)
			{
			case EItemDropZone::AboveItem: return &TreeViewStyle.DropIndicator_Above;
			case EItemDropZone::BelowItem: return &TreeViewStyle.DropIndicator_Below;
			default: return nullptr;
			};
		}
		
		virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
		{
			LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

			if (ItemDropZone.IsSet())
			{
				// Draw feedback for user dropping an item above, below, or onto a row.
				const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(ItemDropZone.GetValue());

				FSlateDrawElement::MakeBox
				(
					OutDrawElements,
					LayerId++,
					AllottedGeometry.ToPaintGeometry(),
					DropIndicatorBrush,
					ESlateDrawEffect::None,
					DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
				);
			}

			Geometry = AllottedGeometry;
			return LayerId;
		}

		/** @return the zone (above, below, none) based on where the user is hovering over within the row */
		static TOptional<EItemDropZone> ZoneFromPointerPosition(UE::Slate::FDeprecateVector2DParameter LocalPointerPos, UE::Slate::FDeprecateVector2DParameter LocalSize)
		{
			const float ZoneBoundarySu = FMath::Clamp(LocalSize.Y * 0.25f, 3.0f, 10.0f);
			if (LocalPointerPos.Y < ZoneBoundarySu)
			{
				return EItemDropZone::AboveItem;
			}
			else if (LocalPointerPos.Y > LocalSize.Y - ZoneBoundarySu)
			{
				return EItemDropZone::BelowItem;
			}
			else
			{
				return TOptional<EItemDropZone>();
			}
		}

		virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			TSharedPtr<FTraitStackDragDropOp> DragOp = DragDropEvent.GetOperationAs<FTraitStackDragDropOp>();
			if (!DragOp.IsValid())
			{
				return;
			}

			const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			ItemDropZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize());
		}

		virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override
		{
			ItemDropZone = TOptional<EItemDropZone>();
		}

		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			TSharedPtr<FTraitStackDragDropOp> DragOp = DragDropEvent.GetOperationAs<FTraitStackDragDropOp>();
			if (!DragOp.IsValid())
			{
				return FReply::Unhandled();
			}

			if (DragOp->WeakPropertyHandle == WeakPropertyHandle)
			{
				ItemDropZone.Reset();
				return FReply::Handled();
			}
			
			const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			ItemDropZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize());

			return FReply::Handled();
		}

		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			ON_SCOPE_EXIT { ItemDropZone = TOptional<EItemDropZone>(); };
		
			TSharedPtr<FTraitStackDragDropOp> DragOp = DragDropEvent.GetOperationAs<FTraitStackDragDropOp>();
			if (!DragOp.IsValid())
			{
				return FReply::Unhandled();
			}
			
			UAnimNextController* Controller = Cast<UAnimNextController>(WeakController.Get());
			if (Controller == nullptr)
			{
				return FReply::Unhandled();
			}
			
			TSharedPtr<IPropertyHandle> SourceHandle = DragOp->WeakPropertyHandle.Pin();
			TSharedPtr<IPropertyHandle> TargetHandle = WeakPropertyHandle.Pin();

			if (!SourceHandle.IsValid() || !TargetHandle.IsValid())
			{
				return FReply::Unhandled();
			}

			TSharedPtr<IPropertyHandleStruct> SourceStructHandle = SourceHandle->AsStruct();
			TSharedPtr<IPropertyHandleStruct> TargetStructHandle = TargetHandle->AsStruct();
			check(SourceStructHandle.IsValid() && TargetStructHandle.IsValid());

			TSharedPtr<FStructOnScope> SourceStructData = SourceStructHandle->GetStructData();
			TSharedPtr<FStructOnScope> TargetStructData = TargetStructHandle->GetStructData();

			FRigVMControllerCompileBracketScope CompileBracketScope(Controller);
			Controller->OpenUndoBracket(LOCTEXT("ReorderTraitsTransaction", "Reorder Traits").ToString());

			for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
			{
				if (UAnimNextEdGraphNode* Node = WeakNode.Get())
				{
					TArray<URigVMPin*> TraitPins = Node->GetModelNode()->GetTraitPins();
					URigVMPin* SourcePin = nullptr;
					int32 SourcePinIndex = INDEX_NONE;
					for (URigVMPin* Pin : TraitPins)
					{
						TSharedPtr<FStructOnScope> TraitScope = Pin->GetTraitInstance();
						if (TraitScope.IsValid())
						{
							const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
							if (VMTrait->GetTraitSharedDataStruct() == SourceStructData->GetStruct())
							{
								SourcePin = Pin;
								SourcePinIndex = Pin->GetPinIndex();
								break;
							}
						}
					}

					if (SourcePin == nullptr)
					{
						continue;
					}

					int32 TargetPinIndex = INDEX_NONE;
					for (URigVMPin* Pin : TraitPins)
					{
						TSharedPtr<FStructOnScope> TraitScope = Pin->GetTraitInstance();
						if (TraitScope.IsValid())
						{
							const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
							if (VMTrait->GetTraitSharedDataStruct() == TargetStructData->GetStruct())
							{
								TargetPinIndex = Pin->GetPinIndex();

								if (ItemDropZone.IsSet() && ItemDropZone.GetValue() == EItemDropZone::BelowItem)
								{
									TargetPinIndex = FMath::Max(0, TargetPinIndex + 1);
								}

								if (SourcePinIndex != TargetPinIndex)
								{
									FRigVMNodeLayout NodeLayout = Node->GetModelNode()->GetNodeLayout(true);
									Controller->SetTraitPinIndex(Node->GetModelNodeName(), SourcePin->GetFName(), TargetPinIndex, true, true);
									if(NodeLayout.IsValid())
									{
										Controller->SetNodeLayout(Node->GetModelNodeName(), NodeLayout, true, true);
									}
								}
								break;
							}
						}
					}
				}
			}

			Controller->CloseUndoBracket();

			return FReply::Handled();
		}

		mutable FGeometry Geometry;
		TOptional<EItemDropZone> ItemDropZone;
		TWeakPtr<IPropertyHandle> WeakPropertyHandle;
		TWeakObjectPtr<URigVMController> WeakController;
		TArray<TWeakObjectPtr<UAnimNextEdGraphNode>> Nodes;
		const UScriptStruct* SharedDataStruct; 
	};

	class FStructFilter : public IStructViewerFilter
	{
	public:
		FStructFilter(TConstArrayView<TWeakObjectPtr<UAnimNextEdGraphNode>> InNodes, ETraitMode InMode)
			: Nodes(InNodes)
			, Mode(InMode)
		{
		}
		
		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			if (InStruct->HasMetaData(TEXT("Hidden")))
			{
				return false;
			}

			if (!InStruct->IsChildOf(FAnimNextTraitSharedData::StaticStruct()))
			{
				return false;
			}
	
			const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
			const FTrait* Trait = TraitRegistry.Find(InStruct);
			if (Trait == nullptr)
			{
				return false;
			}

			if (Trait->GetTraitMode() != Mode)
			{
				return false;
			}

			// Check for duplicates unless they are allowed
			if (InStruct->GetMetaData(TEXT("AllowDuplicates")) != TEXT("true"))
			{
				for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
				{
					if (UAnimNextEdGraphNode* Node = WeakNode.Get())
					{
						TArray<URigVMPin*> TraitPins = Node->GetModelNode()->GetTraitPins();
						for (URigVMPin* Pin : TraitPins)
						{
							TSharedPtr<FStructOnScope> TraitScope = Pin->GetTraitInstance();
							if (TraitScope.IsValid())
							{
								const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
								if (VMTrait->GetTraitSharedDataStruct() == InStruct)
								{
									return false;
								}
							}
						}
					}
				}
			}

			return true;
		}

		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs)
		{
			return false;
		};

		TConstArrayView<TWeakObjectPtr<UAnimNextEdGraphNode>> Nodes;
		ETraitMode Mode;
	};

	class SDragHandle : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SDragHandle) {}

		SLATE_ARGUMENT(FText, DisplayName)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<STraitHeader>& InContainerWidget)
		{
			DisplayName = InArgs._DisplayName;
			WeakPropertyHandle = InPropertyHandle;
			WeakContainerWidget = InContainerWidget;

			ChildSlot
			[
				SNew(SHorizontalBox)
				.Cursor(EMouseCursor::GrabHand)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Visibility_Lambda([this]()
					{
						if( const TSharedPtr<SWidget> Widget = WeakContainerWidget.Pin())
						{
							return Widget->IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
						}
						return EVisibility::Hidden;
					})
					.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3.0f)
				[
					SNew(STextBlock)
					.Text(DisplayName)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			];
		}

		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (!MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				return FReply::Unhandled();
			}

			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		};
		
		virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (!MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				return FReply::Unhandled();
			}

			TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
			if (!PropertyHandle.IsValid())
			{
				return FReply::Unhandled();
			}

			TSharedPtr<STraitHeader> ContainerWidget = WeakContainerWidget.Pin();
			if (!ContainerWidget.IsValid())
			{
				return FReply::Unhandled();
			}

			TSharedPtr<FDragDropOperation> DragDropOp = FTraitStackDragDropOp::New(PropertyHandle.ToSharedRef(), DisplayName, ContainerWidget->Geometry.GetLocalSize());
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}

		TWeakPtr<IPropertyHandle> WeakPropertyHandle;
		TWeakPtr<STraitHeader> WeakContainerWidget;
		FText DisplayName;
	};
	
	void STraitHeader::Construct(const FArguments& InArgs, IDetailCategoryBuilder& InCategory, IDetailPropertyRow& InPropertyRow, URigVMController* InController)
	{
		WeakPropertyHandle = InPropertyRow.GetPropertyHandle();
		WeakController = InController;
		Nodes = InCategory.GetParentLayout().GetObjectsOfTypeBeingCustomized<UAnimNextEdGraphNode>();
		
		TSharedPtr<IPropertyHandleStruct> StructHandle = InPropertyRow.GetPropertyHandle()->AsStruct();
		check(StructHandle.IsValid());

		SharedDataStruct = Cast<UScriptStruct>(StructHandle->GetStructData()->GetStruct());
		check(SharedDataStruct);

		TSharedRef<Private::SDragHandle> HandleWidget =
			SNew(Private::SDragHandle, InPropertyRow.GetPropertyHandle().ToSharedRef(), SharedThis(this))
			.DisplayName(InCategory.GetDisplayName());

		TSharedRef<SWidget> Widget =
			SNew(SOverlay)
			.ToolTipText_Lambda([this]()->FText
			{
				if (const FTrait* Trait = FTraitRegistry::Get().Find(SharedDataStruct))
				{
					static const FTextFormat EntryFormat(LOCTEXT("InterfaceListEntryFormat", "- {0}"));

					FTextBuilder TraitInfoText;
					TraitInfoText.AppendLine(SharedDataStruct->GetDisplayNameText());
					TraitInfoText.AppendLine(FText::GetEmpty());
					TraitInfoText.AppendLine(SharedDataStruct->GetToolTipText());

					if (Trait->GetTraitInterfaces().Num() > 0)
					{
						TraitInfoText.AppendLine(FText::GetEmpty());
						TraitInfoText.AppendLine(LOCTEXT("TraitInfoImplementedInterfaces", "Implements:"));
						for (const FTraitInterfaceUID& ImplementedInterfaceUID : Trait->GetTraitInterfaces())
						{
							if (const ITraitInterface* ImplementedInterface = FTraitInterfaceRegistry::Get().Find(ImplementedInterfaceUID))
							{
								TraitInfoText.AppendLineFormat(EntryFormat, ImplementedInterface->GetDisplayName());
							}
						}
					}

					if (Trait->GetTraitRequiredInterfaces().Num() > 0)
					{
						TraitInfoText.AppendLine(FText::GetEmpty());
						TraitInfoText.AppendLine(LOCTEXT("TraitInfoRequiredInterfaces", "Requires:").ToString());
						for (const FTraitInterfaceUID& RequiredInterfaceUID : Trait->GetTraitRequiredInterfaces())
						{
							if (const ITraitInterface* RequiredInterface = FTraitInterfaceRegistry::Get().Find(RequiredInterfaceUID))
							{
								TraitInfoText.AppendLineFormat(EntryFormat, RequiredInterface->GetDisplayName());
							}
						}
					}

					return TraitInfoText.ToText();
				}
				return FText();
			})
			.Visibility(EVisibility::Visible)
			+SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					HandleWidget
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.FillContentWidth(1.0f, 0.0f) 
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
					.Visibility_Lambda([this]() -> EVisibility
						{
							if (FSourceCodeNavigation::CanNavigateToStruct(SharedDataStruct))
							{
								return EVisibility::Visible;
							}
							return EVisibility::Collapsed;
						})
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked_Lambda([this]()
							{
								if (SharedDataStruct)
								{
									FSourceCodeNavigation::NavigateToStruct(SharedDataStruct);
								}
								return FReply::Handled();
							})
						.ToolTipText(FText::Format(LOCTEXT("GoToCode_ToolTip", "Click to open the trait shared data source file in {0}"), FSourceCodeNavigation::GetSelectedSourceCodeIDE()))
						.ContentPadding(0.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.OpenSourceLocation"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(3.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
						.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentYellow"))
						.ToolTipText(LOCTEXT("DeprecatedWarning", "This trait is deprecated. Please remove, replace or upgrade it to remove this warning."))
						.Visibility_Lambda([this]()
						{
							TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
							if (!PropertyHandle.IsValid())
							{
								return EVisibility::Collapsed;
							}

							TSharedPtr<IPropertyHandleStruct> StructHandle = PropertyHandle->AsStruct();
							check(StructHandle.IsValid());

							TSharedPtr<FStructOnScope> StructData = StructHandle->GetStructData();
							return StructData.IsValid() && StructData->GetStruct() && StructData->GetStruct()->HasMetaData(TEXT("Deprecated")) ? EVisibility::Visible : EVisibility::Collapsed;
						})
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SSimpleComboButton)
						.HasDownArrow(true)
						.OnGetMenuContent_Lambda([this]()
						{
							constexpr bool bShouldCloseWindowAfterMenuSelection = true;
							FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

							MenuBuilder.AddSubMenu(
								LOCTEXT("AddLabel", "Add..."),
								LOCTEXT("AddTooltip", "Add a new trait after this one"),
								FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder)
								{
									FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

									FStructViewerInitializationOptions Options;
									Options.Mode = EStructViewerMode::StructPicker;
									Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
									Options.StructFilter = MakeShared<FStructFilter>(Nodes, ETraitMode::Additive);

									InMenuBuilder.AddWidget(
										SNew(SBox)
										.WidthOverride(300.0f)
										.HeightOverride(300.0f)
										[
											StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([this](const UScriptStruct* InStruct)
											{
												FSlateApplication::Get().DismissAllMenus();

												UAnimNextController* Controller = Cast<UAnimNextController>(WeakController.Get());
												if (Controller == nullptr)
												{
													return;
												}

												TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
												if (!PropertyHandle.IsValid())
												{
													return;
												}

												TSharedPtr<IPropertyHandleStruct> StructHandle = PropertyHandle->AsStruct();
												check(StructHandle.IsValid());

												TSharedPtr<FStructOnScope> StructData = StructHandle->GetStructData();

												TInstancedStruct<FAnimNextTraitSharedData> Defaults;
												Defaults.InitializeAsScriptStruct(InStruct);
												
												FRigVMControllerCompileBracketScope CompileBracketScope(Controller);
												Controller->OpenUndoBracket(LOCTEXT("AddTraitTransaction", "Add Trait").ToString());

												for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
												{
													if (UAnimNextEdGraphNode* Node = WeakNode.Get())
													{
														TArray<URigVMPin*> TraitPins = Node->GetModelNode()->GetTraitPins();
														int32 SourcePinIndex = INDEX_NONE;
														for (URigVMPin* Pin : TraitPins)
														{
															TSharedPtr<FStructOnScope> TraitScope = Pin->GetTraitInstance();
															if (TraitScope.IsValid()) 
															{
																const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
																if (VMTrait->GetTraitSharedDataStruct() == StructData->GetStruct())
																{
																	SourcePinIndex = Pin->GetPinIndex();
																	break;
																}
															}
														}

														FName NewTrait = Controller->AddTraitStruct(CastChecked<URigVMUnitNode>(Node->GetModelNode()), Defaults, FMath::Min(SourcePinIndex + 1, Node->GetModelNode()->GetPins().Num()), true, true);
														if (NewTrait != NAME_None)
														{
															FRigVMNodeLayout NodeLayout = Node->GetModelNode()->GetNodeLayout();
															UUAFGraphNodeTemplate::AddDefaultTraitPinsToLayout(InStruct, NodeLayout);
															Controller->SetNodeLayout(Node->GetModelNodeName(), NodeLayout, true, true);
														}
													}
												}

												Controller->CloseUndoBracket();
											}))
										], FText::GetEmpty(), true);
								}));

							MenuBuilder.AddSubMenu(
								LOCTEXT("ReplaceLabel", "Replace..."),
								LOCTEXT("ReplaceTooltip", "Replace this trait with another"),
								FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder)
								{
									const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();
									const FTrait* Trait = TraitRegistry.Find(SharedDataStruct);
									check(Trait);

									FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

									FStructViewerInitializationOptions Options;
									Options.Mode = EStructViewerMode::StructPicker;
									Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
									Options.StructFilter = MakeShared<FStructFilter>(Nodes, Trait->GetTraitMode());

									InMenuBuilder.AddWidget(
										SNew(SBox)
										.WidthOverride(300.0f)
										.HeightOverride(300.0f)
										[
											StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([this](const UScriptStruct* InStruct)
											{
												FSlateApplication::Get().DismissAllMenus();

												UAnimNextController* Controller = Cast<UAnimNextController>(WeakController.Get());
												if (Controller == nullptr)
												{
													return;
												}

												TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
												if (!PropertyHandle.IsValid())
												{
													return;
												}

												TSharedPtr<IPropertyHandleStruct> StructHandle = PropertyHandle->AsStruct();
												check(StructHandle.IsValid());

												TSharedPtr<FStructOnScope> StructData = StructHandle->GetStructData();

												TInstancedStruct<FAnimNextTraitSharedData> Defaults;
												Defaults.InitializeAsScriptStruct(InStruct);
												
												FRigVMControllerCompileBracketScope CompileBracketScope(Controller);
												Controller->OpenUndoBracket(LOCTEXT("ReplaceTraitTransaction", "Replace Trait").ToString());

												for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
												{
													if (UAnimNextEdGraphNode* Node = WeakNode.Get())
													{
														TArray<URigVMPin*> TraitPins = Node->GetModelNode()->GetTraitPins();
														int32 SourcePinIndex = INDEX_NONE;
														FName SourcePinName = NAME_None;
														for (URigVMPin* Pin : TraitPins)
														{
															TSharedPtr<FStructOnScope> TraitScope = Pin->GetTraitInstance();
															if (TraitScope.IsValid()) 
															{
																const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
																if (VMTrait->GetTraitSharedDataStruct() == StructData->GetStruct())
																{
																	SourcePinIndex = Pin->GetPinIndex();
																	SourcePinName = Pin->GetFName();
																	break;
																}
															}
														}

														if (SourcePinIndex == INDEX_NONE || SourcePinName == NAME_None)
														{
															continue;
														}

														if (Controller->RemoveTraitByName(Node->GetModelNodeName(),SourcePinName, true, true))
														{
															FName NewTrait = Controller->AddTraitStruct(CastChecked<URigVMUnitNode>(Node->GetModelNode()), Defaults, SourcePinIndex, true, true);
															if (NewTrait != NAME_None)
															{
																FRigVMNodeLayout NodeLayout = Node->GetModelNode()->GetNodeLayout();
																UUAFGraphNodeTemplate::AddDefaultTraitPinsToLayout(InStruct, NodeLayout);
																Controller->SetNodeLayout(Node->GetModelNodeName(), NodeLayout, true, true);
															}
														}
													}
												}

												Controller->CloseUndoBracket();
											}))
										], FText::GetEmpty(), true);
								}));

							MenuBuilder.AddMenuEntry(
								LOCTEXT("RemoveLabel", "Remove"),
								LOCTEXT("RemoveTooltip", "Remove this trait"),
								FSlateIcon(),
								FExecuteAction::CreateLambda([this]()
								{
									FSlateApplication::Get().DismissAllMenus();

									UAnimNextController* Controller = Cast<UAnimNextController>(WeakController.Get());
									if (Controller == nullptr)
									{
										return;
									}

									TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
									if (!PropertyHandle.IsValid())
									{
										return;
									}

									TSharedPtr<IPropertyHandleStruct> StructHandle = PropertyHandle->AsStruct();
									check(StructHandle.IsValid());

									TSharedPtr<FStructOnScope> StructData = StructHandle->GetStructData();
									
									FRigVMControllerCompileBracketScope CompileBracketScope(Controller);
									Controller->OpenUndoBracket(LOCTEXT("RemoveTraitTransaction", "Remove Trait").ToString());

									for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
									{
										if (UAnimNextEdGraphNode* Node = WeakNode.Get())
										{
											Controller->RemoveTraitByName(Node->GetModelNodeName(),StructData->GetStruct()->GetFName(), true, true);
										}
									}

									Controller->CloseUndoBracket();
								}));

							TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
							if (PropertyHandle.IsValid())
							{
								TSharedPtr<IPropertyHandleStruct> StructHandle = PropertyHandle->AsStruct();
								check(StructHandle.IsValid());

								TSharedPtr<FStructOnScope> StructData = StructHandle->GetStructData();
								if (StructData.IsValid() && StructData->GetStruct() && StructData->GetStruct()->HasMetaData(TEXT("Deprecated")))
								{
									const FTrait* Trait = FTraitRegistry::Get().Find(CastChecked<UScriptStruct>(StructData->GetStruct()));
									check(Trait);

									FRigVMStructUpgradeInfo UpgradeInfo = Trait->GetUpgradeInfo();
									if (UpgradeInfo.IsValid())
									{
										MenuBuilder.AddMenuEntry(
											LOCTEXT("UpgradeLabel", "Upgrade"),
											LOCTEXT("UpgradeTooltip", "Upgrade this trait"),
											FSlateIcon(),
											FExecuteAction::CreateLambda([this, UpgradeInfo = MoveTemp(UpgradeInfo)]()
											{
												FSlateApplication::Get().DismissAllMenus();

												UAnimNextController* Controller = Cast<UAnimNextController>(WeakController.Get());
												if (Controller == nullptr)
												{
													return;
												}

												TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
												if (!PropertyHandle.IsValid())
												{
													return;
												}

												TSharedPtr<IPropertyHandleStruct> StructHandle = PropertyHandle->AsStruct();
												check(StructHandle.IsValid());

												TSharedPtr<FStructOnScope> StructData = StructHandle->GetStructData();

												FRigVMControllerCompileBracketScope CompileBracketScope(Controller);
												Controller->OpenUndoBracket(LOCTEXT("UpgradeTraitTransaction", "Upgrade Trait").ToString());

												for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
												{
													if (UAnimNextEdGraphNode* Node = WeakNode.Get())
													{
														Controller->UpgradeTraitByName(Node->GetModelNodeName(), StructData->GetStruct()->GetFName(), true, true);
													}
												}

												Controller->CloseUndoBracket();
											}));
									}
								}
							}

							return MenuBuilder.MakeWidget();
						})
					]
				]
			];

		ChildSlot
		[
			Widget
		];
	}
}

TSharedRef<SWidget> FTraitStackEditor::CreateTraitHeaderWidget(IDetailCategoryBuilder& InCategory, IDetailPropertyRow& InPropertyRow, URigVMController* InController)
{
	return SNew(Private::STraitHeader, InCategory, InPropertyRow, InController);
}

TSharedRef<SWidget> FTraitStackEditor::CreateTraitStackHeaderWidget(IDetailCategoryBuilder& InCategory, URigVMController* InController)
{
	TArray<TWeakObjectPtr<UAnimNextEdGraphNode>> Nodes = InCategory.GetParentLayout().GetObjectsOfTypeBeingCustomized<UAnimNextEdGraphNode>();
	
	return
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SPositiveActionButton)
			.Text(LOCTEXT("AddTrait", "Add Trait"))
			.ToolTipText(LOCTEXT("AddTraitTooltip", "Add a new trait to this node"))
			.OnGetMenuContent_Lambda([Nodes = MoveTemp(Nodes), InController]()
			{
				ETraitMode Mode = ETraitMode::Additive;
				bool bAreAllNodesEmpty = true;

				for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
				{
					if (UAnimNextEdGraphNode* Node = WeakNode.Get())
					{

						bAreAllNodesEmpty &= (Node->GetModelNode()->GetTraitPins().Num() == 0);
					}
				}

				if (bAreAllNodesEmpty)
				{
					Mode = ETraitMode::Base;
				}

				FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

				FStructViewerInitializationOptions Options;
				Options.Mode = EStructViewerMode::StructPicker;
				Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
				Options.StructFilter = MakeShared<Private::FStructFilter>(Nodes, Mode);

				return SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(300.0f)
				[
					StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([InController, &Nodes](const UScriptStruct* InStruct)
					{
						FSlateApplication::Get().DismissAllMenus();

						UAnimNextController* Controller = Cast<UAnimNextController>(InController);
						if (Controller == nullptr)
						{
							return;
						}

						TInstancedStruct<FAnimNextTraitSharedData> Defaults;
						Defaults.InitializeAsScriptStruct(InStruct);

						FRigVMControllerCompileBracketScope CompileBracketScope(Controller);
						Controller->OpenUndoBracket(LOCTEXT("AddTraitTransaction", "Add Trait").ToString());

						for (TWeakObjectPtr<UAnimNextEdGraphNode> WeakNode : Nodes)
						{
							if (UAnimNextEdGraphNode* Node = WeakNode.Get())
							{
								FName NewTrait = Controller->AddTraitStruct(CastChecked<URigVMUnitNode>(Node->GetModelNode()), Defaults, Node->GetModelNode()->GetPins().Num(), true, true);
								if (NewTrait != NAME_None)
								{
									FRigVMNodeLayout NodeLayout = Node->GetModelNode()->GetNodeLayout();
									UUAFGraphNodeTemplate::AddDefaultTraitPinsToLayout(InStruct, NodeLayout);
									Controller->SetNodeLayout(Node->GetModelNodeName(), NodeLayout, true, true);
								}
							}
						}

						Controller->CloseUndoBracket();
					}))
				];
			})
		];
}

}

#undef LOCTEXT_NAMESPACE