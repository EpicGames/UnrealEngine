// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITraitStackEditor.h"

class IDetailPropertyRow;
class IDetailCategoryBuilder;
class SWidget;
class URigVMController;

namespace UE::UAF::Editor
{

// Interface used to talk to the trait stack editor embedded in a workspace
class FTraitStackEditor : public ITraitStackEditor
{
	// ITraitStackEditor interface
	virtual void SetTraitData(const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor, const FTraitStackData& InTraitStackData) override;
	virtual TSharedRef<SWidget> CreateTraitHeaderWidget(IDetailCategoryBuilder& InCategory, IDetailPropertyRow& InPropertyRow, URigVMController* InController) override;
	virtual TSharedRef<SWidget> CreateTraitStackHeaderWidget(IDetailCategoryBuilder& InCategory, URigVMController* InController) override;
};

}