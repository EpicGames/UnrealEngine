// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFCallFunctionInfoDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "UncookedOnlyUtils.h"
#include "Traits/CallFunction.h"
#include "Common/SRigVMFunctionPicker.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "ScopedTransaction.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "Modules/ModuleManager.h"
#include "RigVMModel/RigVMController.h"
#include "UAFCompilationScope.h"
#include "Workspace/AnimNextWorkspaceFactory.h"

#define LOCTEXT_NAMESPACE "FCallFunctionInfoDetails"

namespace UE::UAF::Editor
{

void FCallFunctionInfoDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	using namespace UE::UAF::UncookedOnly;

	PropertyHandle = InPropertyHandle;
	FunctionPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFCallFunctionInfo, Function));
	FunctionHeaderPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFCallFunctionInfo, FunctionHeader));
	FunctionEventPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFCallFunctionInfo, FunctionEvent));

	TArray<TWeakObjectPtr<UObject>> Objects = InCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
	UObject* RigVMAssetSubObject = Objects.Num() ? Objects[0].Get() : nullptr;
	UUAFRigVMAsset* CurrentAsset = nullptr;
	if (RigVMAssetSubObject)
	{
		CurrentAsset = RigVMAssetSubObject->GetTypedOuter<UUAFRigVMAsset>();
	}

	auto OnFunctionPicked = [this, WeakCurrentAsset = TWeakObjectPtr<UUAFRigVMAsset>(CurrentAsset)](const FRigVMGraphFunctionHeader& InFunctionHeader)
	{
		UUAFRigVMAsset* CurrentAsset = WeakCurrentAsset.Get();
		if(CurrentAsset == nullptr)
		{
			return;
		}

		FCompilationScope CompilerResults(LOCTEXT("ModifiedAssets_FunctionPicked", "Modified Function Picked"), MakeConstArrayView({ CurrentAsset }));
		FScopedTransaction Transaction(LOCTEXT("SetFunctionTransaction", "Set Function"));

		// Update function event & header from function name
		FunctionPropertyHandle->SetValue(InFunctionHeader.IsValid() ? InFunctionHeader.Name : NAME_None);
		FunctionEventPropertyHandle->SetValue(InFunctionHeader.IsValid() ? FName(*UncookedOnly::FUtils::MakeFunctionWrapperEventName(InFunctionHeader.Name)): NAME_None);

		FunctionHeaderPropertyHandle->NotifyPreChange();
		FunctionHeaderPropertyHandle->EnumerateRawData([&InFunctionHeader](void* InRawData, const int32 InDataIndex, const int32 InNumDatas)
		{
			FRigVMGraphFunctionHeader* FunctionHeader = static_cast<FRigVMGraphFunctionHeader*>(InRawData);
			if(InFunctionHeader.IsValid())
			{
				*FunctionHeader = InFunctionHeader;
			}
			else
			{
				*FunctionHeader = FRigVMGraphFunctionHeader();
			}
			return true;
		});
		FunctionHeaderPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		FunctionHeaderPropertyHandle->NotifyFinishedChangingProperties();
	};
	InHeaderRow
	.NameContent()
	[
		FunctionPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SRigVMFunctionPicker)
		.CurrentAsset(FAssetData(CurrentAsset))
		.FunctionName_Lambda([this]()
		{
			bool bMultipleValues = false;
			TOptional<FRigVMGraphFunctionHeader> Header;
			PropertyHandle->EnumerateConstRawData([&Header, &bMultipleValues](const void* InRawData, const int32 InDataIndex, const int32 InNumDatas)
			{
				const FUAFCallFunctionInfo* FunctionInfo = static_cast<const FUAFCallFunctionInfo*>(InRawData);
				if (FunctionInfo)
				{
					if (!Header.IsSet())
					{
						Header = FunctionInfo->FunctionHeader;
					}
					else if (Header.GetValue() != FunctionInfo->FunctionHeader)
					{
						Header = FRigVMGraphFunctionHeader();
						bMultipleValues = true;
						return false;
					}
				}
				return true;
			});

			if(bMultipleValues)
			{
				return LOCTEXT("MultipleValuesLabel", "Multiple Values");
			}
			else if(Header.IsSet() && Header.GetValue().IsValid())
			{
				return FText::FromName(Header.GetValue().Name);
			}
			return LOCTEXT("NoFunctionSelectedLabel", "None");
		})
		.FunctionToolTip_Lambda([this]()
		{
			bool bMultipleValues = false;
			TOptional<FRigVMGraphFunctionHeader> Header;
			PropertyHandle->EnumerateConstRawData([&Header, &bMultipleValues](const void* InRawData, const int32 InDataIndex, const int32 InNumDatas)
			{
				const FUAFCallFunctionInfo* FunctionInfo = static_cast<const FUAFCallFunctionInfo*>(InRawData);
				if (FunctionInfo)
				{
					if (!Header.IsSet())
					{
						Header = FunctionInfo->FunctionHeader;
					}
					else if (Header.GetValue() != FunctionInfo->FunctionHeader)
					{
						Header = FRigVMGraphFunctionHeader();
						bMultipleValues = true;
						return false;
					}
				}
				return true;
			});

			if(bMultipleValues)
			{
				return LOCTEXT("MultipleValuesLabel", "Multiple Values");
			}
			else if(Header.IsSet() && Header.GetValue().IsValid())
			{
				static const FTextFormat ToolTipFormat(LOCTEXT("FunctionToolTipFormat", "{0}\n{1}"));
				
				return Header.GetValue().Description.Len() > 0 ?
					FText::FromString(Header.GetValue().Description) :
					FText::Format(ToolTipFormat, FText::FromString(Header.GetValue().LibraryPointer.GetFunctionName()), FText::FromString(Header.GetValue().LibraryPointer.GetLibraryNodePath()));
			}
			return LOCTEXT("NoFunctionSelectedLabel", "None");
		})
		.OnRigVMFunctionPicked_Lambda(OnFunctionPicked)
		.OnNewFunction_Lambda([this, WeakCurrentAsset = TWeakObjectPtr<UUAFRigVMAsset>(CurrentAsset), OnFunctionPicked]()
		{
			UUAFRigVMAsset* CurrentAsset = WeakCurrentAsset.Get();
			if (CurrentAsset == nullptr)
			{
				return;
			}

			UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(CurrentAsset);
			URigVMLibraryNode* NewFunction = nullptr;
			{
				FCompilationScope CompilerResults(LOCTEXT("ModifiedAssets_FunctionAdded", "Modified Asset Added Function"), MakeConstArrayView({ CurrentAsset }));
				FScopedTransaction Transaction(LOCTEXT("AddFunctionTransaction", "Add Function"));
				NewFunction = EditorData->AddFunction(TEXT("NewFunction"), true, true, true);

				if(NewFunction == nullptr)
				{
					return;
				}

				const FRigVMGraphFunctionData* FunctionData = EditorData->GraphFunctionStore.FindFunction(NewFunction->GetFunctionIdentifier());
				if(FunctionData == nullptr)
				{
					return;
				}

				// Set our function
				OnFunctionPicked(FunctionData->Header);
			}

			// Open the new function's graph
			UE::Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<UE::Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
			if(UE::Workspace::IWorkspaceEditor* WorkspaceEditor = WorkspaceEditorModule.OpenWorkspaceForObject(CurrentAsset, UE::Workspace::EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass()))
			{
				UObject* EditorObject = EditorData->GetEditorObjectForRigVMGraph(NewFunction->GetContainedGraph());
				WorkspaceEditor->OpenObjects({EditorObject});
			}
		})
	];
}

void FCallFunctionInfoDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InChildBuilder.AddProperty(FunctionPropertyHandle.ToSharedRef())
	.Visibility(EVisibility::Collapsed);

	InChildBuilder.AddProperty(FunctionHeaderPropertyHandle.ToSharedRef())
	.Visibility(EVisibility::Collapsed);

	InChildBuilder.AddProperty(FunctionEventPropertyHandle.ToSharedRef())
	.Visibility(EVisibility::Collapsed);
}

}

#undef LOCTEXT_NAMESPACE