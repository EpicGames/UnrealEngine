// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFAnimGraphEditor : ModuleRules
	{
		public UAFAnimGraphEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"UnrealEd",
					"InputCore",
					"SlateCore",
					"Slate",
					"UAF", 
					"UAFEditor", 
					"UAFUncookedOnly",
					"UAFAnimGraph",
					"UAFAnimGraphUncookedOnly",
					"Settings",
					"StructUtilsEditor",
					"WorkspaceEditor", 
					"EditorWidgets", 
					"MessageLog", 
					"AssetDefinition",
					"RigVM",
					"RigVMDeveloper",
					"ToolMenus", 
					"RigVMEditor",
					"GraphEditor",
					"Persona",
					"PropertyEditor",
					"TraceAnalysis",
					"TraceLog",
					"TraceServices",
					"TraceInsights",
					"RewindDebuggerInterface",
					"GameplayInsights",
					"BlueprintGraph",
					"ToolWidgets",
					"Kismet",
					"DeveloperSettings",
				}
			);
		}
	}
}