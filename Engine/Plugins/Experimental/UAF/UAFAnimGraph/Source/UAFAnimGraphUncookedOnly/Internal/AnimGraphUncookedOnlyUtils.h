// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

class URigVMNode;
class UAnimNextController;
class URigVMController;
class UUAFRigVMAssetEditorData;
struct FAnimNextAssetRegistryExports;

// Gets a 'pin path' (e.g. MyTrait.MyMember from GET_PIN_PATH_STRING_CHECKED(FMyTrait, MyMember)) while statically verifying that the
// struct amen member exist
#define GET_PIN_PATH_STRING_CHECKED(StructName, MemberName) \
	((void)sizeof(UEAsserts_Private::GetMemberNameCheckedJunk(((StructName*)0)->MemberName)), StructName::StaticStruct()->GetName() + TEXT("." #MemberName))

namespace UE::UAF::UncookedOnly
{

struct FAnimGraphUtils
{
	/** Set up a simple animation graph */
	static UE_API void SetupAnimGraph(const FName EntryName, URigVMController* InController, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	/** Check whether the supplied node is a trait stack */
	static UE_API bool IsTraitStackNode(const URigVMNode* InModelNode);

	static UE_API bool RequestVMAutoRecompile(UUAFRigVMAssetEditorData* EditorData);
};

}

#undef UE_API
