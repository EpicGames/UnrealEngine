// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Graph/AnimNextAnimationGraph.h"

#include "AnimNextAnimGraphWorkspaceAssetUserData.generated.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

USTRUCT()
struct FAnimNextAnimationGraphOutlinerData : public FAnimNextRigVMAssetOutlinerData
{
	GENERATED_BODY()

	FAnimNextAnimationGraphOutlinerData() = default;

	UUAFAnimGraph* GetAnimationGraph() const
	{
		return Cast<UUAFAnimGraph>(GetAsset());
	}
};

UCLASS(MinimalAPI)
class UAnimNextAnimGraphWorkspaceAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:

	virtual bool IsEditorOnly() const override { return true; }

protected:

	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	/** Get root asset workspace exports. */
	UE_API virtual void GetRootAssetExport(FAssetRegistryTagsContext Context) const;

	/** Get all workspace asset exports except for the root asset. */
	UE_API virtual void GetWorkspaceAssetExports(FAssetRegistryTagsContext Context) const;

protected:

	mutable FWorkspaceOutlinerItemExports CachedExports;
};

#undef UE_API