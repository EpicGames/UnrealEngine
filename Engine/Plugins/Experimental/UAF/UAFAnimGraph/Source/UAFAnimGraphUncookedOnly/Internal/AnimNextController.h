// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextControllerBase.h"
#include "TraitCore/TraitSharedData.h"
#include "AnimNextController.generated.h"

namespace UE::UAF
{
	struct FTrait;
	struct FTraitUID;
}

/**
  * Implements AnimNext RigVM controller extensions
  */
UCLASS(MinimalAPI)
class UAnimNextController : public UAnimNextControllerBase
{
	GENERATED_BODY()

public:

	// Adds a new Trait to the Stack, with default struct values
	// Returns Trait Instance Name (or NAME_None on failure)
	UFUNCTION(BlueprintCallable, Category = "UAF|Controller")
	UAFANIMGRAPHUNCOOKEDONLY_API FName AddTraitStruct(URigVMUnitNode* InNode, const TInstancedStruct<FAnimNextTraitSharedData>& InTraitDefaults, int32 InTraitIndex, bool bSetupUndoRedo, bool bPrintPythonCommand);

	// Adds a new Trait to the Stack, with default struct values
	// Returns Trait Instance Name (or NAME_None on failure)
	UFUNCTION(BlueprintCallable, Category = "UAF|Controller")
	UAFANIMGRAPHUNCOOKEDONLY_API FName AddTraitByName(FName InNodeName, FName InNewTraitTypeName, int32 InPinIndex, const FString& InNewTraitDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes a Trait from the Stack, using Trait Instance Name
	// Returns operation success (true) or failure (false)
	UFUNCTION(BlueprintCallable, Category = "UAF|Controller")
	UAFANIMGRAPHUNCOOKEDONLY_API bool RemoveTraitByName(FName InNodeName, FName InTraitInstanceName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Swap a Trait from the Stack with a new one, using existing Trait Instance Name and new Trait Type Name
	// Returns Trait Instance Name (or NAME_None on failure)
	UFUNCTION(BlueprintCallable, Category = "UAF|Controller")
	UAFANIMGRAPHUNCOOKEDONLY_API FName SwapTraitByName(FName InNodeName, FName InTraitInstanceName, int32 InCurrentTraitPinIndex, FName InNewTraitTypeName, const FString& InNewTraitDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Move a Trait from its current PinIndex to the specified one (moving it visually in the stack)
	// Returns operation success (true) or failure (false)
	UFUNCTION(BlueprintCallable, Category = "UAF|Controller")
	UAFANIMGRAPHUNCOOKEDONLY_API bool SetTraitPinIndex(FName InNodeName, FName InTraitInstanceName, int32 InNewPinIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Upgrade a trait if it needs to be
	UAFANIMGRAPHUNCOOKEDONLY_API bool UpgradeTraitByName(FName InNodeName, FName InTraitInstanceName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a unit node with a dynamic number of pins
	URigVMUnitNode* AddUnitNodeWithPins(UScriptStruct* InScriptStruct, const FRigVMPinInfoArray& PinArray, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
};
