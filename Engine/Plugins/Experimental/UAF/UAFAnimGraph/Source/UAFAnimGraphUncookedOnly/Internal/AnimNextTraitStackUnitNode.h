// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextUnitNode.h"
#include "Styling/SlateBrush.h"
#include "AnimNextTraitStackUnitNode.generated.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

class UUAFGraphNodeTemplate;
class UUAFAnimGraph_EditorData;
class UAnimNextController;

namespace UE::UAF::Editor
{
	struct FAnimationGraphMenuExtensions;
}

/**
  * Implements AnimNext RigVM unit node extensions for Trait Stacks
  */
UCLASS(MinimalAPI)
class UAnimNextTraitStackUnitNode : public URigVMUnitNode
{
	GENERATED_BODY()

public:
	// Override node functions
	UE_API virtual FString GetNodeSubTitle() const override;
	UE_API virtual FText GetToolTipText() const override;

	// Wrap the template for defaults
	UE_API FString GetDefaultNodeTitle() const;
	UE_API FLinearColor GetDefaultNodeColor() const;
	UE_API const FSlateBrush* GetDefaultNodeIconBrush() const;

private:
	friend UUAFGraphNodeTemplate;
	friend UUAFAnimGraph_EditorData;
	friend UE::UAF::Editor::FAnimationGraphMenuExtensions;

	void HandlePinDefaultValueChanged(UAnimNextController* InController, URigVMPin* InPin);
	void HandleAssetRenamed(UAnimNextController* InController, const FAssetData& InAssetData, const FString& InOldName);

	// The template that defines our behavior
	UPROPERTY(VisibleAnywhere, Category = "Template")
	TSubclassOf<UUAFGraphNodeTemplate> Template;
};

#undef UE_API