// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "IAnimNextRigVMExportInterface.h"
#include "IAnimNextRigVMGraphInterface.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AnimNextAnimationGraphEntry.generated.h"

class UUAFRigVMAssetEditorData;
class UAnimNextEdGraph;
enum class ERigVMGraphNotifType : uint8;

namespace UE::UAF::Editor
{
	struct FUtils;
	class FAnimNextAnimGraphEditorModule;
}

namespace UE::UAF::Tests
{
	class FEditor_Graphs;
	class FEditor_AnimGraph_Graphs;
}


/** An Animation Graph entry in an AnimNext module asset */
UCLASS(MinimalAPI, Category = "Animation Graphs", DisplayName = "Animation Graph")
class UAnimNextAnimationGraphEntry : public UUAFRigVMAssetEntry, public IUAFRigVMGraphInterface, public IUAFRigVMExportInterface
{
	GENERATED_BODY()

	friend class UUAFAnimGraph_EditorData;
	friend struct UE::UAF::Editor::FUtils;
	friend class UE::UAF::Editor::FAnimNextAnimGraphEditorModule;
	friend class UE::UAF::Tests::FEditor_Graphs;
	friend class UE::UAF::Tests::FEditor_AnimGraph_Graphs;

	// IAnimNextRigVMExportInterface interface
	virtual FAnimNextParamType GetExportType() const override;
	virtual FName GetExportName() const override;
	virtual EAnimNextExportAccessSpecifier GetExportAccessSpecifier() const override;
	virtual void SetExportAccessSpecifier(EAnimNextExportAccessSpecifier InAccessSpecifier, bool bSetupUndoRedo = true) override;

	// UUAFRigVMAssetEntry interface
	virtual FName GetEntryName() const override;
	virtual void SetEntryName(FName InName, bool bSetupUndoRedo = true) override;

	// IAnimNextRigVMGraphInterface interface
	virtual const FName& GetGraphName() const override;
	virtual URigVMGraph* GetRigVMGraph() const override;
	virtual URigVMEdGraph* GetEdGraph() const override;
	virtual void SetRigVMGraph(URigVMGraph* InGraph) override;
	virtual void SetEdGraph(URigVMEdGraph* InGraph) override;

protected:
	/** Access specifier - whether the graph's entry point is visible external to this asset */
	UPROPERTY(EditAnywhere, Category = AnimationGraph)
	EAnimNextExportAccessSpecifier Access = EAnimNextExportAccessSpecifier::Private;

	/** The name of the graph */
	UPROPERTY(VisibleAnywhere, Category = AnimationGraph)
	FName GraphName;

	/** RigVM graph */
	UPROPERTY()
	TObjectPtr<URigVMGraph> Graph;

	/** Editor graph */
	UPROPERTY()
	TObjectPtr<UAnimNextEdGraph> EdGraph;
};
