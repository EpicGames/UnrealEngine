// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_CustomProperty.h"
#include "Graph/AnimGraph/AnimNode_AnimNextGraph.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "AnimGraphNode_AnimNextGraph.generated.h"

struct FVariableMappingInfo;

UCLASS(MinimalAPI)
class UAnimGraphNode_AnimNextGraph : public UAnimGraphNode_CustomProperty
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_AnimNextGraph Node;

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostReconstructNode() override;
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;

private:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	virtual void PreloadRequiredAssets() override;
	//virtual bool IsSinkNode() const { return true; }

	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() override { return &Node; }
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const override { return &Node; }

	// property related things
	void GetVariables(bool bInput, TMap<FName, FRigVMExternalVariable>& OutParameters) const;

	TMap<FName, FRigVMExternalVariable> InputVariables;
	TMap<FName, FRigVMExternalVariable> OutputVariables;

	void RebuildExposedProperties();
	virtual void CreateCustomPins(TArray<UEdGraphPin*>* OldPins) override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	
	// pin option related
	bool IsPropertyExposeEnabled(FName PropertyName) const;
	ECheckBoxState IsPropertyExposed(FName PropertyName) const;
	void OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName);

	bool IsInputProperty(const FName& PropertyName) const;

	virtual UObject* GetJumpTargetForDoubleClick() const override;
};

