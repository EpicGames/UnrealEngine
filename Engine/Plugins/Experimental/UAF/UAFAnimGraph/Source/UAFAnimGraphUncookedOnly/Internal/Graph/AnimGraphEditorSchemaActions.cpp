// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphEditorSchemaActions.h"
#include "EditorUtils.h"
#include "EdGraphNode_Comment.h"
#include "Editor.h"
#include "Misc/StringOutputDevice.h"
#include "Module/AnimNextModule_EditorData.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "EdGraphSchema_K2.h"
#include "GraphEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "UncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "AnimNextTraitStackUnitNode.h"
#include "Editor/RigVMEditorTools.h"
#include "Exporters/Exporter.h"
#include "UnrealExporter.h"
#include "PersonaModule.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Modules/ModuleManager.h"
#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "Widgets/Input/STextEntryPopup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphEditorSchemaActions)

#define LOCTEXT_NAMESPACE "AnimNextAnimGraphSchemaActions"

FAnimNextSchemaAction_AddTemplateNode::FAnimNextSchemaAction_AddTemplateNode(const UE::UAF::FGraphNodeTemplateInfo& InNodeTemplateInfo, const FText& InKeywords)
	: FAnimNextSchemaAction(InNodeTemplateInfo.Category, InNodeTemplateInfo.MenuDescription, InNodeTemplateInfo.Tooltip, InKeywords)
	, NodeTemplateInfo(InNodeTemplateInfo)
{
}

UEdGraphNode* FAnimNextSchemaAction_AddTemplateNode::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	IRigVMClientHost* Host = ParentGraph->GetImplementingOuter<IRigVMClientHost>();
	URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ParentGraph);

	if (Host == nullptr || EdGraph == nullptr)
	{
		return nullptr;
	}
	
	UAnimNextController* Controller = Cast<UAnimNextController>(EdGraph->GetController());
	if (Controller == nullptr)
	{
		return nullptr;
	}

	FSoftObjectPath ClassPath(NodeTemplateInfo.ClassPath);
	UClass* TemplateClass = Cast<UClass>(ClassPath.TryLoad());
	if (TemplateClass == nullptr)
	{
		return nullptr;
	}
	
	const UUAFGraphNodeTemplate* TemplateCDO = TemplateClass->GetDefaultObject<UUAFGraphNodeTemplate>();
	if (TemplateCDO == nullptr)
	{
		return nullptr;
	}
	
	FText NodeTransaction = TemplateCDO->GetTitle();
	Controller->OpenUndoBracket(FString::Printf(TEXT("Add %s Node"), *NodeTransaction.ToString()));

	URigVMEdGraphNode* NewNode = nullptr;
	URigVMUnitNode* NewUnitNode = TemplateCDO->CreateNewNode(Controller, FVector2D(Location));
	if (NewUnitNode == nullptr)
	{
		Controller->CancelUndoBracket();
	}
	else
	{
		NewNode = Cast<URigVMEdGraphNode>(EdGraph->FindNodeForModelNodeName(NewUnitNode->GetFName()));
		Controller->CloseUndoBracket();
	}
	
	return NewNode;
}

const FSlateBrush* FAnimNextSchemaAction_AddTemplateNode::GetIconBrush() const
{
	// Only 'load' C++ classes as loading BP classes for all templates is too expensive
	if (NodeTemplateInfo.ClassPath.GetPackageName().ToString().StartsWith(TEXT("/Script")))
	{
		FSoftObjectPath ClassPath(NodeTemplateInfo.ClassPath);
		UClass* TemplateClass = Cast<UClass>(ClassPath.TryLoad());
		if (TemplateClass == nullptr)
		{
			return FAnimNextSchemaAction::GetIconBrush();
		}

		CachedIcon = TemplateClass->GetDefaultObject<UUAFGraphNodeTemplate>()->GetIcon(); 
		return &CachedIcon;
	}
	return FAnimNextSchemaAction::GetIconBrush();
}

FAnimNextSchemaAction_NotifyEvent::FAnimNextSchemaAction_NotifyEvent()
	: FAnimNextSchemaAction(LOCTEXT("NotifiesCategory", "Notifies"), LOCTEXT("AddNotifyEventLabel", "Add Notify Event..."), LOCTEXT("AddNotifyEventTooltip", "Add a custom event node to handle a named notify event"))
{
}

UEdGraphNode* FAnimNextSchemaAction_NotifyEvent::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ParentGraph);
	URigVMController* Controller = Cast<URigVMController>(EdGraph->GetController());
	if(Controller == nullptr)
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("AddNotify", LOCTEXT("AddNotifyEventSection", "Add Notify Event"));

	auto CreateNotifyEventWithName = [Controller, Location](FName InNotifyEventName)
	{
		if(!Controller->GetAllEventNames().Contains(InNotifyEventName))
		{
			Controller->OpenUndoBracket(LOCTEXT("AddNotifyEventTransaction", "Add Notify Event").ToString());
			URigVMUnitNode* CustomEventNode = Controller->AddUnitNode(FRigVMFunction_UserDefinedEvent::StaticStruct(), FRigVMStruct::ExecuteName, FDeprecateSlateVector2D(Location));
			Controller->SetPinDefaultValue(CustomEventNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_UserDefinedEvent, EventName))->GetPinPath(), InNotifyEventName.ToString(), true, true, true, true, true);
			Controller->CloseUndoBracket();
		}
	};

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddNewNotifyEventLabel", "Add New Notify Event..."),
		LOCTEXT("AddNewNotifyEventTooltip", "Add a new notify event as a custom event"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([CreateNotifyEventWithName]()
		{
			// Show dialog to enter new track name
			TSharedRef<STextEntryPopup> TextEntry =
				SNew(STextEntryPopup)
				.Label(LOCTEXT("NewNotifyLabel", "Notify Name"))
				.OnTextCommitted_Lambda([CreateNotifyEventWithName](const FText& InText, ETextCommit::Type InCommitType)
				{
					FSlateApplication::Get().DismissAllMenus();
					FName NotifyName = *InText.ToString();
					CreateNotifyEventWithName(NotifyName);
				});

			// Show dialog to enter new event name
			FSlateApplication::Get().PushMenu(
				FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
				FWidgetPath(),
				TextEntry,
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup));
		})),
		NAME_None,
		EUserInterfaceActionType::Button);
	
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TSharedRef<SWidget> NotifyPickerWidget = PersonaModule.CreateSkeletonNotifyPicker(FOnNotifyPicked::CreateLambda([CreateNotifyEventWithName](FName InNotifyName)
	{
		FSlateApplication::Get().DismissAllMenus();
		CreateNotifyEventWithName(InNotifyName);
	}));

	MenuBuilder.AddWidget(
		SNew(SBox)
		.WidthOverride(300.0f)
		.HeightOverride(400.0f)
		[
			NotifyPickerWidget
		],
		FText::GetEmpty(),
		true, false);

	MenuBuilder.EndSection();

	FSlateApplication::Get().PushMenu(
		FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
