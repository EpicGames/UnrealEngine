// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Common/GraphEditorSchemaActions.h"
#include "UncookedOnlyUtils.h"
#include "Templates/GraphNodeTemplateRegistry.h"
#include "AnimGraphEditorSchemaActions.generated.h"

class UUAFGraphNodeTemplate;

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

USTRUCT()
struct FAnimNextSchemaAction_AddTemplateNode : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction_AddTemplateNode() = default;

	UE_API FAnimNextSchemaAction_AddTemplateNode(const UE::UAF::FGraphNodeTemplateInfo& InNodeTemplateInfo, const FText& InKeywords = FText::GetEmpty());

	// FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	
	UE::UAF::FGraphNodeTemplateInfo NodeTemplateInfo;
	mutable FSlateBrush CachedIcon;
};

USTRUCT()
struct FAnimNextSchemaAction_NotifyEvent : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	UE_API FAnimNextSchemaAction_NotifyEvent();

	// FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }
};

#undef UE_API
