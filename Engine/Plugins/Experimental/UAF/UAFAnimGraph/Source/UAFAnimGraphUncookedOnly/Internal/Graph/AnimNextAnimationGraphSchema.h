// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetSchema.h"
#include "AnimNextAnimationGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UAnimNextAnimationGraphSchema : public UUAFRigVMAssetSchema
{
	GENERATED_BODY()

public:
	// URigVMSchema interface
	virtual bool CanSetNodeTitle(URigVMController* InController, const URigVMNode* InNode) const override;
	virtual bool CanRecolorNode(URigVMController* InController, const URigVMNode* InNode, const FLinearColor& InNewColor) const override;
	virtual bool CanUnfoldPin(URigVMController* InController, const URigVMPin* InPinToUnfold) const override;
	virtual bool ShouldUnfoldStruct(URigVMController* InController, const UStruct* InStruct) const override;
};
