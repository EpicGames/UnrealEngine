// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextController.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "AnimNextExecuteContext.h"

#include "AnimNextAnimationGraph_EditorData.generated.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

class FAnimationAnimNextRuntimeTest_GraphAddTrait;
class FAnimationAnimNextRuntimeTest_GraphExecute;
class FAnimationAnimNextRuntimeTest_GraphExecuteLatent;

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
	struct FAnimGraphUtils;
}

namespace UE::UAF::Editor
{
	class FModuleEditor;
	class SAnimNextGraphView;
	struct FUtils;
}

// Script-callable editor API hoisted onto UUAFAnimGraph
UCLASS()
class UAnimNextAnimationGraphLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Adds an animation graph to an AnimNext asset */
	UFUNCTION(BlueprintCallable, Category = "AnimNext|Entries", meta=(ScriptMethod))
	static UAFANIMGRAPHUNCOOKEDONLY_API UAnimNextAnimationGraphEntry* AddAnimationGraph(UUAFAnimGraph* InAsset, FName InName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
};

/** Editor data for UAF animation graphs */
UCLASS(MinimalAPI)
class UUAFAnimGraph_EditorData : public UUAFSharedVariables_EditorData
{
	GENERATED_BODY()

	friend class UUAFAnimGraphFactory;
	friend class UAnimNextEdGraph;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend struct UE::UAF::Editor::FUtils;
	friend class UE::UAF::Editor::FModuleEditor;
	friend class UE::UAF::Editor::SAnimNextGraphView;
	friend struct UE::UAF::UncookedOnly::FAnimGraphUtils;
	friend struct FAnimNextGraphSchemaAction_RigUnit;
	friend struct FAnimNextGraphSchemaAction_DispatchFactory;
	friend class FAnimationAnimNextEditorTest_GraphAddTrait;
	friend class FAnimationAnimNextEditorTest_GraphTraitOperations;
	friend class FAnimationAnimNextRuntimeTest_GraphExecute;
	friend class FAnimationAnimNextRuntimeTest_GraphExecuteLatent;
	friend class FAnimationUAFRuntimeTest_SharedDataGC;
	friend class FAnimationUAFRuntimeTest_SharedDataMissingGC;
	friend class FAnimationUAFRuntimeTest_InstanceDataGC;
	friend class FAnimationUAFRuntimeTest_InstanceDataGCMissing;
	friend class FAnimationAnimNextEditorTest_GraphManifest;

public:
	UE_API UUAFAnimGraph_EditorData();

	/** Adds an animation graph to this asset */
	UE_API UAnimNextAnimationGraphEntry* AddAnimationGraph(FName InName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

protected:
	// UUAFRigVMAssetEditorData interface
	virtual TSubclassOf<URigVMController> GetControllerClass() const override { return UAnimNextController::StaticClass(); }
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> GetEntryClasses() const override;
	UE_API virtual bool CanAddNewEntry(TSubclassOf<UUAFRigVMAssetEntry> InClass) const override;
	UE_API virtual TSubclassOf<UAssetUserData> GetAssetUserDataClass() const override;
	UE_API virtual void InitializeAssetUserData() override;
	UE_API virtual void OnCompileJobStarted() override;
	UE_API virtual void OnPreCompileAsset(FRigVMCompileSettings& InSettings) override;
	UE_API virtual void BuildFunctionHeadersContext(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) const override;
	UE_API virtual void OnPreCompileProcessGraphs(const FRigVMCompileSettings& InSettings, FAnimNextProcessGraphCompileContext& OutCompileContext) override;
	UE_API virtual void OnCompileJobFinished() override;
	UE_API virtual UClass* GetRigVMEdGraphSchemaClass() const override;
	UE_API virtual void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) override;

private:
	void HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldName);
};

#undef UE_API
