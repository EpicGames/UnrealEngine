// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_BaseAsyncTask.h"
#include "K2Node_AnimNextInjection.generated.h"

UCLASS()
class UK2Node_AnimNextInjection : public UK2Node_BaseAsyncTask
{
	GENERATED_BODY()

	UK2Node_AnimNextInjection();

	//~ Begin UEdGraphNode Interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface
};
