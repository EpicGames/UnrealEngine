// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "K2Node_BaseAsyncTask.h"

#include "K2Node_AnimNextPlayAsset.generated.h"

UCLASS(deprecated)
class UDEPRECATED_K2Node_AnimNextPlayAsset : public UK2Node_BaseAsyncTask
{
	GENERATED_BODY()

	UDEPRECATED_K2Node_AnimNextPlayAsset();

	// UEdGraphNode Interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	// UK2Node Interface
	virtual FText GetMenuCategory() const override;
};
