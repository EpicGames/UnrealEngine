// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SubclassOf.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API 

struct FAnimNextParamType;

namespace UE::UAF::AnimGraph::UncookedOnly
{
	class FAnimNextAnimGraphUncookedOnlyModule;
}

namespace UE::UAF
{

struct FGraphNodeTemplateInfo
{
	FTopLevelAssetPath ClassPath;
	FText MenuDescription;
	FText Tooltip;
	FText Category;
	TArray<FTopLevelAssetPath> DragDropAssetTypes;
	TArray<FAnimNextParamType> DragDropVariableTypes;
};

class FGraphNodeTemplateRegistry
{
public:
	// Get all templates map (class -> template info)
	UE_API static const TMap<FTopLevelAssetPath, FGraphNodeTemplateInfo>& GetAllTemplates();

	// Get all drag drop handlers registered to an asset type
	UE_API static TConstArrayView<FGraphNodeTemplateInfo> GetDragDropHandlersForAsset(const FAssetData& InAssetData);

	// Get all drag drop handlers registered to a variable type
	UE_API static TConstArrayView<FGraphNodeTemplateInfo> GetDragDropHandlersForVariable(const FAnimNextParamType& InVariableType);

private:
	friend UE::UAF::AnimGraph::UncookedOnly::FAnimNextAnimGraphUncookedOnlyModule;

	// Initialize the registry (called internally when needed)
	UE_API static void LazyInitialize();

	// Release any resources
	UE_API static void Shutdown();
};

}

#undef UE_API