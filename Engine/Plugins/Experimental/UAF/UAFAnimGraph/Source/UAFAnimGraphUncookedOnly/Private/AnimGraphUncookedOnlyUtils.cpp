// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphUncookedOnlyUtils.h"

#include "AnimNextEdGraphNode.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "TraitCore/TraitRegistry.h"
#include "UncookedOnlyUtils.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "AnimNextTraitStackUnitNode.h"

namespace UE::UAF::UncookedOnly
{

bool FAnimGraphUtils::IsTraitStackNode(const URigVMNode* InModelNode)
{
	if (const URigVMUnitNode* VMNode = Cast<URigVMUnitNode>(InModelNode))
	{
		const UScriptStruct* ScriptStruct = VMNode->GetScriptStruct();
		return ScriptStruct == FRigUnit_AnimNextTraitStack::StaticStruct();
	}

	return false;
}

void FAnimGraphUtils::SetupAnimGraph(const FName EntryName, URigVMController* InController, bool bSetupUndoRedo /*= true*/,  bool bPrintPythonCommand /*= false*/ )
{
	// Clear the graph
	InController->RemoveNodes(InController->GetGraph()->GetNodes(), bSetupUndoRedo, bPrintPythonCommand);

	// Add root node
	URigVMUnitNode* MainEntryPointNode = InController->AddUnitNode(FRigUnit_AnimNextGraphRoot::StaticStruct(), FRigUnit_AnimNextGraphRoot::EventName, FVector2D(-400.0f, 0.0f), FString(), bSetupUndoRedo, bPrintPythonCommand);
	if(MainEntryPointNode == nullptr)
	{
		return;
	}

	URigVMPin* BeginExecutePin = MainEntryPointNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));
	if(BeginExecutePin == nullptr)
	{
		return;
	}

	URigVMPin* EntryPointPin = MainEntryPointNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, EntryPoint));
	if(EntryPointPin == nullptr)
	{
		return;
	}

	InController->SetPinDefaultValue(EntryPointPin->GetPinPath(), EntryName.ToString(), true, bSetupUndoRedo, true, bPrintPythonCommand);
}

bool FAnimGraphUtils::RequestVMAutoRecompile(UUAFRigVMAssetEditorData* EditorData)
{
	if (EditorData)
	{
		if (IRigVMClientHost* ClientHost = CastChecked<IRigVMClientHost>(EditorData))
		{
			ClientHost->RequestAutoVMRecompilation();
			return true;
		}
	}
	return false;
}

} // end namespace
