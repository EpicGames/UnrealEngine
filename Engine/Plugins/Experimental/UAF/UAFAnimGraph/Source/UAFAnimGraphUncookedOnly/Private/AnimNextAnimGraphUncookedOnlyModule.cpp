// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UAFAnimGraphUncookedOnlyStyle.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Graph/RigVMTrait_AnimNextPublicVariablesUncookedOnly.h"
#include "Templates/GraphNodeTemplateRegistry.h"

#define LOCTEXT_NAMESPACE "FAnimNextAnimGraphUncookedOnlyModule"

namespace UE::UAF::AnimGraph::UncookedOnly
{

class FAnimNextAnimGraphUncookedOnlyModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		using namespace UE::UAF::UncookedOnly;

		FPublicVariablesImpl::Register();
		FUAFAnimGraphUncookedOnlyStyle::Get();
	}

	virtual void ShutdownModule() override
	{
		using namespace UE::UAF::UncookedOnly;

		FGraphNodeTemplateRegistry::Shutdown();
	}
};

}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::UAF::AnimGraph::UncookedOnly::FAnimNextAnimGraphUncookedOnlyModule, UAFAnimGraphUncookedOnly);
