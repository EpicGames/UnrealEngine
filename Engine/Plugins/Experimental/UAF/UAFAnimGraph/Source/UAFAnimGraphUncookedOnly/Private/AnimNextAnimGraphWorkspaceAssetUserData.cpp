// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphWorkspaceAssetUserData.h"

#include "UncookedOnlyUtils.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraphWorkspaceAssetUserData)

void UAnimNextAnimGraphWorkspaceAssetUserData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (!Context.IsSaving())
	{
		CachedExports.Exports.Reset();
		GetRootAssetExport(Context);
		GetWorkspaceAssetExports(Context);
	}

	FString TagValue;
	FWorkspaceOutlinerItemExports::StaticStruct()->ExportText(TagValue, &CachedExports, nullptr, nullptr, PPF_None, nullptr);
	Context.AddTag(FAssetRegistryTag(UE::Workspace::ExportsWorkspaceItemsRegistryTag, TagValue, FAssetRegistryTag::TT_Hidden));
}

void UAnimNextAnimGraphWorkspaceAssetUserData::GetRootAssetExport(FAssetRegistryTagsContext Context) const
{
	UUAFAnimGraph* Asset = CastChecked<UUAFAnimGraph>(GetOuter());
	FWorkspaceOutlinerItemExport& RootAssetExport = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(Asset->GetFName(), Asset));

	RootAssetExport.GetData().InitializeAsScriptStruct(FAnimNextAnimationGraphOutlinerData::StaticStruct());
	FAnimNextRigVMAssetOutlinerData& Data = RootAssetExport.GetData().GetMutable<FAnimNextRigVMAssetOutlinerData>();
	Data.SoftAssetPtr = Asset;
}

void UAnimNextAnimGraphWorkspaceAssetUserData::GetWorkspaceAssetExports(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	UUAFAnimGraph* Asset = CastChecked<UUAFAnimGraph>(GetOuter());
	const UUAFAnimGraph_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(Asset);

	UE::UAF::UncookedOnly::FUtils::GetAssetWorkspaceExports(EditorData, CachedExports, Context);
}

