// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextController.h"

#include "RigVMPythonUtils.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextTraitStackUnitNode.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Param/ParamType.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitUID.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextController)

FName UAnimNextController::AddTraitStruct(URigVMUnitNode* InNode, const TInstancedStruct<FAnimNextTraitSharedData>& InTraitDefaults, int32 InTraitIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return NAME_None;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (InNode == nullptr)
	{
		ReportError(TEXT("Invalid node supplied to AddTraitStruct."));
		return NAME_None;
	}

	UE::UAF::FTraitRegistry& TraitRegistry = UE::UAF::FTraitRegistry::Get();
	const UE::UAF::FTrait* Trait = TraitRegistry.Find(InTraitDefaults.GetScriptStruct());
	if (Trait == nullptr)
	{
		ReportError(TEXT("Unknown Trait Type."));
		return NAME_None;
	}

	UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();

	const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
	FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
	CppDecoratorStructInstance.DecoratorSharedDataStruct = const_cast<UScriptStruct*>(InTraitDefaults.GetScriptStruct());	// TODO: Refactor const here

	if (!CppDecoratorStructInstance.CanBeAddedToNode(InNode, nullptr))
	{
		ReportError(TEXT("Trait is not supported by the Node."));
		return NAME_None;	// This trait isn't supported on this node
	}

	FString DefaultValue;
	FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_SerializedAsImportText, nullptr);
	
	// Avoid multiple VM recompilations for internal operations
	FRigVMControllerCompileBracketScope CompileScope(this);

	// Must use struct name for name to allow for consistent pathing in node layouts
	const FName TraitName = InTraitDefaults.GetScriptStruct()->GetFName();

	const FName NewTraitName = Super::AddTrait(InNode, CppDecoratorStruct, TraitName, DefaultValue, InTraitIndex, bSetupUndoRedo);
	if (NewTraitName != NAME_None)
	{
		// Append defaults for shared data
		FString TraitDefaultValue;
		InTraitDefaults.GetScriptStruct()->ExportText(TraitDefaultValue, InTraitDefaults.GetMemory(), InTraitDefaults.GetMemory(), nullptr, PPF_SerializedAsImportText, nullptr);
		URigVMPin* TraitPin = InNode->FindPin(NewTraitName.ToString());

		TArray<FString> MemberValuePairs = URigVMPin::SplitDefaultValue(TraitDefaultValue);
		for (const FString& MemberValuePair : MemberValuePairs)
		{
			FString MemberName, MemberValue;
			if (MemberValuePair.Split(TEXT("="), &MemberName, &MemberValue))
			{
				if (URigVMPin* SubPin = TraitPin->FindSubPin(MemberName))
				{
					if(!MemberValue.IsEmpty())
					{
						PostProcessDefaultValue(SubPin, MemberValue);
						if (!MemberValue.IsEmpty())
						{
							SetPinDefaultValue(SubPin, MemberValue, true, bSetupUndoRedo, true, true);
						}
					}
				}
			}
		}
	}
	
	if(!NewTraitName.IsNone() && bPrintPythonCommand)
	{
		const TArray<FString> TraitCommands = GetAddTraitPythonCommands(InNode, NewTraitName);
		for(const FString& TraitCommand : TraitCommands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), TraitCommand);
		}
	}

	return NewTraitName;
}

FName UAnimNextController::AddTraitByName(FName InNodeName, FName InNewTraitTypeName, int32 InPinIndex, const FString& InNewTraitDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return NAME_None;
	}

	if (InNodeName == NAME_None)
	{
		ReportError(TEXT("Invalid node name."));
		return NAME_None;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode * Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportError(TEXT("This graph does not contain a node with the provided name."));
		return NAME_None;
	}

	UE::UAF::FTraitRegistry& TraitRegistry = UE::UAF::FTraitRegistry::Get();

	const UE::UAF::FTrait* Trait = TraitRegistry.Find(InNewTraitTypeName);
	if (Trait == nullptr)
	{
		ReportError(TEXT("Unknown Trait Type."));
		return NAME_None;
	}

	const UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();
	UScriptStruct* ScriptStruct = Trait->GetTraitSharedDataStruct();

	FString DefaultValue = InNewTraitDefaultValue;
	if (DefaultValue.IsEmpty())
	{
		const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
		FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
		CppDecoratorStructInstance.DecoratorSharedDataStruct = ScriptStruct;

		if (!CppDecoratorStructInstance.CanBeAddedToNode(Node, nullptr))
		{
			ReportError(TEXT("Trait is not supported by the Node."));
			return NAME_None;	// This trait isn't supported on this node
		}

		FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_SerializedAsImportText, nullptr);
	}

	// Avoid multiple VM recompilations for internal operations
	FRigVMControllerCompileBracketScope CompileScope(this);

	// Must use struct name for name to allow for consistent pathing in node layouts
	const FName TraitName = ScriptStruct->GetFName();

	return AddTrait(InNodeName, *CppDecoratorStruct->GetPathName(), TraitName, DefaultValue, InPinIndex, bSetupUndoRedo, bPrintPythonCommand);
}

bool UAnimNextController::RemoveTraitByName(FName InNodeName, FName InTraitInstanceName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return false;
	}

	if (InNodeName == NAME_None)
	{
		ReportError(TEXT("Invalid node name."));
		return false;
	}

	// Avoid multiple VM recompilations for internal operations
	FRigVMControllerCompileBracketScope CompileScope(this);
	return RemoveTrait(InNodeName, InTraitInstanceName, bSetupUndoRedo, bPrintPythonCommand);
}

FName UAnimNextController::SwapTraitByName(FName InNodeName, FName InTraitInstanceName, int32 InCurrentTraitPinIndex, FName InNewTraitTypeName, const FString& InNewTraitDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return NAME_None;
	}

	if (InNodeName == NAME_None)
	{
		ReportError(TEXT("Invalid node name."));
		return NAME_None;
	}

	// Avoid multiple VM recompilations, for each operation
	FRigVMControllerCompileBracketScope CompileScope(this);
	if (RemoveTraitByName(InNodeName, InTraitInstanceName, bSetupUndoRedo, bPrintPythonCommand))
	{
		return AddTraitByName(InNodeName, InNewTraitTypeName, InCurrentTraitPinIndex, InNewTraitDefaultValue, bSetupUndoRedo, bPrintPythonCommand);
	}

	return NAME_None;
}

bool UAnimNextController::SetTraitPinIndex(FName InNodeName, FName InTraitInstanceName, int32 InNewPinIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return false;
	}

	if (InNodeName == NAME_None)
	{
		ReportError(TEXT("Invalid node name."));
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportError(TEXT("This graph does not contain a node with the provided name."));
		return false;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return false;
	}

	URigVMPin* TraitPin = Node->FindTrait(InTraitInstanceName);
	if (TraitPin == nullptr)
	{
		ReportError(TEXT("The node does not contain a Trait with the provided name."));
		return false;
	}
	
	// Save current pin data for later
	const FString TraitDefaultValue = TraitPin->GetDefaultValue();

	// TODO zzz : Is there a better way to get a Trait* from a TraitPin ?
	if (TSharedPtr<FStructOnScope> ScopedTrait = Node->GetTraitInstance(TraitPin->GetFName()))
	{
		const FRigVMTrait* Trait = (FRigVMTrait*)ScopedTrait->GetStructMemory();
		if (const UScriptStruct* TraitSharedInstanceData = Trait->GetTraitSharedDataStruct())
		{
			UE::UAF::FTraitRegistry& TraitRegistry = UE::UAF::FTraitRegistry::Get();
			if (const UE::UAF::FTrait* AnimNextTrait = TraitRegistry.Find(TraitSharedInstanceData))
			{
				// Avoid multiple VM recompilations, for each operation
				FRigVMControllerCompileBracketScope CompileScope(this);
				const int32 TraitPinInitialIndex = TraitPin->GetPinIndex();
				
				if (RemoveTraitByName(InNodeName, InTraitInstanceName, bSetupUndoRedo, bPrintPythonCommand))
				{
					// Adjust indices due to pin removal
					if (TraitPinInitialIndex < InNewPinIndex)
					{
						--InNewPinIndex;
					}
					
					// TOOO zzz : Why TraitName is a string ?
					if (AddTraitByName(InNodeName, FName(AnimNextTrait->GetTraitName()), InNewPinIndex, TraitDefaultValue, bSetupUndoRedo, bPrintPythonCommand) != NAME_None)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

URigVMUnitNode* UAnimNextController::AddUnitNodeWithPins(UScriptStruct* InScriptStruct, const FRigVMPinInfoArray& PinArray, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	const bool bHasDynamicPins = PinArray.Num() != 0;

	if (bHasDynamicPins)
	{
		OpenUndoBracket(TEXT("Add unit node with pins"));
	}

	URigVMUnitNode* Node = AddUnitNodeOfClass(InScriptStruct->GetPathName(), UAnimNextUnitNode::StaticClass()->GetPathName(), InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);

	if (Node == nullptr)
	{
		if (bHasDynamicPins)
		{
			CancelUndoBracket();
		}

		return nullptr;
	}

	if (bHasDynamicPins)
	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		const FRigVMPinInfoArray PreviousPins(Node, this);

		for (int32 PinIndex = 0; PinIndex < PinArray.Num(); ++PinIndex)
		{
			const FString& PinPath = PinArray.GetPinPath(PinIndex);
			FString ParentPinPath, PinName;
			UObject* OuterForPin = Node;
			if (URigVMPin::SplitPinPathAtEnd(PinPath, ParentPinPath, PinName))
			{
				OuterForPin = Node->FindPin(ParentPinPath);
			}

			CreatePinFromPinInfo(Registry, PreviousPins, PinArray[PinIndex], PinPath, OuterForPin);
		}

		CloseUndoBracket();
	}

	return Node;
}

bool UAnimNextController::UpgradeTraitByName(FName InNodeName, FName InTraitInstanceName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!IsTransacting() && !IsGraphEditable())
	{
		return false;
	}

	if (InNodeName == NAME_None)
	{
		ReportError(TEXT("Invalid node name."));
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportError(TEXT("This graph does not contain a node with the provided name."));
		return false;
	}
	
	URigVMPin* TraitPin = Node->FindTrait(InTraitInstanceName);
	if (TraitPin == nullptr)
	{
		ReportError(TEXT("The node does not contain a Trait with the provided name."));
		return false;
	}

	// Avoid multiple VM recompilations for internal operations
	FRigVMControllerCompileBracketScope CompileScope(this);

	TMap<FString, FString> RedirectedPinPaths;
	TMap<FString, FPinState> PinStates = GetPinStates(Node, true);
	EjectAllInjectedNodes(Node, bSetupUndoRedo);

	FRigVMBaseAction Action(this);
	if(bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Upgrade trait"));
		GetActionStack()->BeginAction(Action);
	}

	if (TSharedPtr<FStructOnScope> ScopedTrait = Node->GetTraitInstance(TraitPin->GetFName()))
	{
		const FRigVMTrait* OldTrait = (FRigVMTrait*)ScopedTrait->GetStructMemory();
		if (const UScriptStruct* TraitSharedInstanceData = OldTrait->GetTraitSharedDataStruct())
		{
			UE::UAF::FTraitRegistry& TraitRegistry = UE::UAF::FTraitRegistry::Get();
			if (const UE::UAF::FTrait* OldUAFTrait = TraitRegistry.Find(TraitSharedInstanceData))
			{
				FRigVMStructUpgradeInfo UpgradeInfo = OldUAFTrait->GetUpgradeInfo();
				if (UpgradeInfo.IsValid())
				{
					if(UpgradeInfo.GetNewStruct()->IsChildOf(FAnimNextTraitSharedData::StaticStruct()))
					{
						if (!RemoveTraitByName(InNodeName, InTraitInstanceName, bSetupUndoRedo, bPrintPythonCommand))
						{
							if(bSetupUndoRedo)
							{
								GetActionStack()->CancelAction(Action);
							}
							ReportErrorf(TEXT("Unable to upgrade trait %s."), *InTraitInstanceName.ToString());
							return false;
						}

						TInstancedStruct<FAnimNextTraitSharedData> Defaults(UpgradeInfo.GetNewStruct());
						FName NewTraitName = AddTraitStruct(CastChecked<URigVMUnitNode>(Node), Defaults, TraitPin->GetPinIndex(), bSetupUndoRedo, bPrintPythonCommand);
						if (NewTraitName.IsNone())
						{
							if(bSetupUndoRedo)
							{
								GetActionStack()->CancelAction(Action);
							}
							ReportErrorf(TEXT("Unable to upgrade trait %s."), *InTraitInstanceName.ToString());
							return false;
						}

						URigVMPin* NewTraitPin = Node->FindTrait(NewTraitName);
						check(NewTraitPin);

						for(URigVMPin* Pin : NewTraitPin->GetSubPins())
						{
							const FString DefaultValue = UpgradeInfo.GetDefaultValueForPin(Pin->GetFName());
							if(!DefaultValue.IsEmpty())
							{
								SetPinDefaultValue(Pin, DefaultValue, true, bSetupUndoRedo, false);

								if(FPinState* PinState = PinStates.Find(Pin->GetPinPath()))
								{
									PinState->DefaultValue.Reset();
								}
							}
						}
						
						// redirect pin state paths
						for(TPair<FString, FPinState>& PinState : PinStates)
						{
							for(int32 TrueFalse = 0; TrueFalse < 2; TrueFalse++)
							{
								const FString RemappedInputPath = UpgradeInfo.RemapPin(PinState.Key, TrueFalse == 0, false);
								if(RemappedInputPath != PinState.Key)
								{
									if(!RedirectedPinPaths.Contains(PinState.Key))
									{
										RedirectedPinPaths.Add(PinState.Key, RemappedInputPath);
									}
								}
							}
						}
					}
				}
			}
		}
	}

	ApplyPinStates(Node, PinStates, RedirectedPinPaths, bSetupUndoRedo);

	if(bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}
	
	return true;
}