// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextTraitStackUnitNode.h"

#include "Styling/StyleDefaults.h"
#include "Templates/UAFGraphNodeTemplate.h"

FString UAnimNextTraitStackUnitNode::GetDefaultNodeTitle() const
{
	if (Template.Get() != nullptr)
	{
		FEditorScriptExecutionGuard AllowScripts;
		return Template->GetDefaultObject<UUAFGraphNodeTemplate>()->GetTitle().ToString();
	}

	return Super::GetNodeTitle();
}

FString UAnimNextTraitStackUnitNode::GetNodeSubTitle() const
{
	if (Template.Get() != nullptr)
	{
		FEditorScriptExecutionGuard AllowScripts;
		return Template->GetDefaultObject<UUAFGraphNodeTemplate>()->GetSubTitle().ToString();
	}

	return Super::GetNodeSubTitle();
}

FText UAnimNextTraitStackUnitNode::GetToolTipText() const
{
	if (Template.Get() != nullptr)
	{
		FEditorScriptExecutionGuard AllowScripts;
		return Template->GetDefaultObject<UUAFGraphNodeTemplate>()->GetTooltipText();
	}

	return Super::GetToolTipText();
}

FLinearColor UAnimNextTraitStackUnitNode::GetDefaultNodeColor() const
{
	if (Template.Get() != nullptr)
	{
		FEditorScriptExecutionGuard AllowScripts;
		return Template->GetDefaultObject<UUAFGraphNodeTemplate>()->GetColor();
	}

	return Super::GetNodeColor();
}

const FSlateBrush* UAnimNextTraitStackUnitNode::GetDefaultNodeIconBrush() const
{
	if (Template.Get() != nullptr)
	{
		FEditorScriptExecutionGuard AllowScripts;
		return Template->GetDefaultObject<UUAFGraphNodeTemplate>()->GetIconBrush();
	}

	return FSlateIcon("UAFAnimGraphUncookedOnlyStyle", "NodeTemplate.DefaultIcon").GetIcon();
}

void UAnimNextTraitStackUnitNode::HandlePinDefaultValueChanged(UAnimNextController* InController, URigVMPin* InPin)
{
	if (Template.Get() != nullptr)
	{
		Template->GetDefaultObject<UUAFGraphNodeTemplate>()->HandlePinDefaultValueChanged(InController, InPin);
	}
}

void UAnimNextTraitStackUnitNode::HandleAssetRenamed(UAnimNextController* InController, const FAssetData& InAssetData, const FString& InOldName)
{
	if (Template.Get() != nullptr)
	{
		Template->GetDefaultObject<UUAFGraphNodeTemplate>()->HandleAssetRenamed(InController, this, InAssetData, InOldName);
	}
}