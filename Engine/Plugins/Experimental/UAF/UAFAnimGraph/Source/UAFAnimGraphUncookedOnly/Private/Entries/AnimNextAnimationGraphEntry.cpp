// Copyright Epic Games, Inc. All Rights Reserved.

#include "Entries/AnimNextAnimationGraphEntry.h"

#include "AnimNextRigVMAsset.h"
#include "UncookedOnlyUtils.h"
#include "AnimNextEdGraph.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Param/AnimNextTag.h"
#include "Param/ParamType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimationGraphEntry)

FAnimNextParamType UAnimNextAnimationGraphEntry::GetExportType() const
{
	return FAnimNextParamType::GetType<FAnimNextEntryPoint>();
}

FName UAnimNextAnimationGraphEntry::GetExportName() const
{
	return GraphName;
}

EAnimNextExportAccessSpecifier UAnimNextAnimationGraphEntry::GetExportAccessSpecifier() const
{
	return Access;
}

void UAnimNextAnimationGraphEntry::SetExportAccessSpecifier(EAnimNextExportAccessSpecifier InAccessSpecifier, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	};

	Access = InAccessSpecifier;

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAccessSpecifierChanged);
}

FName UAnimNextAnimationGraphEntry::GetEntryName() const
{
	return GraphName;
}

void UAnimNextAnimationGraphEntry::SetEntryName(FName InName, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	};

	GraphName = InName;

	// Forward to entry point node
	URigVMController* Controller = GetImplementingOuter<IRigVMClientHost>()->GetController(Graph);
	for(URigVMNode* Node : Graph->GetNodes())
	{
		if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
		{
			if(UnitNode->GetScriptStruct() == FRigUnit_AnimNextGraphRoot::StaticStruct())
			{
				URigVMPin* EntryPointPin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, EntryPoint));
				check(EntryPointPin);
				check(EntryPointPin->GetDirection() == ERigVMPinDirection::Hidden);

				Controller->SetPinDefaultValue(EntryPointPin->GetPinPath(), InName.ToString(), true, true, false, true);
			}
		}
	}
	
	BroadcastModified(EAnimNextEditorDataNotifType::EntryRenamed);
}

const FName& UAnimNextAnimationGraphEntry::GetGraphName() const
{
	return GraphName;
}

URigVMGraph* UAnimNextAnimationGraphEntry::GetRigVMGraph() const
{
	return Graph;
}

URigVMEdGraph* UAnimNextAnimationGraphEntry::GetEdGraph() const
{
	return EdGraph;
}

void UAnimNextAnimationGraphEntry::SetRigVMGraph(URigVMGraph* InGraph)
{
	Graph = InGraph;
}

void UAnimNextAnimationGraphEntry::SetEdGraph(URigVMEdGraph* InGraph)
{
	EdGraph = CastChecked<UAnimNextEdGraph>(InGraph, ECastCheckedType::NullAllowed);
}
