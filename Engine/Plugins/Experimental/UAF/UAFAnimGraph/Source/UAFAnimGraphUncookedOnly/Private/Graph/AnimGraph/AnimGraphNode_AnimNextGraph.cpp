// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimGraph/AnimGraphNode_AnimNextGraph.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMBlueprintGeneratedClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_AnimNextGraph)

#define LOCTEXT_NAMESPACE "AnimGraphNode_AnimNextGraph"

UAnimGraphNode_AnimNextGraph::UAnimGraphNode_AnimNextGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_AnimNextGraph::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_UAFModule_Title", "UAF Animation Graph");
}

FText UAnimGraphNode_AnimNextGraph::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_UAFAnimationGraph_Tooltip", "Evaluates a UAF Animation Graph");
}

FText UAnimGraphNode_AnimNextGraph::GetMenuCategory() const
{
	return LOCTEXT("Category", "Animation Framework");
}

void UAnimGraphNode_AnimNextGraph::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();

	if (Node.AnimationGraph)
	{
		PreloadObject(Node.AnimationGraph);
		PreloadObject(Node.AnimationGraph->EditorData);
	}
}

void UAnimGraphNode_AnimNextGraph::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
}

void UAnimGraphNode_AnimNextGraph::CreateCustomPins(TArray<UEdGraphPin*>* OldPins)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	
	// we do this to refresh input variables
	//RebuildExposedProperties();

	// Grab the SKELETON class here as when we are reconstructed during during BP compilation
	// the full generated class is not yet present built.
	if (!GetTargetSkeletonClass())
	{
		// Nothing to search for properties
		return;
	}
}

void UAnimGraphNode_AnimNextGraph::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (UClass* TargetClass = GetTargetClass())
	{
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

void UAnimGraphNode_AnimNextGraph::RebuildExposedProperties()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

}

bool UAnimGraphNode_AnimNextGraph::IsInputProperty(const FName& PropertyName) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// this is true for both input variables and controls
	return InputVariables.Contains(PropertyName) || !OutputVariables.Contains(PropertyName);
}

bool UAnimGraphNode_AnimNextGraph::IsPropertyExposeEnabled(FName PropertyName) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// if known exposable, and and if it hasn't been exposed yet
	if (CustomPinProperties.ContainsByPredicate([PropertyName](const FOptionalPinFromProperty& InOptionalPin){ return InOptionalPin.PropertyName == PropertyName; }))
	{
		return IsInputProperty(PropertyName);
	}

	return false;
}

ECheckBoxState UAnimGraphNode_AnimNextGraph::IsPropertyExposed(FName PropertyName) const
{
	if (CustomPinProperties.ContainsByPredicate([PropertyName](const FOptionalPinFromProperty& InOptionalPin){ return InOptionalPin.bShowPin && InOptionalPin.PropertyName == PropertyName; }))
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void UAnimGraphNode_AnimNextGraph::OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
}

void UAnimGraphNode_AnimNextGraph::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Super::CustomizeDetails(DetailBuilder);

	// We dont allow multi-select here
	if (DetailBuilder.GetSelectedObjects().Num() > 1)
	{
		return;
	}

	// input/output exposure feature START
	RebuildExposedProperties();

	// ***************************************************************


	TSharedRef<IPropertyHandle> ClassHandle = DetailBuilder.GetProperty(TEXT("Node.AnimNextGraph"), GetClass());
	if (ClassHandle->IsValidHandle())
	{
		ClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateUObject(this, &UAnimGraphNode_AnimNextGraph::OnInstanceClassChanged, &DetailBuilder));
	}
}

void UAnimGraphNode_AnimNextGraph::GetVariables(bool bInput, TMap<FName, FRigVMExternalVariable>& OutVariables) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	OutVariables.Reset();

}

void UAnimGraphNode_AnimNextGraph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresNodeReconstruct = false;
	FProperty* ChangedProperty = PropertyChangedEvent.Property;

	if (ChangedProperty)
	{
		if (ChangedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimNode_AnimNextGraph, AnimationGraph))
		{
			bRequiresNodeReconstruct = true;
			RebuildExposedProperties();
		}
	}

	if (bRequiresNodeReconstruct)
	{
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_AnimNextGraph::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UAnimGraphNode_AnimNextGraph::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);
}

UObject* UAnimGraphNode_AnimNextGraph::GetJumpTargetForDoubleClick() const
{
	return Node.AnimationGraph;
}

#undef LOCTEXT_NAMESPACE
