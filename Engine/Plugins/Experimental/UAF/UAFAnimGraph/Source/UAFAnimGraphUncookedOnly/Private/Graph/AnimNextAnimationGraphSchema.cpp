// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextAnimationGraphSchema.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "TraitCore/TraitHandle.h"
#include "Variables/AnimNextVariableReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimationGraphSchema)

bool UAnimNextAnimationGraphSchema::CanSetNodeTitle(URigVMController* InController, const URigVMNode* InNode) const
{
	if (InNode != nullptr)
	{
		if (UE::UAF::UncookedOnly::FAnimGraphUtils::IsTraitStackNode(InNode))
		{
			return true;
		}
	}
	return false;
}

bool UAnimNextAnimationGraphSchema::CanRecolorNode(URigVMController* InController, const URigVMNode* InNode, const FLinearColor& InNewColor) const
{
	if (InNode != nullptr)
	{
		if (UE::UAF::UncookedOnly::FAnimGraphUtils::IsTraitStackNode(InNode))
		{
			return true;
		}
	}
	return false;
}

bool UAnimNextAnimationGraphSchema::CanUnfoldPin(URigVMController* InController, const URigVMPin* InPinToUnfold) const
{
	if (Super::CanUnfoldPin(InController, InPinToUnfold))
	{
		return true;
	}

	if (InPinToUnfold->IsArray())
	{
		// Required in order to be able to set pin default values to arrays that exist as hidden pins at trait shared data
		// URigVMController::SetPinDefaultValue only allows setting default values to arrays if can be unfolded
		if (InPinToUnfold->IsTraitPin() && InPinToUnfold->GetDirection() == ERigVMPinDirection::Hidden)
		{
			return true;
		}
	}
	return false;
}

bool UAnimNextAnimationGraphSchema::ShouldUnfoldStruct(URigVMController* InController, const UStruct* InStruct) const
{
	if(!Super::ShouldUnfoldStruct(InController, InStruct))
	{
		return false;
	}

	if (InStruct->IsChildOf<FAnimNextTraitHandle>())
	{
		return false;
	}

	if (InStruct->IsChildOf<FAnimNextVariableReference>())
	{
		return false;
	}

	return true;
}