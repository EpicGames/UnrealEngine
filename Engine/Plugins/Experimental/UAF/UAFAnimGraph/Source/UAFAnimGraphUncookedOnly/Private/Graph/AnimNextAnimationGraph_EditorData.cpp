// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextAnimGraphWorkspaceAssetUserData.h"
#include "UAFCompilationScope.h"
#include "AnimNextTraitStackUnitNode.h"
#include "RigVMPythonUtils.h"
#include "UAFAnimGraphEdGraphSchema.h"
#include "Compilation/AnimNextGetFunctionHeaderCompileContext.h"
#include "Compilation/AnimNextGetVariableCompileContext.h"
#include "Compilation/AnimNextProcessGraphCompileContext.h"
#include "UncookedOnlyUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Curves/CurveFloat.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Graph/AnimNextAnimationGraphSchema.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Graph/RigUnit_AnimNextShimRoot.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Logging/MessageLog.h"
#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitWriter.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "Traits/CallFunction.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Graph/RigUnit_AnimNextGraphEvaluator.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "TraitCore/TraitInterfaceRegistry.h"
#include "TraitInterfaces/IGarbageCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimationGraph_EditorData)

#define LOCTEXT_NAMESPACE "AnimNextAnimationGraph_EditorData"

namespace UE::UAF::UncookedOnly::Private
{
	// Represents a trait entry on a node
	struct FTraitEntryMapping
	{
		// The RigVM node that hosts this RigVM decorator
		const URigVMNode* DecoratorStackNode = nullptr;

		// The RigVM decorator pin on our host node
		const URigVMPin* DecoratorEntryPin = nullptr;

		// The AnimNext trait
		const FTrait* Trait = nullptr;

		// A map from latent property names to their corresponding RigVM memory handle index
		TMap<FName, uint16> LatentPropertyNameToIndexMap;

		FTraitEntryMapping(const URigVMNode* InDecoratorStackNode, const URigVMPin* InDecoratorEntryPin, const FTrait* InTrait)
			: DecoratorStackNode(InDecoratorStackNode)
			, DecoratorEntryPin(InDecoratorEntryPin)
			, Trait(InTrait)
		{}
	};

	// Represents a node that contains a trait list
	struct FTraitStackMapping
	{
		// The RigVM node that hosts the RigVM decorators
		const URigVMNode* DecoratorStackNode = nullptr;

		// The trait list on this node
		TArray<FTraitEntryMapping> TraitEntries;

		// The node handle assigned to this RigVM node
		FNodeHandle TraitStackNodeHandle;

		explicit FTraitStackMapping(const URigVMNode* InDecoratorStackNode)
			: DecoratorStackNode(InDecoratorStackNode)
		{}
	};

	struct FTraitGraph
	{
		FName EntryPoint;
		URigVMNode* RootNode;
		TArray<FTraitStackMapping> TraitStackNodes;

		explicit FTraitGraph(const UUAFAnimGraph* InAnimationGraph, URigVMNode* InRootNode)
			: RootNode(InRootNode)
		{
			EntryPoint = FName(InRootNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, EntryPoint))->GetDefaultValue());
		}
	};

	template<typename TraitAction>
	void ForEachTraitInStack(const URigVMNode* DecoratorStackNode, const TraitAction& Action)
	{
		const TArray<URigVMPin*>& Pins = DecoratorStackNode->GetPins();
		for (URigVMPin* Pin : Pins)
		{
			if (!Pin->IsTraitPin())
			{
				continue;	// Not a decorator pin
			}

			if (Pin->GetScriptStruct() == FRigDecorator_AnimNextCppDecorator::StaticStruct())
			{
				TSharedPtr<FStructOnScope> DecoratorScope = Pin->GetTraitInstance();
				FRigDecorator_AnimNextCppDecorator* VMDecorator = (FRigDecorator_AnimNextCppDecorator*)DecoratorScope->GetStructMemory();

				if (const FTrait* Trait = VMDecorator->GetTrait())
				{
					Action(DecoratorStackNode, Pin, Trait);
				}
			}
		}
	}

	TArray<FTraitUID> GetTraitUIDs(const URigVMNode* DecoratorStackNode)
	{
		TArray<FTraitUID> Traits;

		ForEachTraitInStack(DecoratorStackNode,
			[&Traits](const URigVMNode* DecoratorStackNode, const URigVMPin* DecoratorPin, const FTrait* Trait)
			{
				Traits.Add(Trait->GetTraitUID());
			});

		return Traits;
	}

	FNodeHandle RegisterTraitNodeTemplate(FTraitWriter& TraitWriter, const URigVMNode* DecoratorStackNode)
	{
		const TArray<FTraitUID> TraitUIDs = GetTraitUIDs(DecoratorStackNode);

		TArray<uint8> NodeTemplateBuffer;
		const FNodeTemplate* NodeTemplate = FNodeTemplateBuilder::BuildNodeTemplate(TraitUIDs, NodeTemplateBuffer);

		return TraitWriter.RegisterNode(*NodeTemplate);
	}

	FString GetTraitProperty(const FTraitStackMapping& TraitStack, uint32 TraitIndex, FName PropertyName, const TArray<FTraitStackMapping>& TraitStackNodes)
	{
		URigVMGraph* Graph = TraitStack.DecoratorStackNode->GetGraph();
		check(Graph);
		IRigVMClientHost* ClientHost = Graph->GetImplementingOuter<IRigVMClientHost>();
		check(ClientHost);
		URigVMController* Controller = ClientHost->GetOrCreateController(Graph);
		check(Controller);

		const TArray<URigVMPin*>& Pins = TraitStack.TraitEntries[TraitIndex].DecoratorEntryPin->GetSubPins();
		for (URigVMPin* Pin : Pins)
		{
			if (Pin->GetDirection() != ERigVMPinDirection::Input &&
				Pin->GetDirection() != ERigVMPinDirection::Hidden)
			{
				continue;	// We only look for input or hidden pins
			}

			if (Pin->GetFName() == PropertyName)
			{
				// If we contain child handles, we need to patch them to the correct source node/trait indices
				// Note patching the pins in this way here is only OK because we operate on a temporary intermediate copy graph, not the authored graph
				auto PatchDefaultValue = [&TraitStackNodes, Controller](URigVMPin* InPin)
				{
					// Trait handle pins don't have a value, just an optional link
					const TArray<URigVMLink*>& PinLinks = InPin->GetLinks();
					if (!PinLinks.IsEmpty())
					{
						// Something is connected to us, find the corresponding node handle so that we can encode it as our property value
						check(PinLinks.Num() == 1);

						const URigVMNode* SourceNode = PinLinks[0]->GetSourceNode();

						FNodeHandle SourceNodeHandle;
						int32 SourceTraitIndex = INDEX_NONE;

						const FTraitStackMapping* SourceTraitStack = TraitStackNodes.FindByPredicate([SourceNode](const FTraitStackMapping& Mapping) { return Mapping.DecoratorStackNode == SourceNode; });
						if (SourceTraitStack != nullptr)
						{
							SourceNodeHandle = SourceTraitStack->TraitStackNodeHandle;

							// If the source pin is null, we are a node where the result pin lives on the stack node instead of a decorator sub-pin
							// If this is the case, we bind to the first trait index since we only allowed a single base trait per stack
							// Otherwise we lookup the trait index we are linked to
							const URigVMPin* SourceDecoratorPin = PinLinks[0]->GetSourcePin()->GetParentPin();
							SourceTraitIndex = SourceDecoratorPin != nullptr ? SourceTraitStack->DecoratorStackNode->GetTraitPins().IndexOfByKey(SourceDecoratorPin) : 0;
						}

						if (SourceNodeHandle.IsValid())
						{
							check(SourceTraitIndex != INDEX_NONE);

							const FAnimNextTraitHandle TraitHandle(SourceNodeHandle, SourceTraitIndex);
							const FAnimNextTraitHandle DefaultTraitHandle;

							// We need an instance of a trait handle property to be able to serialize it into text, grab it from the root
							const FProperty* Property = FRigUnit_AnimNextGraphRoot::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));

							FString PropertyValue;
							Property->ExportText_Direct(PropertyValue, &TraitHandle, &DefaultTraitHandle, nullptr, PPF_SerializedAsImportText);
							Controller->SetPinDefaultValue(InPin, PropertyValue, false, false, false);
						}
					}
				};

				if (Pin->GetCPPTypeObject() == FAnimNextTraitHandle::StaticStruct())
				{
					PatchDefaultValue(Pin);
				}
				else
				{
					const TArray<URigVMPin*>& SubPins = Pin->GetAllSubPinsRecursively();
					for (URigVMPin* SubPin : SubPins)
					{
						if (SubPin->GetCPPTypeObject() == FAnimNextTraitHandle::StaticStruct())
						{
							PatchDefaultValue(SubPin);
						}
					}
				}

				return Pin->GetDefaultValue();
			}
		}

		// Unknown property
		return FString();
	}

	uint16 GetTraitLatentPropertyIndex(const FTraitStackMapping& TraitStack, uint32 TraitIndex, FName PropertyName)
	{
		const FTraitEntryMapping& Entry = TraitStack.TraitEntries[TraitIndex];
		if (const uint16* RigVMIndex = Entry.LatentPropertyNameToIndexMap.Find(PropertyName))
		{
			return *RigVMIndex;
		}

		return MAX_uint16;
	}

	void WriteTraitProperties(FTraitWriter& TraitWriter, const FTraitStackMapping& Mapping, const TArray<FTraitStackMapping>& TraitStackNodes)
	{
		TraitWriter.WriteNode(Mapping.TraitStackNodeHandle,
			[&Mapping, &TraitStackNodes](uint32 TraitIndex, FName PropertyName)
			{
				return GetTraitProperty(Mapping, TraitIndex, PropertyName, TraitStackNodes);
			},
			[&Mapping](uint32 TraitIndex, FName PropertyName)
			{
				return GetTraitLatentPropertyIndex(Mapping, TraitIndex, PropertyName);
			});
	}

	URigVMUnitNode* FindRootNode(const TArray<URigVMNode*>& VMNodes)
	{
		for (URigVMNode* VMNode : VMNodes)
		{
			if (URigVMUnitNode* VMUnitNode = Cast<URigVMUnitNode>(VMNode))
			{
				const UScriptStruct* ScriptStruct = VMUnitNode->GetScriptStruct();
				if (ScriptStruct == FRigUnit_AnimNextGraphRoot::StaticStruct())
				{
					return VMUnitNode;
				}
			}
		}

		return nullptr;
	}

	void AddMissingInputLinks(const URigVMPin* DecoratorPin, URigVMController* VMController)
	{
		const TArray<URigVMPin*>& Pins = DecoratorPin->GetSubPins();
		for (URigVMPin* Pin : Pins)
		{
			const ERigVMPinDirection PinDirection = Pin->GetDirection();
			if (PinDirection != ERigVMPinDirection::Input && PinDirection != ERigVMPinDirection::Hidden)
			{
				continue;	// We only look for hidden or input pins
			}

			if (Pin->GetCPPTypeObject() != FAnimNextTraitHandle::StaticStruct())
			{
				continue;	// We only look for trait handle pins
			}

			const TArray<URigVMLink*>& PinLinks = Pin->GetLinks();
			if (!PinLinks.IsEmpty())
			{
				continue;	// This pin already has a link, all good
			}

			// Add a dummy node that will output a reference pose to ensure every link is valid.
			// RigVM doesn't let us link two decorators on a same node together or linking a child back to a parent
			// as this would create a cycle in the RigVM graph. The AnimNext graph traits do support it
			// and so perhaps we could have a merging pass later on to remove useless dummy nodes like this.

			URigVMUnitNode* VMReferencePoseNode = VMController->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
			check(VMReferencePoseNode != nullptr);

			const UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();

			FString DefaultValue;
			{
				constexpr UE::UAF::FTraitUID ReferencePoseTraitUID = UE::UAF::FTraitUID::MakeUID(TEXT("FReferencePoseTrait"));	// Trait header is private, reference by UID directly
				const FTrait* Trait = FTraitRegistry::Get().Find(ReferencePoseTraitUID);
				check(Trait != nullptr);

				const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
				FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
				CppDecoratorStructInstance.DecoratorSharedDataStruct = Trait->GetTraitSharedDataStruct();

				FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_SerializedAsImportText, nullptr);
			}

			const FName ReferencePoseDecoratorName = VMController->AddTrait(VMReferencePoseNode->GetFName(), *CppDecoratorStruct->GetPathName(), TEXT("ReferencePose"), DefaultValue, INDEX_NONE, false, false);
			check(!ReferencePoseDecoratorName.IsNone());

			URigVMPin* OutputPin = VMReferencePoseNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextTraitStack, Result));
			check(OutputPin != nullptr);

			ensure(VMController->AddLink(OutputPin, Pin, false));
		}
	}

	void AddMissingInputLinks(const URigVMGraph* VMGraph, URigVMController* VMController)
	{
		const TArray<URigVMNode*> VMNodes = VMGraph->GetNodes();	// Copy since we might add new nodes
		for (URigVMNode* VMNode : VMNodes)
		{
			if (const URigVMUnitNode* VMUnitNode = Cast<URigVMUnitNode>(VMNode))
			{
				const UScriptStruct* ScriptStruct = VMUnitNode->GetScriptStruct();
				if (ScriptStruct != FRigUnit_AnimNextTraitStack::StaticStruct())
				{
					continue;	// Skip non-trait nodes
				}

				ForEachTraitInStack(VMNode,
					[VMController](const URigVMNode* DecoratorStackNode, const URigVMPin* DecoratorPin, const FTrait* Trait)
					{
						AddMissingInputLinks(DecoratorPin, VMController);
					});
			}
		}
	}

	bool ValidateTraitStack(const FRigVMCompileSettings& InSettings, const FTraitStackMapping& InMapping, URigVMUnitNode* InVMUnitNode)
	{
		bool bResult = true; 
		for (int32 TraitIndex = 0; TraitIndex < InMapping.TraitEntries.Num(); ++TraitIndex)
		{
			const FTraitEntryMapping& Entry = InMapping.TraitEntries[TraitIndex];

			const UScriptStruct* TraitStruct = Entry.Trait->GetTraitSharedDataStruct();
			if (InSettings.bWarnAboutDeprecatedNodes && TraitStruct->HasMetaData(TEXT("Deprecated")))
			{
				InSettings.ASTSettings.Report(EMessageSeverity::Warning, InVMUnitNode, FText::Format(LOCTEXT("DeprecatedTraitWarning", "@@: The trait '{0}' is deprecated. Please remove or upgrade the trait."), TraitStruct->GetDisplayNameText()).ToString());
			}

			switch (Entry.Trait->GetTraitMode())
			{
			case ETraitMode::Base:
				{
					if (Entry.Trait->GetTraitUID() == FTraitUID())
					{
						InSettings.ASTSettings.Report(EMessageSeverity::Error, InVMUnitNode, LOCTEXT("TraitStatusInStack_InvalidBaseTrait", "@@: Base trait is invalid. Please select a new base trait.").ToString());
						bResult = false;
					}
					if (Entry.Trait->GetTraitUID() != InMapping.TraitEntries[0].Trait->GetTraitUID())
					{
						InSettings.ASTSettings.Report(EMessageSeverity::Error, InVMUnitNode, LOCTEXT("TraitStatusInStack_BaseNotAtTop", "@@: Base traits should be the first trait in the stack.").ToString());
						bResult = false;
					}
					break;
				}
			case ETraitMode::Additive:
				{
					if (Entry.Trait->GetTraitUID() == FTraitUID())
					{
						InSettings.ASTSettings.Report(EMessageSeverity::Error, InVMUnitNode, LOCTEXT("TraitStatusInStack_InvalidAdditiveTrait", "@@: Additive trait is invalid. Please fix the stack.").ToString());
						bResult = false;
					}
					if (Entry.Trait->GetTraitUID() == InMapping.TraitEntries[0].Trait->GetTraitUID())
					{
						InSettings.ASTSettings.Report(EMessageSeverity::Error, InVMUnitNode, LOCTEXT("TraitStatusInStack_AdditiveAtTop", "@@: Additive traits cannot be at the top of the stack.").ToString());
						bResult = false;
					}
					break;
				}
			default:
				{
					InSettings.ASTSettings.Report(EMessageSeverity::Error, InVMUnitNode, LOCTEXT("TraitStatusInStack_InvalidTraitData", "@@: Trait is invalid, please correct the stack.").ToString());
					bResult = false;
					break;
				}
			}

			const TConstArrayView<FTraitInterfaceUID> RequiredInterfaces = Entry.Trait->GetTraitRequiredInterfaces();
			const int32 NumRequired = RequiredInterfaces.Num();
			if (NumRequired > 0)
			{
				for (int32 RequiredIndex = 0; RequiredIndex < NumRequired; RequiredIndex++)
				{
					const FTraitInterfaceUID& RequiredInterface = RequiredInterfaces[RequiredIndex];

					bool bFound = false;
					const int32 StartIndex = (Entry.Trait->GetTraitMode() == ETraitMode::Base) 
						? InMapping.TraitEntries.Num() - 1 // Base traits scan all stack to find a valid interface
						: TraitIndex; // Additive traits start search from current trait to enable traits that inherit from a trait with a required interface that is implemented in the derived class
					
					for (int32 SearchStartIndex = StartIndex; SearchStartIndex >= 0 && !bFound; SearchStartIndex--)
					{
						const FTraitEntryMapping& ParentEntry = InMapping.TraitEntries[SearchStartIndex];

						const TConstArrayView<FTraitInterfaceUID> ParentImplementedInterfaces = ParentEntry.Trait->GetTraitInterfaces();
						for (const FTraitInterfaceUID& ParentImplementedInterface : ParentImplementedInterfaces)
						{
							if (ParentImplementedInterface == RequiredInterface)
							{
								bFound = true;
								break;
							}
						}
					}
					if (!bFound)
					{
						if (const ITraitInterface* TraitInterface = FTraitInterfaceRegistry::Get().Find(RequiredInterface))
						{
							const FText InterfaceName(TraitInterface->GetDisplayName());
							const FText MissingError = FText::Format(LOCTEXT("TraitStatusInStack_MissingInterface", "@@: Trait '{0}' requires a parent implementing interface '{1}'"), Entry.Trait->GetTraitSharedDataStruct()->GetDisplayNameText(), InterfaceName);
							InSettings.ASTSettings.Report(EMessageSeverity::Warning, InVMUnitNode, MissingError.ToString());
						}
						else
						{
							const FText UID = FText::FromString(FString::Printf(TEXT("0x%x"), RequiredInterface.GetUID()));
							const FText UnknownError = FText::Format(LOCTEXT("TraitStatusInStack_UnknownInterface", "@@: Trait '{0}' requires a parent implementing unknown interface with ID '{1}'"), Entry.Trait->GetTraitSharedDataStruct()->GetDisplayNameText(), UID);
							InSettings.ASTSettings.Report(EMessageSeverity::Warning, InVMUnitNode, UnknownError.ToString());
						}
					}
				}
			}
		}

		const FTraitStackMapping& TraitStack = InMapping;
		{
			for (const FTraitEntryMapping& TraitEntry : TraitStack.TraitEntries)
			{
				for (URigVMPin* Pin : TraitEntry.DecoratorEntryPin->GetSubPins())
				{
					if (!Pin->IsLazy())
					{
						continue;
					}
					
					const bool bIsProgrammaticPin = Pin->IsProgrammaticPin();
					const bool bHasLinks = !Pin->GetLinks().IsEmpty();
					if (bHasLinks || bIsProgrammaticPin)
					{
						// This pin has something linked to it, it is a latent pin
						FString PinCPPType = Pin->GetCPPType();
						if (RigVMTypeUtils::IsUObjectType(PinCPPType))
						{
							const bool bImplementsGarbageCollection = TraitEntry.Trait->GetTraitInterface(IGarbageCollection::InterfaceUID) != nullptr;
							if (!bImplementsGarbageCollection)
							{
								const FText MissingInterfaceError = FText::Format(
									LOCTEXT("TraitStatusInStack_DoesNotImplementGarbageCollection", "@@: Trait is connected to a UObject but does not implement IGarbageCollection ({0} - {1})"),
									TraitEntry.Trait->GetTraitSharedDataStruct()->GetDisplayNameText(),
									FText::FromName(Pin->GetDisplayName()));
								InSettings.ASTSettings.Report(EMessageSeverity::Error, InVMUnitNode, MissingInterfaceError.ToString());
								bResult = false;
							}
						}
					}
				}
			}
		}
		
		return bResult;
	}

	FTraitGraph CollectGraphInfo(const FRigVMCompileSettings& InSettings, const UUAFAnimGraph* InAnimationGraph, const URigVMGraph* VMGraph, URigVMController* VMController)
	{
		const TArray<URigVMNode*>& VMNodes = VMGraph->GetNodes();
		URigVMUnitNode* VMRootNode = FindRootNode(VMNodes);

		// Pre-validate all nodes, connected or otherwise, to discover invalid trait stacks
		TMap<URigVMNode*, FTraitStackMapping> ValidNodeMappings;
		ValidNodeMappings.Reserve(VMNodes.Num());
		for (URigVMNode* VMNode : VMNodes)
		{
			if (const URigVMUnitNode* VMUnitNode = Cast<URigVMUnitNode>(VMNode))
			{
				const UScriptStruct* ScriptStruct = VMUnitNode->GetScriptStruct();
				if (ScriptStruct == FRigUnit_AnimNextTraitStack::StaticStruct())
				{
					FTraitStackMapping Mapping(VMNode);
					ForEachTraitInStack(VMNode,
						[&Mapping](const URigVMNode* DecoratorStackNode, const URigVMPin* DecoratorPin, const FTrait* Trait)
						{
							Mapping.TraitEntries.Add(FTraitEntryMapping(DecoratorStackNode, DecoratorPin, Trait));
						});

					if(ValidateTraitStack(InSettings, Mapping, const_cast<URigVMUnitNode*>(VMUnitNode)))
					{
						ValidNodeMappings.Add(VMNode, MoveTemp(Mapping));
					}
				}
			}
		}

		if (VMRootNode == nullptr)
		{
			// Root node wasn't found, add it, we'll need it to compile
			VMRootNode = VMController->AddUnitNode(FRigUnit_AnimNextGraphRoot::StaticStruct(), FRigUnit_AnimNextGraphRoot::EventName, FVector2D(0.0f, 0.0f), FString(), false);
		}

		// Make sure we don't have empty input pins
		AddMissingInputLinks(VMGraph, VMController);

		FTraitGraph TraitGraph(InAnimationGraph, VMRootNode);

		TArray<const URigVMNode*> NodesToVisit;
		NodesToVisit.Add(VMRootNode);

		while (NodesToVisit.Num() != 0)
		{
			const URigVMNode* VMNode = NodesToVisit[0];
			NodesToVisit.RemoveAt(0);

			if (FTraitStackMapping* MappingPtr = ValidNodeMappings.Find(VMNode))
			{
				TraitGraph.TraitStackNodes.Add(MoveTemp(*MappingPtr));
			}

			const TArray<URigVMNode*> SourceNodes = VMNode->GetLinkedSourceNodes();
			NodesToVisit.Append(SourceNodes);
		}

		if (TraitGraph.TraitStackNodes.IsEmpty())
		{
			// If the graph is empty, add a dummy node that just pushes a reference pose
			URigVMUnitNode* VMNode = VMController->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);

			UAnimNextController* AnimNextController = CastChecked<UAnimNextController>(VMController);
			constexpr UE::UAF::FTraitUID ReferencePoseTraitUID = UE::UAF::FTraitUID::MakeUID(TEXT("FReferencePoseTrait"));	// Trait header is private, reference by UID directly
			const FName RigVMTraitName =  AnimNextController->AddTraitByName(VMNode->GetFName(), *UE::UAF::FTraitRegistry::Get().Find(ReferencePoseTraitUID)->GetTraitName(), INDEX_NONE, TEXT(""), false);
		
			check(RigVMTraitName != NAME_None);

			FTraitStackMapping Mapping(VMNode);
			ForEachTraitInStack(VMNode,
				[&Mapping](const URigVMNode* DecoratorStackNode, const URigVMPin* DecoratorPin, const FTrait* Trait)
				{
					Mapping.TraitEntries.Add(FTraitEntryMapping(DecoratorStackNode, DecoratorPin, Trait));
				});

			TraitGraph.TraitStackNodes.Add(MoveTemp(Mapping));
		}

		return TraitGraph;
	}

	void CollectLatentPins(TArray<FTraitStackMapping>& TraitStackNodes, FRigVMPinInfoArray& OutLatentPins, TMap<FName, URigVMPin*>& OutLatentPinMapping)
	{
		for (FTraitStackMapping& TraitStack : TraitStackNodes)
		{
			for (FTraitEntryMapping& TraitEntry : TraitStack.TraitEntries)
			{
				TSharedPtr<FStructOnScope> DecoratorScope = TraitEntry.DecoratorEntryPin->GetTraitInstance();
				const FRigDecorator_AnimNextCppDecorator* Decorator = (const FRigDecorator_AnimNextCppDecorator*)DecoratorScope->GetStructMemory();
				const UScriptStruct* SharedDataStruct = Decorator->GetTraitSharedDataStruct();

				for (URigVMPin* Pin : TraitEntry.DecoratorEntryPin->GetSubPins())
				{
					if (!Pin->IsLazy())
					{
						continue;
					}
					
					// note that Pin->IsProgrammaticPin(); does not work, it does not check the shared struct
					const bool bIsProgrammaticPin = SharedDataStruct->FindPropertyByName(Pin->GetFName()) == nullptr;
					const bool bHasLinks = !Pin->GetLinks().IsEmpty();
					if (bHasLinks || bIsProgrammaticPin)
					{
						// This pin has something linked to it, it is a latent pin
						check(OutLatentPins.Num() < ((1 << 16) - 1));	// We reserve MAX_uint16 as an invalid value and we must fit on 15 bits when packed
						TraitEntry.LatentPropertyNameToIndexMap.Add(Pin->GetFName(), (uint16)OutLatentPins.Num());

						const FName LatentPinName(TEXT("LatentPin"), OutLatentPins.Num());	// Create unique latent pin names

						FRigVMPinInfo PinInfo;
						PinInfo.Name = LatentPinName;
						PinInfo.TypeIndex = Pin->GetTypeIndex();

						// All our programmatic pins are lazy inputs
						PinInfo.Direction = ERigVMPinDirection::Input;
						PinInfo.bIsLazy = true;
						PinInfo.DefaultValue = Pin->GetDefaultValue();
						PinInfo.DefaultValueType = ERigVMPinDefaultValueType::AutoDetect;

						OutLatentPins.Pins.Emplace(PinInfo);

						if (bHasLinks)
						{
							const TArray<URigVMLink*>& PinLinks = Pin->GetLinks();
							check(PinLinks.Num() == 1);

							OutLatentPinMapping.Add(LatentPinName, PinLinks[0]->GetSourcePin());
						}
						else if(bIsProgrammaticPin)
						{
							// this is a programmatic pin, we make it latent with itself, so we can remap it at trait level
							OutLatentPinMapping.Add(LatentPinName, Pin);
						}
					}
				}
			}
		}
	}

	FAnimNextGraphEvaluatorExecuteDefinition GetGraphEvaluatorExecuteMethod(const FRigVMPinInfoArray& LatentPins)
	{
		const uint32 LatentPinListHash = GetTypeHash(LatentPins);
		if (const FAnimNextGraphEvaluatorExecuteDefinition* ExecuteDefinition = FRigUnit_AnimNextGraphEvaluator::FindExecuteMethod(LatentPinListHash))
		{
			return *ExecuteDefinition;
		}

		const FRigVMRegistry& Registry = FRigVMRegistry::Get();

		// Generate a new method for this argument list
		FAnimNextGraphEvaluatorExecuteDefinition ExecuteDefinition;
		ExecuteDefinition.Hash = LatentPinListHash;
		ExecuteDefinition.MethodName = FString::Printf(TEXT("Execute_%X"), LatentPinListHash);
		ExecuteDefinition.Arguments.Reserve(LatentPins.Num());

		for (const FRigVMPinInfo& Pin : LatentPins)
		{
			const FRigVMTemplateArgumentType& TypeArg = Registry.GetType(Pin.TypeIndex);

			FAnimNextGraphEvaluatorExecuteArgument Argument;
			Argument.Name = Pin.Name.ToString();
			Argument.CPPType = TypeArg.GetBaseCPPType();

			ExecuteDefinition.Arguments.Add(Argument);
		}

		FRigUnit_AnimNextGraphEvaluator::RegisterExecuteMethod(ExecuteDefinition);

		return ExecuteDefinition;
	}
}

UUAFAnimGraph_EditorData::UUAFAnimGraph_EditorData()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetRenamed().AddUObject(this, &UUAFAnimGraph_EditorData::HandleAssetRenamed);
	}
}

void UUAFAnimGraph_EditorData::OnPreCompileAsset(FRigVMCompileSettings& InSettings)
{
	using namespace UE::UAF::UncookedOnly;

	InSettings.ASTSettings.bSetupTraits = false; // disable the default implementation of decorators for now

	UUAFAnimGraph* AnimationGraph = FUtils::GetAsset<UUAFAnimGraph>(this);

	AnimationGraph->EntryPoints.Empty();
	AnimationGraph->ResolvedRootTraitHandles.Empty();
	AnimationGraph->ResolvedEntryPoints.Empty();
	AnimationGraph->ExecuteDefinition = FAnimNextGraphEvaluatorExecuteDefinition();
	AnimationGraph->SharedDataBuffer.Empty();
	AnimationGraph->GraphReferencedObjects.Empty();
	AnimationGraph->GraphReferencedSoftObjects.Empty();
	AnimationGraph->DefaultEntryPoint = NAME_None;
}

void UUAFAnimGraph_EditorData::BuildFunctionHeadersContext(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) const
{
	using namespace UE::UAF::UncookedOnly;
	using namespace UE::UAF::UncookedOnly::Private;

	Super::BuildFunctionHeadersContext(InSettings, OutCompileContext);

	// Gather all 'call function' traits and create shim-calls for them.
	// For the compiler to pick them up if they are not public we need a calling reference to the function from a graph
	const FRigVMClient* VMClient = GetRigVMClient();
	for (const URigVMGraph* Graph : VMClient->GetAllModels(false, false))
	{
		for(const URigVMNode* Node : Graph->GetNodes())
		{
			for(const URigVMPin* TraitPin : Node->GetTraitPins())
			{
				if (TraitPin->IsExecuteContext())
				{
					continue;
				}

				TSharedPtr<FStructOnScope> ScopedTrait = Node->GetTraitInstance(TraitPin->GetFName());
				if (!ScopedTrait.IsValid())
				{
					continue;
				}

				const FRigVMTrait* Trait = (FRigVMTrait*)ScopedTrait->GetStructMemory();
				UScriptStruct* TraitSharedInstanceData = Trait->GetTraitSharedDataStruct();
				if (TraitSharedInstanceData == nullptr)
				{
					continue;
				}

				if(!TraitSharedInstanceData->IsChildOf(FAnimNextCallFunctionSharedData::StaticStruct()))
				{
					continue;
				}

				const FString DefaultValue = TraitPin->GetDefaultValue();
				TInstancedStruct<FAnimNextCallFunctionSharedData> InstancedStruct = TInstancedStruct<FAnimNextCallFunctionSharedData>::Make();
				FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
				LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity());
				TraitSharedInstanceData->ImportText(*DefaultValue, InstancedStruct.GetMutableMemory(), nullptr, PPF_SerializedAsImportText, &ErrorPipe, TraitSharedInstanceData->GetName());

				const FRigVMGraphFunctionHeader& FunctionHeader = InstancedStruct.Get<FAnimNextCallFunctionSharedData>().FunctionHeader;
				if(FunctionHeader.IsValid())
				{
					FAnimNextProgrammaticFunctionHeader AnimNextFunctionHeader = {};
					AnimNextFunctionHeader.FunctionHeader = FunctionHeader;
					OutCompileContext.AddUniqueFunctionHeader(AnimNextFunctionHeader);
				}
			}
		}
	}
}

void UUAFAnimGraph_EditorData::OnPreCompileProcessGraphs(const FRigVMCompileSettings& InSettings, FAnimNextProcessGraphCompileContext& OutCompileContext)
{
	using namespace UE::UAF::UncookedOnly;
	using namespace UE::UAF::UncookedOnly::Private;

	FRigVMClient* VMClient = GetRigVMClient();
	UUAFAnimGraph* AnimationGraph = FUtils::GetAsset<UUAFAnimGraph>(this);
	TArray<URigVMGraph*>& InOutGraphs = OutCompileContext.GetMutableAllGraphs();

	TArray<URigVMGraph*> AnimGraphs;
	TArray<URigVMGraph*> NonAnimGraphs;
	for(URigVMGraph* SourceGraph : InOutGraphs)
	{
		// We use a temporary graph models to build our final graphs that we'll compile
		if(SourceGraph->GetSchemaClass() == UAnimNextAnimationGraphSchema::StaticClass())
		{
			TMap<UObject*, UObject*> CreatedObjects;
			FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(SourceGraph, this, NAME_None, RF_Transient);
			Parameters.CreatedObjects = &CreatedObjects;
			URigVMGraph* TempGraph = CastChecked<URigVMGraph>(StaticDuplicateObjectEx(Parameters));
			TempGraph->SetExternalPackage(nullptr);
			for(URigVMNode* SourceNode : SourceGraph->GetNodes())
			{
				FCompilerResultsLog& CompileLog = FCompilationScope::GetLogForObject(AnimationGraph);
				CompileLog.NotifyIntermediateObjectCreation(CreatedObjects.FindChecked(SourceNode), SourceNode);
			}

			TempGraph->SetFlags(RF_Transient);
			AnimGraphs.Add(TempGraph);
		}
		else
		{
			NonAnimGraphs.Add(SourceGraph);
		}
	}

	if(AnimGraphs.Num() > 0)
	{
		UAnimNextController* TempController = CastChecked<UAnimNextController>(VMClient->GetOrCreateController(AnimGraphs[0]));

		UE::UAF::FTraitWriter TraitWriter;

		FRigVMPinInfoArray LatentPins;
		TMap<FName, URigVMPin*> LatentPinMapping;
		TArray<FTraitGraph> TraitGraphs;

		// Build entry points and extract their required latent pins
		for(const URigVMGraph* AnimGraph : AnimGraphs)
		{
			// Gather our trait stacks
			FTraitGraph& TraitGraph = TraitGraphs.Add_GetRef(CollectGraphInfo(InSettings, AnimationGraph, AnimGraph, TempController->GetControllerForGraph(AnimGraph)));
			check(!TraitGraph.TraitStackNodes.IsEmpty());

			FAnimNextGraphEntryPoint& EntryPoint = AnimationGraph->EntryPoints.AddDefaulted_GetRef();
			EntryPoint.EntryPointName = TraitGraph.EntryPoint;

			// Extract latent pins for this graph
			CollectLatentPins(TraitGraph.TraitStackNodes, LatentPins, LatentPinMapping);

			// Iterate over every trait stack and register our node templates
			for (FTraitStackMapping& NodeMapping : TraitGraph.TraitStackNodes)
			{
				NodeMapping.TraitStackNodeHandle = RegisterTraitNodeTemplate(TraitWriter, NodeMapping.DecoratorStackNode);
			}

			// Find our root node handle, if we have any stack nodes, the first one is our root stack
			if (TraitGraph.TraitStackNodes.Num() != 0)
			{
				EntryPoint.RootTraitHandle = FAnimNextEntryPointHandle(TraitGraph.TraitStackNodes[0].TraitStackNodeHandle);
			}
		}

		// Set default entry point
		if(AnimationGraph->EntryPoints.Num() > 0)
		{
			AnimationGraph->DefaultEntryPoint = AnimationGraph->EntryPoints[0].EntryPointName;
		}

		// Remove our old root nodes
		for (FTraitGraph& TraitGraph : TraitGraphs)
		{
			URigVMController* GraphController = TempController->GetControllerForGraph(TraitGraph.RootNode->GetGraph());
			GraphController->RemoveNode(TraitGraph.RootNode, false, false);
		}

		if(LatentPins.Num() > 0)
		{
			// We need a unique method name to match our unique argument list
			AnimationGraph->ExecuteDefinition = GetGraphEvaluatorExecuteMethod(LatentPins);

			// Add our runtime shim root node
			URigVMUnitNode* TempShimRootNode = TempController->AddUnitNode(FRigUnit_AnimNextShimRoot::StaticStruct(), FRigUnit_AnimNextShimRoot::EventName, FVector2D::ZeroVector, FString(), false);
			URigVMUnitNode* GraphEvaluatorNode = TempController->AddUnitNodeWithPins(FRigUnit_AnimNextGraphEvaluator::StaticStruct(), LatentPins, *AnimationGraph->ExecuteDefinition.MethodName, FVector2D::ZeroVector, FString(), false);

			// Link our shim and evaluator nodes together using the execution context
			TempController->AddLink(
				TempShimRootNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextShimRoot, ExecuteContext)),
				GraphEvaluatorNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphEvaluator, ExecuteContext)),
				false);

			// Link our latent pins
			for (const FRigVMPinInfo& LatentPin : LatentPins)
			{
				TempController->AddLink(
					LatentPinMapping[LatentPin.Name],
					GraphEvaluatorNode->FindPin(LatentPin.Name.ToString()),
					false);
			}
		}

		// Write our node shared data
		TraitWriter.BeginNodeWriting();

		for(FTraitGraph& TraitGraph : TraitGraphs)
		{
			for (const FTraitStackMapping& NodeMapping : TraitGraph.TraitStackNodes)
			{
				WriteTraitProperties(TraitWriter, NodeMapping, TraitGraph.TraitStackNodes);
			}
		}

		TraitWriter.EndNodeWriting();

		// Cache our compiled metadata
		AnimationGraph->SharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
		AnimationGraph->GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
		AnimationGraph->GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();

		// Populate our runtime metadata
		AnimationGraph->LoadFromArchiveBuffer(AnimationGraph->SharedDataArchiveBuffer);
	}

	InOutGraphs = MoveTemp(AnimGraphs);
	InOutGraphs.Append(NonAnimGraphs);
}

void UUAFAnimGraph_EditorData::OnCompileJobStarted()
{
	using namespace UE::UAF::UncookedOnly;

	UUAFAnimGraph* AnimationGraph = FUtils::GetAsset<UUAFAnimGraph>(this);

	// Before we re-compile a graph, we need to release and live instances since we need the metadata we are about to replace
	// to call trait destructors etc
	AnimationGraph->FreezeGraphInstances();
}

void UUAFAnimGraph_EditorData::OnCompileJobFinished()
{
	using namespace UE::UAF::UncookedOnly;

	UUAFAnimGraph* AnimationGraph = FUtils::GetAsset<UUAFAnimGraph>(this);

	// Now that the graph has been re-compiled, re-allocate the previous live instances
	AnimationGraph->ThawGraphInstances();
}

TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> UUAFAnimGraph_EditorData::GetEntryClasses() const
{
	static const TSubclassOf<UUAFRigVMAssetEntry> Classes[] =
	{
		UAnimNextAnimationGraphEntry::StaticClass(),
		UAnimNextVariableEntry::StaticClass(),
		UUAFSharedVariablesEntry::StaticClass(),
	};

	return Classes;
}

bool UUAFAnimGraph_EditorData::CanAddNewEntry(TSubclassOf<UUAFRigVMAssetEntry> InClass) const
{
	// Prevent users adding more than one animation graph
	if(InClass == UAnimNextAnimationGraphEntry::StaticClass())
	{
		auto IsAnimNextGraphEntry = [](const UUAFRigVMAssetEntry* InEntry)
		{
			if (InEntry)
			{
				return InEntry->IsA<UAnimNextAnimationGraphEntry>();
			}

			return false;
		};

		if(Entries.ContainsByPredicate(IsAnimNextGraphEntry))
		{
			return false;
		};
	}
	
	return true;
}


UAnimNextAnimationGraphEntry* UAnimNextAnimationGraphLibrary::AddAnimationGraph(UUAFAnimGraph* InAsset, FName InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(InAsset)->AddAnimationGraph(InName, bSetupUndoRedo, bPrintPythonCommand);
}

UAnimNextAnimationGraphEntry* UUAFAnimGraph_EditorData::AddAnimationGraph(FName InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InName == NAME_None)
	{
		ReportError(TEXT("UUAFAnimGraph_EditorData::AddAnimationGraph: Invalid graph name supplied."));
		return nullptr;
	}

	if(!GetEntryClasses().Contains(UAnimNextAnimationGraphEntry::StaticClass()) || !CanAddNewEntry(UAnimNextAnimationGraphEntry::StaticClass()))
	{
		ReportError(TEXT("UUAFAnimGraph_EditorData::AddAnimationGraph: Cannot add an animation graph to this asset - entry is not allowed."));
		return nullptr;
	}

	// Check for duplicate name
	FName NewGraphName = InName;
	auto DuplicateNamePredicate = [&NewGraphName](const UUAFRigVMAssetEntry* InEntry)
	{
		return InEntry->GetEntryName() == NewGraphName;
	};

	bool bAlreadyExists = Entries.ContainsByPredicate(DuplicateNamePredicate);
	int32 NameNumber = InName.GetNumber() + 1;
	while(bAlreadyExists)
	{
		NewGraphName = FName(InName, NameNumber++);
		bAlreadyExists =  Entries.ContainsByPredicate(DuplicateNamePredicate);
	}

	UAnimNextAnimationGraphEntry* NewEntry = CreateNewSubEntry<UAnimNextAnimationGraphEntry>(this);
	NewEntry->GraphName = NewGraphName;
	NewEntry->Initialize(this);

	if(bSetupUndoRedo)
	{
		NewEntry->Modify();
		Modify();
	}

	AddEntryInternal(NewEntry);

	// Add new graph
	{
		TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
		TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);
		// Editor data has to be the graph outer, or RigVM unique name generator will not work
		URigVMGraph* NewRigVMGraphModel = RigVMClient.CreateModel(URigVMGraph::StaticClass()->GetFName(), UAnimNextAnimationGraphSchema::StaticClass(), bSetupUndoRedo, this);
		if (ensure(NewRigVMGraphModel))
		{
			// Then, to avoid the graph losing ref due to external package, set the same package as the Entry
			if (!NewRigVMGraphModel->HasAnyFlags(RF_Transient))
			{
				NewRigVMGraphModel->SetExternalPackage(CastChecked<UObject>(NewEntry)->GetExternalPackage());
			}
			ensure(NewRigVMGraphModel);
			NewEntry->Graph = NewRigVMGraphModel;

			RefreshExternalModels();
			RigVMClient.AddModel(NewRigVMGraphModel, true);
			URigVMController* Controller = RigVMClient.GetController(NewRigVMGraphModel);
			UE::UAF::UncookedOnly::FAnimGraphUtils::SetupAnimGraph(NewEntry->GetEntryName(), Controller);
		}
	}

	CustomizeNewAssetEntry(NewEntry);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, NewEntry);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_animation_graph('%s')"),
				*InName.ToString()));
	}

	return NewEntry;
}

TSubclassOf<UAssetUserData> UUAFAnimGraph_EditorData::GetAssetUserDataClass() const
{
	return UAnimNextAnimGraphWorkspaceAssetUserData::StaticClass();
}

void UUAFAnimGraph_EditorData::InitializeAssetUserData()
{
	// Here we switch user data classes to patch up old assets
	if (IInterface_AssetUserData* OuterUserData = Cast<IInterface_AssetUserData>(GetOuter()))
	{
		if(!OuterUserData->HasAssetUserDataOfClass(GetAssetUserDataClass()))
		{
			UAnimNextAssetWorkspaceAssetUserData* ExistingUserData = Cast<UAnimNextAssetWorkspaceAssetUserData>(OuterUserData->GetAssetUserDataOfClass(UAnimNextAssetWorkspaceAssetUserData::StaticClass()));
			if(ExistingUserData)
			{
				OuterUserData->RemoveUserDataOfClass(UAnimNextAssetWorkspaceAssetUserData::StaticClass());
			}
		}
	}

	Super::InitializeAssetUserData();
}

UClass* UUAFAnimGraph_EditorData::GetRigVMEdGraphSchemaClass() const
{
	return UUAFAnimGraphEdGraphSchema::StaticClass();
}

void UUAFAnimGraph_EditorData::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	Super::HandleModifiedEvent(InNotifType, InGraph, InSubject);

	switch (InNotifType)
	{
	case ERigVMGraphNotifType::PinDefaultValueChanged:
		{
			URigVMPin* Pin = CastChecked<URigVMPin>(InSubject);
			if (UAnimNextTraitStackUnitNode* TraitStack = Cast<UAnimNextTraitStackUnitNode>(Pin->GetNode()))
			{
				TraitStack->HandlePinDefaultValueChanged(CastChecked<UAnimNextController>(GetController(InGraph)), Pin);
			}
			break;
		}
	default:
		break;
	}
}

void UUAFAnimGraph_EditorData::HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldName)
{
	for (URigVMGraph* Graph : GetAllModels())
	{
		if(UAnimNextController* Controller = Cast<UAnimNextController>(GetController(Graph)))
		{
			for (URigVMNode* Node : Graph->GetNodes())
			{
				if (UAnimNextTraitStackUnitNode* TraitStack = Cast<UAnimNextTraitStackUnitNode>(Node))
				{
					TraitStack->HandleAssetRenamed(Controller, InAssetData, InOldName);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
