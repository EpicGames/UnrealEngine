// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_AnimNextPublicVariablesUncookedOnly.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "UncookedOnlyUtils.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMPin.h"
#include "Variables/AnimNextSharedVariables.h"

namespace UE::UAF::UncookedOnly
{

void FPublicVariablesImpl::Register()
{
	FRigVMTrait_AnimNextPublicVariables::GetDisplayNameFunc = GetDisplayName;
	FRigVMTrait_AnimNextPublicVariables::GetProgrammaticPinsFunc = GetProgrammaticPins;
	FRigVMTrait_AnimNextPublicVariables::ShouldCreatePinForPropertyFunc = ShouldCreatePinForProperty;
}

FString FPublicVariablesImpl::GetDisplayName(const FRigVMTrait_AnimNextPublicVariables& InTrait)
{
	TStringBuilder<256> StringBuilder;
	StringBuilder.Appendf(TEXT("Variables: %s"), InTrait.InternalAsset ? *InTrait.InternalAsset->GetFName().ToString() : TEXT("None"));
	return StringBuilder.ToString();
}

void FPublicVariablesImpl::GetProgrammaticPins(const FRigVMTrait_AnimNextPublicVariables& InTrait, URigVMController* InController, int32 InParentPinIndex, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray)
{
	if (InTrait.InternalAsset == nullptr)
	{
		return;
	}

	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InTrait.InternalAsset.Get());
	if(EditorData == nullptr)
	{
		return;
	}

	TArray<UUAFRigVMAssetEditorData::FVariableInfo> PublicVariablesInfoArray;
	EditorData->GetAllVariables(PublicVariablesInfoArray, UUAFRigVMAssetEditorData::EVariableRecursion::SelfOnly, UUAFRigVMAssetEditorData::EVariableAccessFilter::PublicOnly);
	if(PublicVariablesInfoArray.Num() == 0)
	{
		return;
	}

	TArray<FPropertyBagPropertyDesc> Descs;
	Descs.Reserve(PublicVariablesInfoArray.Num());
	TArray<TConstArrayView<uint8>> Values;
	Values.Reserve(PublicVariablesInfoArray.Num());

	// Maintain trait sorted order
	for (FName VariableName : InTrait.InternalVariableNames)
	{
		auto FindMatchingName = [&VariableName](const UUAFRigVMAssetEditorData::FVariableInfo& InVariableInfo)
		{
			return InVariableInfo.Name == VariableName;
		};

		if (UUAFRigVMAssetEditorData::FVariableInfo* PublicVariablePtr = PublicVariablesInfoArray.FindByPredicate(FindMatchingName))
		{
			Descs.Emplace(PublicVariablePtr->Name, PublicVariablePtr->Property);
			Values.Emplace(PublicVariablePtr->DefaultValue);
		}

	}

	ensure(Values.Num() == InTrait.InternalVariableNames.Num());

	FInstancedPropertyBag Defaults;
	Defaults.ReplaceAllPropertiesAndValues(Descs, Values);

	const TFunction<ERigVMPinDefaultValueType(const FName&)> DefaultValueTypeGetter = [](const FName& InPropertyName)
	{
		return ERigVMPinDefaultValueType::AutoDetect;
	};

	OutPinArray.AddPins(const_cast<UPropertyBag*>(Defaults.GetPropertyBagStruct()), InController, ERigVMPinDirection::Input, InParentPinIndex, DefaultValueTypeGetter, Defaults.GetValue().GetMemory(), true);
}

bool FPublicVariablesImpl::ShouldCreatePinForProperty(const FRigVMTrait_AnimNextPublicVariables& InTrait, const FProperty* InProperty)
{
	return
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FRigVMTrait_AnimNextPublicVariables, InternalAsset) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FRigVMTrait_AnimNextPublicVariables, InternalVariableNames) ||
		InTrait.InternalVariableNames.Contains(InProperty->GetFName());
}

}
