// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

struct FRigVMTrait_AnimNextPublicVariables;
class URigVMController;
struct FRigVMPinInfoArray;

namespace UE::UAF::UncookedOnly
{

// UncookedOnly-side impl for FRigVMTrait_AnimNextPublicVariables
struct FPublicVariablesImpl
{
	static UE_API void Register();
	static UE_API FString GetDisplayName(const FRigVMTrait_AnimNextPublicVariables& InTrait);
	static UE_API void GetProgrammaticPins(const FRigVMTrait_AnimNextPublicVariables& InTrait, URigVMController* InController, int32 InParentPinIndex, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray);
	static UE_API bool ShouldCreatePinForProperty(const FRigVMTrait_AnimNextPublicVariables& InTrait, const FProperty* InProperty);
};

}

#undef UE_API
