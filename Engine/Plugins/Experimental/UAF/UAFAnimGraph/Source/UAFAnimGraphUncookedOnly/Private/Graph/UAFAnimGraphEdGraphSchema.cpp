// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimGraphEdGraphSchema.h"

#include "AnimNextController.h"
#include "EdGraph/RigVMEdGraph.h"
#include "Editor/AssetReferenceFilter.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Templates/GraphNodeTemplateRegistry.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "TraitCore/TraitHandle.h"

#define LOCTEXT_NAMESPACE "UAFAnimGraphEdGraphSchema"

void UUAFAnimGraphEdGraphSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	using namespace UE::UAF;
	
	URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(Graph);
	
	UAnimNextController* Controller = Cast<UAnimNextController>(EdGraph->GetController());
	if (Controller == nullptr)
	{
		return;
	}

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);

	auto HandleDrop = [Controller](UUAFGraphNodeTemplate* InTemplate, UObject* InAsset, FVector2D InLocation)
	{
		Controller->OpenUndoBracket(TEXT("Drop assets on graph"));
			
		URigVMUnitNode* NewNode = InTemplate->CreateNewNode(Controller, InLocation);
		if (NewNode == nullptr)
		{
			Controller->CancelUndoBracket();
			return;
		}

		{
			FEditorScriptExecutionGuard AllowScripts;
			InTemplate->HandleAssetDropped(Controller, NewNode, InAsset);
		}

		Controller->CloseUndoBracket();
	};

	TSet<FTopLevelAssetPath> MenuHandlers;
	bool bShowMenu = false;
	FMenuBuilder MenuBuilder(true, nullptr);

	// See if any of the known templates know how to handle the dropped assets
	for (const FAssetData& AssetData : Assets)
	{
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
		{
			continue;
		}

		TConstArrayView<FGraphNodeTemplateInfo> Handlers = FGraphNodeTemplateRegistry::GetDragDropHandlersForAsset(AssetData);
		if (Handlers.Num() > 1)
		{
			bShowMenu = true;

			for (const FGraphNodeTemplateInfo& Handler : Handlers)
			{
				FSoftObjectPath SoftClassPath(Handler.ClassPath);
				UClass* Class = Cast<UClass>(SoftClassPath.TryLoad());
				if (Class == nullptr)
				{
					continue;
				}

				if (MenuHandlers.Contains(Handler.ClassPath))
				{
					continue;
				}

				MenuHandlers.Add(Handler.ClassPath);
				
				UUAFGraphNodeTemplate* Template = Class->GetDefaultObject<UUAFGraphNodeTemplate>();

				FEditorScriptExecutionGuard AllowScripts;
				FText MenuDesc = Template->GetMenuDescription();
				FText Tooltip = Template->GetTooltipText();
				MenuBuilder.AddMenuEntry(MenuDesc, Tooltip, FSlateIcon(), FExecuteAction::CreateLambda([Template, AssetReferenceFilter, HandleDrop, Assets, GraphPosition]()
				{
					FVector2D Location = FVector2D(GraphPosition);
					for (const FAssetData& AssetData : Assets)
					{
						if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
						{
							continue;
						}

						UObject* Asset = AssetData.GetAsset();
						if (Asset == nullptr)
						{
							continue;
						}

						HandleDrop(Template, Asset, Location);

						// Offset multiple spawned nodes
						Location.X += 100.0f;
						Location.Y += 100.0f;
					}
				}));
			}
		}
	}

	// If there are multiple handlers to choose from, show menu for user choice
	if (bShowMenu)
	{
		FSlateApplication::Get().PushMenu(
			FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
			FWidgetPath(),
			MenuBuilder.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	}
	else
	{
		FVector2D Location = FVector2D(GraphPosition);
		for (const FAssetData& AssetData : Assets)
		{
			if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
			{
				continue;
			}

			UObject* Asset = AssetData.GetAsset();
			if (Asset == nullptr)
			{
				continue;
			}

			TConstArrayView<FGraphNodeTemplateInfo> Handlers = FGraphNodeTemplateRegistry::GetDragDropHandlersForAsset(AssetData);
			if (Handlers.Num() == 1)
			{
				FSoftObjectPath SoftClassPath(Handlers[0].ClassPath);
				UClass* Class = Cast<UClass>(SoftClassPath.TryLoad());
				if (Class != nullptr)
				{
					HandleDrop(Class->GetDefaultObject<UUAFGraphNodeTemplate>(), Asset, Location);

					// Offset multiple spawned nodes
					Location.X += 100.0f;
					Location.Y += 100.0f;
				}
			}
		}
	}
}

void UUAFAnimGraphEdGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	using namespace UE::UAF;

	OutTooltipText.Reset();
	OutOkIcon = false;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
	for (const FAssetData& AssetData : Assets)
	{
		TConstArrayView<FGraphNodeTemplateInfo> Handlers = FGraphNodeTemplateRegistry::GetDragDropHandlersForAsset(AssetData);
		if (Handlers.Num() > 0)
		{
			if (AssetReferenceFilter)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
				{
					if (OutTooltipText.IsEmpty())
					{
						OutTooltipText = FailureReason.ToString();
					}
					continue;
				}
			}

			OutOkIcon = true;
			OutTooltipText = (Assets.Num() > 0 ? LOCTEXT("DragDropAssetMessage", "Create node for asset") : LOCTEXT("DragDropAssetsMessage", "Create nodes for assets")).ToString();
			return;
		}
	}
}

FLinearColor UUAFAnimGraphEdGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory != UEdGraphSchema_K2::PC_Struct)
	{
		return Super::GetPinTypeColor(PinType);
	}

	UScriptStruct* Struct = Cast<UScriptStruct>(PinType.PinSubCategoryObject);
	if (Struct == nullptr)
	{
		return Super::GetPinTypeColor(PinType);
	}

	if (!Struct->IsChildOf<FAnimNextTraitHandle>())
	{
		return Super::GetPinTypeColor(PinType);
	}

	return FLinearColor::White;
}

#undef LOCTEXT_NAMESPACE