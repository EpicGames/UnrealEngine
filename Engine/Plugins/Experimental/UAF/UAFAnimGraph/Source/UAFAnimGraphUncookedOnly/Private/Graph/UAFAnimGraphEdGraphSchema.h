// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextEdGraphSchema.h"
#include "UAFAnimGraphEdGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UUAFAnimGraphEdGraphSchema : public UAnimNextEdGraphSchema
{
	GENERATED_BODY()
	
	// UEdGraphSchema interface
	virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
};