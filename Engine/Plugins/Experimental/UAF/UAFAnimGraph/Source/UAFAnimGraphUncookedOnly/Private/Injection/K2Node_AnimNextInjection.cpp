// Copyright Epic Games, Inc. All Rights Reserved.

#include "Injection/K2Node_AnimNextInjection.h"
#include "Injection/InjectionCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AnimNextInjection)

#define LOCTEXT_NAMESPACE "K2Node_AnimNextInjection"

UK2Node_AnimNextInjection::UK2Node_AnimNextInjection()
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInjectionCallbackProxy, CreateProxyObjectForInjection);
	ProxyFactoryClass = UInjectionCallbackProxy::StaticClass();
	ProxyClass = UInjectionCallbackProxy::StaticClass();
}

FText UK2Node_AnimNextInjection::GetTooltipText() const
{
	return LOCTEXT("K2Node_Injection_Tooltip", "Injects animation into an AnimNext component");
}

FText UK2Node_AnimNextInjection::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Inject", "Inject");
}

FText UK2Node_AnimNextInjection::GetMenuCategory() const
{
	return LOCTEXT("InjectCategory", "Animation|AnimNext");
}

#undef LOCTEXT_NAMESPACE
