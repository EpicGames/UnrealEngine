// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Variables/K2Node_AnimNextComponentSetVariable.h"
#include "Injection/InjectionCallbackProxy.h"
#include "K2Node_InjectionCallbackProxySetVariable.generated.h"

/** A custom node that supports SetVariable functionality */
UCLASS()
class UK2Node_InjectionCallbackProxySetVariable : public UK2Node_AnimNextComponentSetVariable
{
	GENERATED_BODY()

public:
	UK2Node_InjectionCallbackProxySetVariable()
	{
		FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UInjectionCallbackProxy, SetVariable), UInjectionCallbackProxy::StaticClass());
	}
};
