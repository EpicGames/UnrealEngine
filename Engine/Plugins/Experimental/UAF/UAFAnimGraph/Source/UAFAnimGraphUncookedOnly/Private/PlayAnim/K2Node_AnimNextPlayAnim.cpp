// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayAnim/K2Node_AnimNextPlayAnim.h"

#include "PlayAnim/PlayAnimCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AnimNextPlayAnim)

#define LOCTEXT_NAMESPACE "K2Node_AnimNextPlayAnim"

UK2Node_AnimNextPlayAnim::UK2Node_AnimNextPlayAnim()
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UPlayAnimCallbackProxy, CreateProxyObjectForPlayAnim);
	ProxyFactoryClass = UPlayAnimCallbackProxy::StaticClass();
	ProxyClass = UPlayAnimCallbackProxy::StaticClass();
}

FText UK2Node_AnimNextPlayAnim::GetTooltipText() const
{
	return LOCTEXT("K2Node_PlayAnim_Tooltip", "Plays an Animation Sequence on an AnimNextComponent");
}

FText UK2Node_AnimNextPlayAnim::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("PlayAnim", "Play Animation");
}

FText UK2Node_AnimNextPlayAnim::GetMenuCategory() const
{
	return LOCTEXT("PlayAnimCategory", "Animation|AnimNext");
}

#undef LOCTEXT_NAMESPACE
