// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayAnim/K2Node_AnimNextPlayAsset.h"

#include "PlayAnim/PlayAnimCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AnimNextPlayAsset)

#define LOCTEXT_NAMESPACE "K2Node_AnimNextPlayAsset"

UDEPRECATED_K2Node_AnimNextPlayAsset::UDEPRECATED_K2Node_AnimNextPlayAsset()
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UPlayAnimCallbackProxy, CreateProxyObjectForPlayAnim);
	ProxyFactoryClass = UPlayAnimCallbackProxy::StaticClass();
	ProxyClass = UPlayAnimCallbackProxy::StaticClass();
}

FText UDEPRECATED_K2Node_AnimNextPlayAsset::GetTooltipText() const
{
	return LOCTEXT("K2Node_PlayAsset_Tooltip", "Plays an asset on an AnimNextComponent");
}

FText UDEPRECATED_K2Node_AnimNextPlayAsset::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("PlayAsset", "Play Asset");
}

FText UDEPRECATED_K2Node_AnimNextPlayAsset::GetMenuCategory() const
{
	return LOCTEXT("PlayAssetCategory", "Animation|AnimNext");
}

#undef LOCTEXT_NAMESPACE
