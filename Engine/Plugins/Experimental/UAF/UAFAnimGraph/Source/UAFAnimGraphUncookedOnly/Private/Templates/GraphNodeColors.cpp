// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/GraphNodeColors.h"

namespace UE::UAF::UncookedOnly
{

FLinearColor FGraphNodeColors::Blends(0.2f, 0.8f, 0.2f);
FLinearColor FGraphNodeColors::Controls(0.8f, 0.2f, 0.2f);
FLinearColor FGraphNodeColors::Generators(0.2f, 0.2f, 0.8f);
FLinearColor FGraphNodeColors::SubGraphs(0.6f, 0.6f, 0.6f);

}