// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/GraphNodeTemplateRegistry.h"

#include "Containers/Array.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "UObject/ObjectKey.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Blueprint/BlueprintSupport.h"
#include "String/ParseTokens.h"
#include "Param/ParamType.h"

namespace UE::UAF
{

namespace Private
{
	// All known templates
	static TMap<FTopLevelAssetPath, FGraphNodeTemplateInfo> GAllTemplates;

	// Map from asset type to drag-drop handlers for that asset
	static TMap<FTopLevelAssetPath, TArray<FGraphNodeTemplateInfo>> GAssetDragDropHandlers;
	
	// Map from variable type to drag-drop handlers for that variable type
	static TMap<FAnimNextParamType, TArray<FGraphNodeTemplateInfo>> GVariableDragDropHandlers;

	// Make sure we dont initialize more than once
	static bool bGIsTemplateRegistryInitialized = false;

	// Handles for delegate hooks to maintain the registry
	static FDelegateHandle GOnAssetsAddedHandle;
	static FDelegateHandle GOnAssetsUpdatedHandle;
	static FDelegateHandle GOnAssetsRemovedHandle;
	static FDelegateHandle GOnAssetRenamedHandle;
	static FDelegateHandle GOnObjectPostCDOCompiledHandle;
}

void FGraphNodeTemplateRegistry::LazyInitialize()
{
	if (Private::bGIsTemplateRegistryInitialized)
	{
		return;
	}

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	auto IsTemplateAsset = [](const FAssetData& InAssetData, FTopLevelAssetPath& OutClassPath)
	{
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		if (InAssetData.GetClass() != UBlueprint::StaticClass())
		{
			return false;
		}

		FString GeneratedClassPathString;
		if (!InAssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, GeneratedClassPathString))
		{
			return false;
		}

		OutClassPath = FTopLevelAssetPath(GeneratedClassPathString);
		TArray<FTopLevelAssetPath> Ancestors;
		AssetRegistry.GetAncestorClassNames(OutClassPath, Ancestors);
		return Ancestors.Contains(UUAFGraphNodeTemplate::StaticClass()->GetClassPathName());
	};

	auto GetTemplateInfoFromAsset = [](const FAssetData& InAssetData, const FTopLevelAssetPath& InClassPath)
	{
		FGraphNodeTemplateInfo GraphNodeTemplateInfo;
		GraphNodeTemplateInfo.ClassPath = InClassPath;
		GraphNodeTemplateInfo.MenuDescription = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UUAFGraphNodeTemplate, MenuDescription));
		GraphNodeTemplateInfo.Tooltip = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UUAFGraphNodeTemplate, TooltipText));
		GraphNodeTemplateInfo.Category = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UUAFGraphNodeTemplate, Category));
		FString DragDropAssetTypesString = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UUAFGraphNodeTemplate, DragDropAssetTypes));
		DragDropAssetTypesString.TrimCharInline(TEXT('('), nullptr);
		DragDropAssetTypesString.TrimCharInline(TEXT(')'), nullptr);
		UE::String::ParseTokensMultiple(DragDropAssetTypesString, TEXT(","), [&GraphNodeTemplateInfo](FStringView InView)
		{
			FString DragDropAssetType(InView);
			DragDropAssetType.TrimQuotesInline();
			GraphNodeTemplateInfo.DragDropAssetTypes.Add(FTopLevelAssetPath(DragDropAssetType));
		}, UE::String::EParseTokensOptions::Trim | UE::String::EParseTokensOptions::SkipEmpty);

		return GraphNodeTemplateInfo;
	};

	auto AddOrUpdateAssets = [&IsTemplateAsset, &GetTemplateInfoFromAsset](TConstArrayView<FAssetData> InAssetDataView)
	{
		for (const FAssetData& AssetData : InAssetDataView)
		{
			FTopLevelAssetPath ClassPath;
			if (IsTemplateAsset(AssetData, ClassPath))
			{
				FGraphNodeTemplateInfo NewInfo = GetTemplateInfoFromAsset(AssetData, ClassPath);
				Private::GAllTemplates.Add(ClassPath, NewInfo);
			}
		}
	};

	TArray<FAssetData> InitialAssetData;
	AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), InitialAssetData);
	AddOrUpdateAssets(InitialAssetData);

	Private::GOnAssetsAddedHandle = AssetRegistry.OnAssetsAdded().AddLambda(AddOrUpdateAssets);
	Private::GOnAssetsUpdatedHandle = AssetRegistry.OnAssetsUpdated().AddLambda(AddOrUpdateAssets);
	Private::GOnAssetsRemovedHandle = AssetRegistry.OnAssetsRemoved().AddLambda([&IsTemplateAsset](TConstArrayView<FAssetData> InAssetDataView)
	{
		for (const FAssetData& AssetData : InAssetDataView)
		{
			FTopLevelAssetPath ClassPath;
			if (IsTemplateAsset(AssetData, ClassPath))
			{
				Private::GAllTemplates.Remove(ClassPath);
			}
		}
	});
	Private::GOnAssetRenamedHandle = AssetRegistry.OnAssetRenamed().AddLambda([&IsTemplateAsset, &GetTemplateInfoFromAsset](const FAssetData& InAssetData, const FString& InOldName)
	{
		FTopLevelAssetPath ClassPath;
		if (IsTemplateAsset(InAssetData, ClassPath))
		{
			FTopLevelAssetPath OldClassPath(InOldName);

			// Remove old drag-drop handlers
			if (FGraphNodeTemplateInfo* OldTemplateInfo = Private::GAllTemplates.Find(OldClassPath))
			{
				for (const FTopLevelAssetPath& AssetType : OldTemplateInfo->DragDropAssetTypes)
				{
					TArray<FGraphNodeTemplateInfo>& Handlers = Private::GAssetDragDropHandlers.Add(AssetType);
					Handlers.RemoveAll([OldTemplateInfo](const FGraphNodeTemplateInfo& InTemplateInfo)
					{
						return InTemplateInfo.ClassPath == OldTemplateInfo->ClassPath;
					});
				}
			}

			// Remove old template
			Private::GAllTemplates.Remove(OldClassPath);

			FGraphNodeTemplateInfo NewInfo = GetTemplateInfoFromAsset(InAssetData, ClassPath);

			// Add new drag-drop handlers
			if (NewInfo.DragDropAssetTypes.Num() > 0)
			{
				for (const FTopLevelAssetPath& AssetType : NewInfo.DragDropAssetTypes)
				{
					TArray<FGraphNodeTemplateInfo>& Handlers = Private::GAssetDragDropHandlers.Add(AssetType);
					Handlers.Add(NewInfo);
				}
			}

			// Add new template
			Private::GAllTemplates.Add(ClassPath, MoveTemp(NewInfo));
		}
	});

	// Subscribe to BP changes
	Private::GOnObjectPostCDOCompiledHandle = FCoreUObjectDelegates::OnObjectPostCDOCompiled.AddLambda([&IsTemplateAsset, &GetTemplateInfoFromAsset](UObject* InObject, const FObjectPostCDOCompiledContext&)
	{
		if (InObject->IsA<UUAFGraphNodeTemplate>() && InObject->GetClass()->ClassGeneratedBy)
		{
			FAssetData AssetData(InObject->GetClass()->ClassGeneratedBy);
			FTopLevelAssetPath ClassPath;
			if (IsTemplateAsset(AssetData, ClassPath))
			{
				// Remove old drag-drop handlers, we might have renewed them
				if (FGraphNodeTemplateInfo* ExistingTemplateInfo = Private::GAllTemplates.Find(ClassPath))
				{
					for (const FTopLevelAssetPath& AssetType : ExistingTemplateInfo->DragDropAssetTypes)
					{
						TArray<FGraphNodeTemplateInfo>& Handlers = Private::GAssetDragDropHandlers.Add(AssetType);
						Handlers.RemoveAll([ExistingTemplateInfo](const FGraphNodeTemplateInfo& InTemplateInfo)
						{
							return InTemplateInfo.ClassPath == ExistingTemplateInfo->ClassPath;
						});
					}
				}

				FGraphNodeTemplateInfo NewInfo = GetTemplateInfoFromAsset(AssetData, ClassPath);

				// Add new drag-drop handlers
				if (NewInfo.DragDropAssetTypes.Num() > 0)
				{
					for (const FTopLevelAssetPath& AssetType : NewInfo.DragDropAssetTypes)
					{
						TArray<FGraphNodeTemplateInfo>& Handlers = Private::GAssetDragDropHandlers.Add(AssetType);
						Handlers.Add(NewInfo);
					}
				}

				Private::GAllTemplates.Add(ClassPath, MoveTemp(NewInfo));
			}
		}
	});

	// Get all the native classes
	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(UUAFGraphNodeTemplate::StaticClass(), DerivedClasses);

	for (UClass* Class : DerivedClasses)
	{
		if (!Class->HasAnyClassFlags(CLASS_Native))
		{
			continue;
		}

		// Ignore deprecated and temporary trash classes.
		if (Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden) ||
			FBlueprintSupport::IsClassPlaceholder(Class) ||
			FKismetEditorUtilities::IsClassABlueprintSkeleton(Class))
		{
			continue;
		}

		UUAFGraphNodeTemplate* CDO = Class->GetDefaultObject<UUAFGraphNodeTemplate>();

		FTopLevelAssetPath ClassPath(Class);
		
		FGraphNodeTemplateInfo GraphNodeTemplateInfo;
		GraphNodeTemplateInfo.ClassPath = ClassPath;
		GraphNodeTemplateInfo.MenuDescription = CDO->GetMenuDescription();
		GraphNodeTemplateInfo.Tooltip = CDO->GetTooltipText();
		GraphNodeTemplateInfo.Category = CDO->GetCategory();
		GraphNodeTemplateInfo.DragDropAssetTypes = CDO->GetDragDropAssetTypes();
		GraphNodeTemplateInfo.DragDropVariableTypes = CDO->GetDragDropVariableTypes();

		if (GraphNodeTemplateInfo.DragDropAssetTypes.Num() > 0)
		{
			for (const FTopLevelAssetPath& AssetType : GraphNodeTemplateInfo.DragDropAssetTypes)
			{
				TArray<FGraphNodeTemplateInfo>& Handlers = Private::GAssetDragDropHandlers.FindOrAdd(AssetType);
				Handlers.Add(GraphNodeTemplateInfo);
			}
		}
		
		if (GraphNodeTemplateInfo.DragDropVariableTypes.Num() > 0)
		{
			for (const FAnimNextParamType& VariableType : GraphNodeTemplateInfo.DragDropVariableTypes)
			{
				TArray<FGraphNodeTemplateInfo>& Handlers = Private::GVariableDragDropHandlers.FindOrAdd(VariableType);
				Handlers.Add(GraphNodeTemplateInfo);
			}
		}

		Private::GAllTemplates.Add(ClassPath, MoveTemp(GraphNodeTemplateInfo));
	}

	Private::bGIsTemplateRegistryInitialized = true;
}

void FGraphNodeTemplateRegistry::Shutdown()
{
	if (!Private::bGIsTemplateRegistryInitialized)
	{
		return;
	}

	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetsAdded().Remove(Private::GOnAssetsAddedHandle);
		AssetRegistry->OnAssetsUpdated().Remove(Private::GOnAssetsUpdatedHandle);
		AssetRegistry->OnAssetsRemoved().Remove(Private::GOnAssetsRemovedHandle);
		AssetRegistry->OnAssetRenamed().Remove(Private::GOnAssetRenamedHandle);
	}

	FCoreUObjectDelegates::OnObjectPostCDOCompiled.Remove(Private::GOnObjectPostCDOCompiledHandle);

	Private::GAllTemplates.Empty();
	Private::GAssetDragDropHandlers.Empty();
	Private::GVariableDragDropHandlers.Empty();

	Private::bGIsTemplateRegistryInitialized = false;
}

const TMap<FTopLevelAssetPath, FGraphNodeTemplateInfo>& FGraphNodeTemplateRegistry::GetAllTemplates()
{
	LazyInitialize();
	return Private::GAllTemplates;
}

TConstArrayView<FGraphNodeTemplateInfo> FGraphNodeTemplateRegistry::GetDragDropHandlersForAsset(const FAssetData& InAssetData)
{
	LazyInitialize();
	if (TArray<FGraphNodeTemplateInfo>* FoundHandlers = Private::GAssetDragDropHandlers.Find(InAssetData.AssetClassPath))
	{
		return *FoundHandlers;
	}
	return TConstArrayView<FGraphNodeTemplateInfo>();
}

TConstArrayView<FGraphNodeTemplateInfo> FGraphNodeTemplateRegistry::GetDragDropHandlersForVariable(const FAnimNextParamType& InVariableType)
{
	LazyInitialize();
	if (TArray<FGraphNodeTemplateInfo>* FoundHandlers = Private::GVariableDragDropHandlers.Find(InVariableType))
	{
		return *FoundHandlers;
	}
	return TConstArrayView<FGraphNodeTemplateInfo>();
}

}
