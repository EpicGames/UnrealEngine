// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/UAFGraphNodeTemplate.h"

#include "AnimNextController.h"
#include "AnimNextTraitStackUnitNode.h"
#include "EditorUtils.h"
#include "UAFAnimGraphUncookedOnlyStyle.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate"

UUAFGraphNodeTemplate::UUAFGraphNodeTemplate()
{
	Title = GetClass()->GetDisplayNameText();
	TooltipText = GetClass()->GetDisplayNameText();
	Category = LOCTEXT("DefaultCategory", "UAF");
	MenuDescription = GetClass()->GetDisplayNameText();
	Icon = *FUAFAnimGraphUncookedOnlyStyle::Get().GetBrush(TEXT("NodeTemplate.DefaultIcon"));
}

bool UUAFGraphNodeTemplate::ConfigureNewNode_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node) const
{
	Controller->OpenUndoBracket(LOCTEXT("AddDefaultTraits", "Add Template Traits").ToString());

	int32 TraitIndex = 0;
	for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Traits)
	{
		FName NewTraitName = Controller->AddTraitStruct(Node, Trait, TraitIndex++, true, true);
		if (NewTraitName == NAME_None)
		{
			Controller->CancelUndoBracket();
			return false;
		}
	}

	FRigVMNodeLayout Layout = GetNodeLayout();
	if (Layout.IsValid())
	{
		Controller->SetNodeLayout(Node->GetFName(), Layout, true, true);
	}
	else
	{
		Controller->SetNodeLayout(Node->GetFName(), GetDefaultCategoryLayout(), true, true);
	}

	Controller->CloseUndoBracket();

	return true;
}

void UUAFGraphNodeTemplate::HandleAssetDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const
{
	Controller->OpenUndoBracket(LOCTEXT("ConfigureNodeOnDrop", "Configure Node On Drop").ToString());

	TArray<URigVMPin*> Pins = Node->GetAllPinsRecursively();
	for (URigVMPin* Pin : Pins)
	{
		if (Pin->HasMetaData(TEXT("FactorySource")))
		{
			if (Pin->IsArray())
			{
				Controller->InsertArrayPin(Pin->GetPinPath(), INDEX_NONE, Asset->GetPathName(), true);
			}
			else
			{
				Controller->SetPinDefaultValue(Pin, Asset->GetPathName(), true, true, true, true);
			}
			break;
		}
	}

	Controller->CloseUndoBracket();
}

void UUAFGraphNodeTemplate::HandleVariableDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, FAnimNextVariableReference Variable) const
{
}

FRigVMNodeLayout UUAFGraphNodeTemplate::GetDefaultCategoryLayout() const
{
	FRigVMNodeLayout Layout;
	TMap<FString, FString> DisplayNames;
	TMap<FString, int32> PinIndexInCategory;
	FRigVMPinCategory NewCategory;
	NewCategory.Path = FRigVMPinCategory::GetDefaultCategoryName();
	for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Traits)
	{
		if (const UScriptStruct* Struct = Trait.GetScriptStruct())
		{
			int32 PinIndex = 0;
			for (TFieldIterator<FProperty> It(Struct); It; ++It, ++PinIndex)
			{
				if (!It->HasMetaData("PinHiddenByDefault") &&
					!It->HasMetaData("Hidden"))
				{
					TStringBuilder<128> PinPathBuilder;
					PinPathBuilder.Append(Struct->GetName());
					PinPathBuilder.AppendChar(TEXT('.'));
					PinPathBuilder.Append(It->GetName());
					FString PinPath = PinPathBuilder.ToString();
					NewCategory.Elements.Add(PinPath);
					DisplayNames.Add(PinPath, It->GetDisplayNameText().ToString());
					PinIndexInCategory.Add(PinPath, PinIndex);
				}
			}
		}
	}

	if (NewCategory.Elements.Num() > 0)
	{
		Layout.Categories.Add(MoveTemp(NewCategory));
		Layout.PinIndexInCategory = MoveTemp(PinIndexInCategory);
		Layout.DisplayNames = MoveTemp(DisplayNames);
	}

	return Layout;
}

FRigVMNodeLayout UUAFGraphNodeTemplate::GetPerTraitCategoriesLayout() const
{
	FRigVMNodeLayout Layout;

	for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Traits)
	{
		if (const UScriptStruct* Struct = Trait.GetScriptStruct())
		{
			TMap<FString, FString> DisplayNames;
			TMap<FString, int32> PinIndexInCategory;
			FRigVMPinCategory NewCategory;
			NewCategory.Path = Struct->GetDisplayNameText().ToString();

			int32 PinIndex = 0;
			for (TFieldIterator<FProperty> It(Struct); It; ++It, ++PinIndex)
			{
				if (!It->HasMetaData("PinHiddenByDefault") &&
					!It->HasMetaData("Hidden"))
				{
					TStringBuilder<128> PinPathBuilder;
					PinPathBuilder.Append(Struct->GetName());
					PinPathBuilder.AppendChar(TEXT('.'));
					PinPathBuilder.Append(It->GetName());
					FString PinPath = PinPathBuilder.ToString();
					NewCategory.Elements.Add(PinPath);
					DisplayNames.Add(PinPath, It->GetDisplayNameText().ToString());
					PinIndexInCategory.Add(PinPath, PinIndex);
				}
			}

			if (NewCategory.Elements.Num() > 0)
			{
				Layout.Categories.Add(MoveTemp(NewCategory));
				Layout.PinIndexInCategory = MoveTemp(PinIndexInCategory);
				Layout.DisplayNames = MoveTemp(DisplayNames);
			}
		}
	}

	return Layout;
}

void UUAFGraphNodeTemplate::AddDefaultTraitPinsToLayout(const UScriptStruct* InStruct, FRigVMNodeLayout& InOutLayout)
{
	if (!InStruct->IsChildOf<FAnimNextTraitSharedData>())
	{
		return;
	}

	TMap<FString, FString> DisplayNames;
	TMap<FString, int32> PinIndexInCategory;
	FRigVMPinCategory NewCategory;
	NewCategory.Path = InStruct->GetDisplayNameText().ToString();

	int32 PinIndex = 0;
	for (TFieldIterator<FProperty> It(InStruct); It; ++It, ++PinIndex)
	{
		if (!It->HasMetaData("PinHiddenByDefault") &&
			!It->HasMetaData("Hidden"))
		{
			TStringBuilder<128> PinPathBuilder;
			PinPathBuilder.Append(InStruct->GetName());
			PinPathBuilder.AppendChar(TEXT('.'));
			PinPathBuilder.Append(It->GetName());
			FString PinPath = PinPathBuilder.ToString();
			NewCategory.Elements.Add(PinPath);
			DisplayNames.Add(PinPath, It->GetDisplayNameText().ToString());
			PinIndexInCategory.Add(PinPath, PinIndex);
		}
	}

	if (NewCategory.Elements.Num() > 0)
	{
		InOutLayout.Categories.Add(MoveTemp(NewCategory));
		InOutLayout.PinIndexInCategory.Append(PinIndexInCategory);
		InOutLayout.DisplayNames.Append(DisplayNames);
	}
}

void UUAFGraphNodeTemplate::SetDisplayNameForPinInLayout(FString PinPath, FString PinDisplayName, FRigVMNodeLayout& Layout)
{
	Layout.DisplayNames.Add(PinPath, PinDisplayName);
}

void UUAFGraphNodeTemplate::SetCategoryForPinsInLayout(const TArray<FString>& PinPaths, FString CategoryPath, FRigVMNodeLayout& Layout, bool bExpandedByDefault)
{
	bool bNewCategory = true;
	for (FRigVMPinCategory& Category : Layout.Categories)
	{
		if (Category.Path != CategoryPath)
		{
			// Remove pins from any existing categories
			for (const FString& PinPath : PinPaths)
			{
				Category.Elements.RemoveAll([&PinPath](const FString& InPinPath)
				{
					return InPinPath == PinPath;
				});
			}
		}
		else
		{
			bNewCategory = false;
			Category.bExpandedByDefault = bExpandedByDefault;

			// Add pins to category
			for (const FString& PinPath : PinPaths)
			{
				Category.Elements.AddUnique(PinPath);
			}
		}
	}

	if (bNewCategory)
	{
		FRigVMPinCategory& NewCategory = Layout.Categories.AddDefaulted_GetRef();
		NewCategory.Path = CategoryPath;
		NewCategory.bExpandedByDefault = bExpandedByDefault;

		// Add pins to category
		for (const FString& PinPath : PinPaths)
		{
			NewCategory.Elements.AddUnique(PinPath);
		}
	}
}

URigVMUnitNode* UUAFGraphNodeTemplate::CreateNewNode(UAnimNextController* Controller, const FVector2D& InLocation) const
{
	UAnimNextTraitStackUnitNode* TraitStackNode = Cast<UAnimNextTraitStackUnitNode>(Controller->AddUnitNodeOfClass(FRigUnit_AnimNextTraitStack::StaticStruct()->GetPathName(), UAnimNextTraitStackUnitNode::StaticClass()->GetPathName(), FRigVMStruct::ExecuteName, InLocation, GetName(), true, true));
	if (TraitStackNode == nullptr)
	{
		Controller->CancelUndoBracket();
		return nullptr;
	}

	TraitStackNode->Template = GetClass();
	Controller->SetNodeTitle(TraitStackNode, TraitStackNode->GetDefaultNodeTitle(), true, true, true);
	Controller->SetNodeColor(TraitStackNode, TraitStackNode->GetDefaultNodeColor(), true, true, true);
	
	{
		FEditorScriptExecutionGuard AllowScripts;
		ConfigureNewNode(Controller, TraitStackNode);
	}

	return TraitStackNode;
}

void UUAFGraphNodeTemplate::InitializeTemplateFromNode(const UAnimNextTraitStackUnitNode* InNode)
{
	Title = FText::FromString(InNode->GetNodeTitleRaw());
	SubTitle = FText::FromString(InNode->GetNodeSubTitle());
	TooltipText = InNode->GetToolTipText();
	Icon = *InNode->GetDefaultNodeIconBrush();
	Color = InNode->GetNodeColor();
	for (URigVMPin* TraitPin : InNode->GetTraitPins())
	{
		TSharedPtr<FStructOnScope> TraitScope = TraitPin->GetTraitInstance();
		if (!TraitScope.IsValid())
		{
			continue;
		}
		const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
		const UScriptStruct* TraitStruct = VMTrait->GetTraitSharedDataStruct();
		if (TraitStruct == nullptr)
		{
			continue;
		}
		TInstancedStruct<FAnimNextTraitSharedData> NewTrait;
		NewTrait.InitializeAsScriptStruct(TraitStruct);
		Traits.Add(MoveTemp(NewTrait));
	}
	NodeLayout = InNode->GetNodeLayout();

	if (InNode->Template)
	{
		UUAFGraphNodeTemplate* CDO = InNode->Template->GetDefaultObject<UUAFGraphNodeTemplate>();
		Category = CDO->Category;
		MenuDescription = CDO->MenuDescription;
		DragDropAssetTypes = CDO->DragDropAssetTypes;
	}
}

void UUAFGraphNodeTemplate::RefreshNodeFromTemplate(UAnimNextController* InController, UAnimNextTraitStackUnitNode* InNode) const
{
	InNode->Modify();

	InNode->Template = GetClass();

	InController->OpenUndoBracket(TEXT("Refresh Node From Template"));

	InController->SetNodeTitle(InNode, InNode->GetDefaultNodeTitle(), true, true, true);
	InController->SetNodeColor(InNode, InNode->GetDefaultNodeColor(), true, true, true);

	// Build a pin mapping to allow us to reconnect our pins after refreshing traits
	TMap<FString, FString> PinLinks;

	// Remove all current traits
	for (URigVMPin* TraitPin : InNode->GetTraitPins())
	{
		// Cache pin paths to try to relink later
		for (URigVMPin* SubPin : TraitPin->GetAllSubPinsRecursively())
		{
			if (SubPin->GetDirection() == ERigVMPinDirection::Input)
			{
				for (URigVMLink* Link : SubPin->GetLinks())
				{
					PinLinks.Add(Link->GetTargetPin()->GetPinPath(), Link->GetSourcePin()->GetPinPath());
				}
			}
		}

		InController->RemoveTrait(InNode, TraitPin->GetFName(), true);
	}

	// Add new traits
	int32 TraitIndex = 0;
	for (const TInstancedStruct<FAnimNextTraitSharedData>& Trait : Traits)
	{
		InController->AddTraitStruct(InNode, Trait, TraitIndex++, true, true);
	}

	// Relink any links we had
	if (PinLinks.Num() > 0)
	{
		for (const TPair<FString, FString>& LinkPair : PinLinks)
		{
			InController->AddLink(LinkPair.Value, LinkPair.Key, true, true);
		}
	}

	InController->SetNodeLayout(InNode->GetFName(), NodeLayout, true, true);

	InController->CloseUndoBracket();
}

#undef LOCTEXT_NAMESPACE
