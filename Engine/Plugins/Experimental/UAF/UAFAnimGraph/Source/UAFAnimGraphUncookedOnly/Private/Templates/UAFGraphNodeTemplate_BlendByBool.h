// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "Templates/GraphNodeColors.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Traits/BlendByBool.h"
#include "Traits/BlendSmoother.h"
#include "UAFGraphNodeTemplate_BlendByBool.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_BlendByBool"

UCLASS()
class UUAFGraphNodeTemplate_BlendByBool : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_BlendByBool()
	{
		Title = LOCTEXT("BlendByBoolTitle", "Blend By Bool");
		TooltipText = LOCTEXT("BlendByBoolTooltip", "Smoothly blends two input poses based on a bool input");
		Category = LOCTEXT("BlendByBoolCategory", "UAF");
		MenuDescription = LOCTEXT("BlendByBoolMenuDesc", "Blend By Bool");
		Color = UE::UAF::UncookedOnly::FGraphNodeColors::Blends;
		Traits =
		{
			TInstancedStruct<FAnimNextBlendByBoolTraitSharedData>::Make(),
			TInstancedStruct<FAnimNextBlendSmootherTraitSharedData>::Make()
		};
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendByBoolTraitSharedData, TrueChild),
				GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendByBoolTraitSharedData, FalseChild),
				GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendByBoolTraitSharedData, bCondition),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
		SetDisplayNameForPinInLayout(GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendByBoolTraitSharedData, TrueChild), TEXT("True"), NodeLayout);
		SetDisplayNameForPinInLayout(GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendByBoolTraitSharedData, FalseChild), TEXT("False"), NodeLayout);
	}
};

#undef LOCTEXT_NAMESPACE