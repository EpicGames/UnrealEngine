// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Traits/NotifyDispatcherTraitData.h"
#include "Traits/BlendSpacePlayerTraitData.h"
#include "UAFGraphNodeTemplate_BlendSpacePlayer.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_SequencePlayer"

UCLASS()
class UUAFGraphNodeTemplate_BlendSpacePlayer : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_BlendSpacePlayer()
	{
		Title = LOCTEXT("BlendSpacePlayerTitle", "Blend Space Player");
		TooltipText = LOCTEXT("BlendSpacePlayerTooltip", "Plays a Blend Space");
		Category = LOCTEXT("BlendSpacePlayerCategory", "UAF");
		MenuDescription = LOCTEXT("BlendSpacePlayerMenuDesc", "Blend Space Player");
		Color = FLinearColor(FColor(255, 168, 111));	// From UAssetDefinition_BlendSpace, but cannot use asset definition here
		Icon = *FSlateIconFinder::FindIconForClass(UBlendSpace::StaticClass()).GetIcon();
		Traits =
		{
			TInstancedStruct<UE::UAF::FBlendSpacePlayerData>::Make()
		};
		DragDropAssetTypes.Add(UBlendSpace::StaticClass());
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendSpacePlayerTraitSharedData, BlendSpace),
				GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendSpacePlayerTraitSharedData, XAxisSamplePoint),
				GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendSpacePlayerTraitSharedData, YAxisSamplePoint),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}

	virtual void HandleAssetDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const
	{
		Super::HandleAssetDropped_Implementation(Controller, Node, Asset);

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Asset)
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("ConfigureNodeOnDrop", "Configure Node On Drop").ToString());
			}

			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Play {0}"), FText::FromString(Asset->GetName())).ToString(), bSetupUndoRedo, true, true);

			// Set looping to asset configuration by default
			UBlendSpace* BlendSpace = CastChecked<UBlendSpace>(Asset);
			Controller->SetPinDefaultValue(TEXT("AnimNextBlendSpacePlayerTraitSharedData.bLoop"), LexToString(BlendSpace->bLoop), true, bSetupUndoRedo, true, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandlePinDefaultValueChanged_Implementation(UAnimNextController* Controller, URigVMPin* Pin) const
	{
		Super::HandlePinDefaultValueChanged_Implementation(Controller, Pin);

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Pin->GetFName() == GET_MEMBER_NAME_CHECKED(UE::UAF::FBlendSpacePlayerData, BlendSpace))
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("SetNodeTitle", "Set Node Title").ToString());
			}

			FSoftObjectPath Path(Pin->GetDefaultValue());
			Controller->SetNodeTitle(Pin->GetNode(), FText::Format(LOCTEXT("NodeTitleFormat", "Play {0}"), FText::FromString(Path.GetAssetName())).ToString(), bSetupUndoRedo, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandleAssetRenamed_Implementation(UAnimNextController* Controller, URigVMNode* Node, const FAssetData& AssetData, const FString& OldName) const
	{
		Super::HandleAssetRenamed_Implementation(Controller, Node, AssetData, OldName);

		// It the asset is not loaded, it cant be our asset so we can safely early out
		UObject* Object = AssetData.FastGetAsset();
		if (Object == nullptr)
		{
			return;
		}

		// NOTE: Not making this undo-able as this is an automated process, not a user action on the node specifically

		URigVMPin* Pin = Node->FindPin(GET_PIN_PATH_STRING_CHECKED(FAnimNextBlendSpacePlayerTraitSharedData, BlendSpace));
		if (Pin && Object == Pin->GetDefaultValueObject())
		{
			FSoftObjectPath Path(Object);
			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Play {0}"), FText::FromString(Path.GetAssetName())).ToString(), false, true, true);
		}
	}
};

#undef LOCTEXT_NAMESPACE