// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Injection/InjectionSiteTrait.h"
#include "Templates/GraphNodeColors.h"
#include "Traits/BlendSmoother.h"
#include "Traits/BlendStackTrait.h"
#include "UAFGraphNodeTemplate_InjectionSite.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_InjectionSite"

UCLASS()
class UUAFGraphNodeTemplate_InjectionSite : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_InjectionSite()
	{
		Title = LOCTEXT("InjectionSiteTitle", "Injection Site");
		TooltipText = LOCTEXT("InjectionSiteTooltip", "Allows external animation logic and content to be inserted into an animation graph");
		Category = LOCTEXT("InjectionSiteCategory", "UAF");
		MenuDescription = LOCTEXT("InjectionSiteMenuDesc", "Injection Site");
		Color = Color = UE::UAF::UncookedOnly::FGraphNodeColors::SubGraphs;
		Traits =
		{
			TInstancedStruct<UE::UAF::FBlendStackCoreTraitData>::Make(),
			TInstancedStruct<UE::UAF::FBlendSmootherCoreData>::Make(),
			TInstancedStruct<UE::UAF::FInjectionSiteTraitData>::Make()
		};
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FAnimNextInjectionSiteTraitSharedData, Source),
				GET_PIN_PATH_STRING_CHECKED(FAnimNextInjectionSiteTraitSharedData, Graph),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}
};

#undef LOCTEXT_NAMESPACE