// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Templates/GraphNodeColors.h"
#include "Traits/InlineSubGraphTraitData.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "UAFGraphNodeTemplate_InlineSubGraph.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_InlineSubGraph"

UCLASS()
class UUAFGraphNodeTemplate_InlineSubGraph : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_InlineSubGraph()
	{
		Title = LOCTEXT("InlineSubGraphTitle", "Inline SubGraph");
		TooltipText = LOCTEXT("InlineSubGraphTooltip", "Hosts an inner sub-graph and wires child nodes from this graph into the inner sub-graph's inputs");
		Category = LOCTEXT("InlineSubGraphCategory", "UAF");
		MenuDescription = LOCTEXT("InlineSubGraphMenuDesc", "Inline SubGraph");
		Color = UE::UAF::UncookedOnly::FGraphNodeColors::SubGraphs;
		Icon = *FSlateIconFinder::FindIconForClass(UUAFAnimGraph::StaticClass()).GetIcon();
		Traits =
		{
			TInstancedStruct<FUAFInlineSubGraphTraitSharedData>::Make()
		};
		DragDropAssetTypes.Add(UUAFAnimGraph::StaticClass());
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FUAFInlineSubGraphTraitSharedData, Graph),
				GET_PIN_PATH_STRING_CHECKED(FUAFInlineSubGraphTraitSharedData, Inputs),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}

	virtual void HandleAssetDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const
	{
		const bool bSetupUndoRedo = !GIsTransacting;
		if (Asset)
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("ConfigureNodeOnDrop", "Configure Node On Drop").ToString());
			}

			Controller->SetPinDefaultValue(
				GET_PIN_PATH_STRING_CHECKED(FUAFInlineSubGraphTraitSharedData, Graph),
				Asset->GetPathName(), true, bSetupUndoRedo, true, true, true);

			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Run {0}"), FText::FromString(Asset->GetName())).ToString(), bSetupUndoRedo, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandlePinDefaultValueChanged_Implementation(UAnimNextController* Controller, URigVMPin* Pin) const
	{
		Super::HandlePinDefaultValueChanged_Implementation(Controller, Pin);

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Pin->GetFName() == GET_MEMBER_NAME_CHECKED(FUAFInlineSubGraphTraitSharedData, Graph))
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("SetNodeTitle", "Set Node Title").ToString());
			}

			FSoftObjectPath Path(Pin->GetDefaultValue());
			Controller->SetNodeTitle(Pin->GetNode(), FText::Format(LOCTEXT("NodeTitleFormat", "Run {0}"), FText::FromString(Path.GetAssetName())).ToString(), bSetupUndoRedo, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandleAssetRenamed_Implementation(UAnimNextController* Controller, URigVMNode* Node, const FAssetData& AssetData, const FString& OldName) const
	{
		Super::HandleAssetRenamed_Implementation(Controller, Node, AssetData, OldName);

		// If the asset is not loaded, it can't be our asset so we can safely early out
		UObject* Object = AssetData.FastGetAsset();
		if (Object == nullptr)
		{
			return;
		}

		// NOTE: Not making this undo-able as this is an automated process, not a user action on the node specifically

		URigVMPin* Pin = Node->FindPin(GET_PIN_PATH_STRING_CHECKED(FUAFInlineSubGraphTraitSharedData, Graph));
		if (Pin && Object == Pin->GetDefaultValueObject())
		{
			FSoftObjectPath Path(Object);
			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Run {0}"), FText::FromString(Path.GetAssetName())).ToString(), false, true, true);
		}
	}
};

#undef LOCTEXT_NAMESPACE
