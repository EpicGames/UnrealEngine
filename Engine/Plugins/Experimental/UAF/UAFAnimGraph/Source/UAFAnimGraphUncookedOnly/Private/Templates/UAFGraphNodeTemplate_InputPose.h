// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "AssetDefinition.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Traits/InputValueTrait.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAFGraphNodeTemplate_InputPose.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_InputValue"

UCLASS()
class UUAFGraphNodeTemplate_InputValue : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_InputValue()
	{
		Title = LOCTEXT("InputValueTitle", "Input Value");
		TooltipText = LOCTEXT("InputValueTooltip", "An input value bundle from an external source");
		Category = LOCTEXT("InputValueCategory", "UAF");
		MenuDescription = LOCTEXT("InputValueMenuDesc", "Input Value");
		Color = FLinearColor(FColor(80, 10, 123));
		Traits =
		{
			TInstancedStruct<FUAFInputValueTraitSharedData>::Make()
		};
		DragDropVariableTypes.Add(FAnimNextParamType::GetType<FUAFValueBundle>());
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FUAFInputValueTraitSharedData, Input),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}

	virtual void HandleVariableDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, FAnimNextVariableReference Variable) const
	{
		if (Variable.IsNone())
		{
			return;
		}

		Controller->OpenUndoBracket(LOCTEXT("ConfigureNodeOnDrop", "Configure Node On Drop").ToString());

		URigVMPin* Pin = Node->FindPin(TEXT("UAFInputValueTraitSharedData.Input"));
		if (ensure(Pin))
		{
			FString DefaultValue;
			FAnimNextVariableReference::StaticStruct()->ExportText(DefaultValue, &Variable, nullptr, nullptr, PPF_None, nullptr);
			Controller->SetPinDefaultValue(Pin, DefaultValue, true, true, true, true);
			
			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Input Value: {0}"), FText::FromName(Variable.GetName())).ToString(), true, true, true);
		}

		Controller->CloseUndoBracket();
	}

	virtual void HandlePinDefaultValueChanged_Implementation(UAnimNextController* Controller, URigVMPin* Pin) const
	{
		Super::HandlePinDefaultValueChanged_Implementation(Controller, Pin);

		if (Pin->GetFName() == GET_MEMBER_NAME_CHECKED(FUAFInputValueTraitSharedData, Input))
		{
			Controller->OpenUndoBracket(LOCTEXT("SetNodeTitle", "Set Node Title").ToString());

			FString DefaultValue = Pin->GetDefaultValue();
			FAnimNextVariableReference VariableReference;
			FAnimNextVariableReference::StaticStruct()->ImportText(*DefaultValue, &VariableReference, nullptr, PPF_None, nullptr, FAnimNextVariableReference::StaticStruct()->GetName());
			Controller->SetNodeTitle(Pin->GetNode(), FText::Format(LOCTEXT("NodeTitleFormat", "Input Value: {0}"), FText::FromName(VariableReference.GetName())).ToString(), true, true, true);

			Controller->CloseUndoBracket();
		}
	}
};

#undef LOCTEXT_NAMESPACE