// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextController.h"
#include "AssetDefinition.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "Traits/NotifyDispatcherTraitData.h"
#include "Traits/SequencePlayerTraitData.h"
#include "UAFGraphNodeTemplate_SequencePlayer.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_SequencePlayer"

UCLASS()
class UUAFGraphNodeTemplate_SequencePlayer : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_SequencePlayer()
	{
		Title = LOCTEXT("SequencePlayerTitle", "Sequence Player");
		TooltipText = LOCTEXT("SequencePlayerTooltip", "Plays an animation sequence");
		Category = LOCTEXT("SequencePlayerCategory", "UAF");
		MenuDescription = LOCTEXT("SequencePlayerMenuDesc", "Sequence Player");
		Color = FLinearColor(FColor(80, 123, 72));	// From UAssetDefinition_AnimationAsset, but cannot use asset definition here
		Icon = *FSlateIconFinder::FindIconForClass(UAnimSequence::StaticClass()).GetIcon();
		Traits =
		{
			TInstancedStruct<UE::UAF::FSequencePlayerData>::Make(),
			TInstancedStruct<UE::UAF::FNotifyDispatcherData>::Make()
		};
		DragDropAssetTypes.Add(UAnimSequence::StaticClass());
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FAnimNextSequencePlayerTraitSharedData, AnimSequence),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}

	virtual void HandleAssetDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const
	{
		Super::HandleAssetDropped_Implementation(Controller, Node, Asset);

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Asset)
		{
			if(bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("ConfigureNodeOnDrop", "Configure Node On Drop").ToString());
			}

			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Play {0}"), FText::FromString(Asset->GetName())).ToString(), bSetupUndoRedo, true, true);

			if(bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandlePinDefaultValueChanged_Implementation(UAnimNextController* Controller, URigVMPin* Pin) const
	{
		Super::HandlePinDefaultValueChanged_Implementation(Controller, Pin);

		const bool bSetupUndoRedo = !GIsTransacting;
		if (Pin->GetFName() == GET_MEMBER_NAME_CHECKED(UE::UAF::FSequencePlayerData, AnimSequence))
		{
			if (bSetupUndoRedo)
			{
				Controller->OpenUndoBracket(LOCTEXT("SetNodeTitle", "Set Node Title").ToString());
			}

			FSoftObjectPath Path(Pin->GetDefaultValue());
			Controller->SetNodeTitle(Pin->GetNode(), FText::Format(LOCTEXT("NodeTitleFormat", "Play {0}"), FText::FromString(Path.GetAssetName())).ToString(), bSetupUndoRedo, true, true);

			if (bSetupUndoRedo)
			{
				Controller->CloseUndoBracket();
			}
		}
	}

	virtual void HandleAssetRenamed_Implementation(UAnimNextController* Controller, URigVMNode* Node, const FAssetData& AssetData, const FString& OldName) const
	{
		Super::HandleAssetRenamed_Implementation(Controller, Node, AssetData, OldName);

		// It the asset is not loaded, it cant be our asset so we can safely early out
		UObject* Object = AssetData.FastGetAsset();
		if (Object == nullptr)
		{
			return;
		}

		// NOTE: Not making this undo-able as this is an automated process, not a user action on the node specifically

		URigVMPin* Pin = Node->FindPin(GET_PIN_PATH_STRING_CHECKED(FAnimNextSequencePlayerTraitSharedData, AnimSequence));
		if (Pin && Object == Pin->GetDefaultValueObject())
		{
			FSoftObjectPath Path(Object);
			Controller->SetNodeTitle(Node, FText::Format(LOCTEXT("NodeTitleFormat", "Play {0}"), FText::FromString(Path.GetAssetName())).ToString(), false, true, true);
		}
	}
};

#undef LOCTEXT_NAMESPACE