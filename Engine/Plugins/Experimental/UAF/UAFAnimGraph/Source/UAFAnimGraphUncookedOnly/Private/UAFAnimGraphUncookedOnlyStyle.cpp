// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimGraphUncookedOnlyStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"

FUAFAnimGraphUncookedOnlyStyle::FUAFAnimGraphUncookedOnlyStyle()
	: FSlateStyleSet(TEXT("UAFAnimGraphUncookedOnlyStyle"))
{
	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("NodeTemplate.DefaultIcon", new IMAGE_BRUSH_SVG("Starship/Common/Animation", FVector2f(16.0f, 16.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FUAFAnimGraphUncookedOnlyStyle::~FUAFAnimGraphUncookedOnlyStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FUAFAnimGraphUncookedOnlyStyle& FUAFAnimGraphUncookedOnlyStyle::Get()
{
	static FUAFAnimGraphUncookedOnlyStyle Instance;
	return Instance;
}

FString FUAFAnimGraphUncookedOnlyStyle::InResources(const FString& RelativePath)
{
	static const FString ResourcesDir = IPluginManager::Get().FindPlugin(TEXT("UAFAnimGraph"))->GetBaseDir() / TEXT("Resources");
	return ResourcesDir / RelativePath;
}