// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FUAFAnimGraphUncookedOnlyStyle final : public FSlateStyleSet
{
public:
	static FUAFAnimGraphUncookedOnlyStyle& Get();

private:
	FUAFAnimGraphUncookedOnlyStyle();
	~FUAFAnimGraphUncookedOnlyStyle();

	// Resolve path relative to plugin's resources directory
	static FString InResources(const FString& RelativePath);
};

