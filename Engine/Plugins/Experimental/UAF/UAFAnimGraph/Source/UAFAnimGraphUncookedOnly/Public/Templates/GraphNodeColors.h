// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

namespace UE::UAF::UncookedOnly
{

struct FGraphNodeColors
{
	static UE_API FLinearColor Blends;
	static UE_API FLinearColor Controls;
	static UE_API FLinearColor SubGraphs;
	static UE_API FLinearColor Generators;
};

}

#undef UE_API