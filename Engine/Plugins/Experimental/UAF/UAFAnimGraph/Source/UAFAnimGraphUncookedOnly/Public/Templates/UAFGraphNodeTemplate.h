// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateNoResource.h"
#include "StructUtils/InstancedStruct.h"
#include "RigVMCore/RigVMNodeLayout.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleDefaults.h"
#include "Textures/SlateIcon.h"
#include "UAFGraphNodeTemplate.generated.h"

struct FAnimNextParamType;
struct FAnimNextTraitSharedData;
class UAnimNextTraitStackUnitNode;
class URigVMController;
class UAnimNextController;

namespace UE::UAF
{
	class FGraphNodeTemplateRegistry;
}

namespace UE::UAF::Editor
{
	class FGraphNodeTemplateDetails;
}

#define UE_API UAFANIMGRAPHUNCOOKEDONLY_API

// A template defining the appearance and behavior of a node in an animation graph
// ~This class is not instanced, rather its functions are called on its default object
UCLASS(MinimalAPI, Blueprintable)
class UUAFGraphNodeTemplate : public UObject
{
	GENERATED_BODY()

public:
	UE_API UUAFGraphNodeTemplate();

	// Get the title of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FText GetTitle() const;
	virtual FText GetTitle_Implementation() const
	{
		return Title;
	}

	// Get the subtitle of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FText GetSubTitle() const;
	virtual FText GetSubTitle_Implementation() const
	{
		return SubTitle;
	}

	// Get the tooltip text of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FText GetTooltipText() const;
	virtual FText GetTooltipText_Implementation() const
	{
		return TooltipText;
	}

	// Get the category of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FText GetCategory() const;
	virtual FText GetCategory_Implementation() const
	{
		return Category;
	}

	// Get the menu description of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FText GetMenuDescription() const;
	virtual FText GetMenuDescription_Implementation() const
	{
		return MenuDescription;
	}

	// Get the icon of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FSlateBrush GetIcon() const;
	FSlateBrush GetIcon_Implementation() const
	{
		return Icon;
	}

	// Get the icon of this node
	const FSlateBrush* GetIconBrush() const
	{
		CachedIcon = GetIcon();
		return &CachedIcon;
	}

	// Get the title color of this node
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FLinearColor GetColor() const;
	virtual FLinearColor GetColor_Implementation() const
	{
		return Color;
	}

	// Called in editor when a node is first spawned
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API bool ConfigureNewNode(UAnimNextController* Controller, URigVMUnitNode* Node) const;
	UE_API virtual bool ConfigureNewNode_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node) const;

	// Get the node layout
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API FRigVMNodeLayout GetNodeLayout() const;
	virtual FRigVMNodeLayout GetNodeLayout_Implementation() const
	{
		return NodeLayout;
	}

	// Called in editor when an asset is drag/dropped. Called after ConfigureNewNode.
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API void HandleAssetDropped(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const;
	UE_API virtual void HandleAssetDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, UObject* Asset) const;

	// Called in editor when a variable is drag/dropped. Called after ConfigureNewNode.
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API void HandleVariableDropped(UAnimNextController* Controller, URigVMUnitNode* Node, FAnimNextVariableReference Variable) const;
	UE_API virtual void HandleVariableDropped_Implementation(UAnimNextController* Controller, URigVMUnitNode* Node, FAnimNextVariableReference Variable) const;
	
	// Called in editor when a pin default value is changed, e.g. to be used to reconfigure the nodes appearance
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API void HandlePinDefaultValueChanged(UAnimNextController* Controller, URigVMPin* Pin) const;
	virtual void HandlePinDefaultValueChanged_Implementation(UAnimNextController* Controller, URigVMPin* Pin) const {}

	// Called in editor when an asset is renamed, e.g. to be used to reconfigure the nodes appearance
	UFUNCTION(BlueprintNativeEvent, Category = "UAF|Template")
	UE_API void HandleAssetRenamed(UAnimNextController* Controller, URigVMNode* Node, const FAssetData& AssetData, const FString& OldName) const;
	virtual void HandleAssetRenamed_Implementation(UAnimNextController* Controller, URigVMNode* Node, const FAssetData& AssetData, const FString& OldName) const {}

	// Get a layout that assigns all trait pins to the default category
	UFUNCTION(BlueprintCallable, Category = "UAF|Template")
	UE_API FRigVMNodeLayout GetDefaultCategoryLayout() const;

	// Get a layout that assigns all trait pins categories of their traits
	UFUNCTION(BlueprintCallable, Category = "UAF|Template")
	UE_API FRigVMNodeLayout GetPerTraitCategoriesLayout() const;

	// Sets the display name for a supplied pin in a node layout
	UFUNCTION(BlueprintCallable, Category = "UAF|Template")
	static UE_API void SetDisplayNameForPinInLayout(FString PinPath, FString PinDisplayName, FRigVMNodeLayout& Layout);

	// Sets the category of the supplied pins in a node layout
	UFUNCTION(BlueprintCallable, Category = "UAF|Template")
	static UE_API void SetCategoryForPinsInLayout(const TArray<FString>& PinPaths, FString CategoryPath, FRigVMNodeLayout& Layout, bool bExpandedByDefault = true);

	// Get all the types of asset this template handles for drag-drop
	TConstArrayView<TSubclassOf<UObject>> GetDragDropAssetTypes() const
	{
		return DragDropAssetTypes;
	}
	
	// Get all the types of variable this template handles for drag-drop
	TConstArrayView<FAnimNextParamType> GetDragDropVariableTypes() const
	{
		return DragDropVariableTypes;
	}
	
	// Utility function used to create and configure a new node according to a template
	UE_API URigVMUnitNode* CreateNewNode(UAnimNextController* Controller, const FVector2D& InLocation) const;

	// Utility function used to initialize this template from an existing trait stack node
	UE_API void InitializeTemplateFromNode(const UAnimNextTraitStackUnitNode* InNode);

	// Utility function used to refresh an existing node from this templates
	UE_API void RefreshNodeFromTemplate(UAnimNextController* InController, UAnimNextTraitStackUnitNode* InNode) const;

	// Utility function used to add all of a trait's pins to a category, named according to their display name
	UE_API static void AddDefaultTraitPinsToLayout(const UScriptStruct* Struct, FRigVMNodeLayout& InOutLayout);

protected:
	friend UE::UAF::FGraphNodeTemplateRegistry;
	friend UE::UAF::Editor::FGraphNodeTemplateDetails;

	// The default title of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Template")
	FText Title;

	// The default subtitle of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Template")
	FText SubTitle;

	// The default tooltip text of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AssetRegistrySearchable, Category="Template", meta=(MultiLine))
	FText TooltipText;

	// The default category of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AssetRegistrySearchable, Category="Template")
	FText Category;

	// The default menu description of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AssetRegistrySearchable, Category="Template")
	FText MenuDescription;

	// The icon of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AssetRegistrySearchable, Category="Template")
	FSlateBrush Icon;

	// Cached icon for the node
	UPROPERTY()
	mutable FSlateBrush CachedIcon;

	// The default color of the node
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Template")
	FLinearColor Color = FLinearColor::Gray;

	// The traits that we will instantiate
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Template")
	TArray<TInstancedStruct<FAnimNextTraitSharedData>> Traits;

	// Asset types that we handle for drag-drop
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AssetRegistrySearchable, Category="Template")
	TArray<TSubclassOf<UObject>> DragDropAssetTypes;

	// Variable types that we handle for drag-drop
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AssetRegistrySearchable, Category="Template")
	TArray<FAnimNextParamType> DragDropVariableTypes;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Template")
	FRigVMNodeLayout NodeLayout;
};

#undef UE_API
