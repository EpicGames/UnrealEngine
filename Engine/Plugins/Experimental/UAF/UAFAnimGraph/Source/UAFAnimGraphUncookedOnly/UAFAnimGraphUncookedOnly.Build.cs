// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFAnimGraphUncookedOnly : ModuleRules
	{
		public UAFAnimGraphUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"SlateCore"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"UAF",
					"UAFUncookedOnly",
					"UAFAnimGraph",
					"RigVMDeveloper",
					"BlueprintGraph",	// For K2 schema
					"AnimGraph",
					"RigVM",
					"Slate",
					"AssetDefinition",
					"Projects",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"BlueprintGraph",
						"WorkspaceEditor",
						"UAFEditor",
						"RigVMEditor",
					}
				);
			}
		}
	}
}