{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UAF Anim Graph",
	"Description" : "Framework for defining animation graphs.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "UAFAnimGraph",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "UAFAnimGraphEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFAnimGraphUncookedOnly",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		}
	]
}
